// Copyright Epic Games, Inc. All Rights Reserved.

#include "SubmitToolOutputLog.h"

void SSubmitToolOutputLog::ScrollToEnd()
{
	RequestForceScroll(false);
}
