// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SOutputLog.h"

class SSubmitToolOutputLog : public SOutputLog
{
public:
	using FArguments = SOutputLog::FArguments;

	void ScrollToEnd();
};
