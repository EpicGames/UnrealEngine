// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/IConsoleManager.h"
#include "Misc/OutputDevice.h"
#include "Logging/SubmitToolLog.h"
#include "SOutputLog.h"

class FSubmitToolOutputLogHistory : public FOutputDevice
{
public:
	FSubmitToolOutputLogHistory()
	{
		GLog->AddOutputDevice(this);
	}

	virtual ~FSubmitToolOutputLogHistory()
	{
		Release();
	}

	void Release()
	{
		if (GLog != nullptr)
		{
			GLog->RemoveOutputDevice(this);
		}
	}

	FSubmitToolOutputLogHistory(const FSubmitToolOutputLogHistory&) = delete;
	FSubmitToolOutputLogHistory(FSubmitToolOutputLogHistory&&) = delete;

	const TArray<TSharedPtr<FOutputLogMessage>>& GetMessages() const
	{
		return Messages;
	}

private:
	virtual void Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const FName& Category) override
	{
		SOutputLog::CreateLogMessages(V, Verbosity, Category, Messages);
	}

	TArray<TSharedPtr<FOutputLogMessage>> Messages;
};
