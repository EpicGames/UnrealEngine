// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChangelistService.h"

#include "Algo/Compare.h"
#include "Algo/Find.h"
#include "Configuration/Configuration.h"
#include "Logging/SubmitToolLog.h"
#include "Logic/Services/Interfaces/ICacheDataService.h"
#include "Logic/Services/Interfaces/ISTSourceControlService.h"
#include "Models/ModelInterface.h"
#include "Models/SubmitToolUserPrefs.h"
#include "HAL/FileManagerGeneric.h"
#include "CommandLine/CmdLineParameters.h"
#include "SubmitToolUtils.h"
#include "SubmitToolCoreUtils.h"

FChangelistService::FChangelistService(
	const FGeneralParameters& InParameters,
	const TSharedPtr<FSubmitToolServiceProvider> InServiceProvider,
	const FOnChangeListReadyDelegate& InCLReadyCallback,
	const FOnChangelistRefreshDelegate& InCLRefreshCallback) :
	Parameters(InParameters),
	CLReadyCallback(InCLReadyCallback),
	CLRefreshCallback(InCLRefreshCallback),
	TickHandle(FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateRaw(this, &FChangelistService::P4Tick))),
	ServiceProvider(InServiceProvider),
	SourceControlService(InServiceProvider->GetService<ISTSourceControlService>())
{
	FCmdLineParameters::Get().GetValue(FSubmitToolCmdLine::P4ChangeList, CLID);
	Init();
}

FChangelistService::~FChangelistService()
{
	OnCLDescriptionUpdated.Clear();
}

void FChangelistService::Init()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FChangelistService::Init);
	if (SourceControlService.IsValid())
	{
		if (!CLID.IsNumeric() && CLID.Equals(TEXT("default"), ESearchCase::IgnoreCase))
		{
			CreateCLFromDefaultCL();
		}
		else
		{
			FetchChangelistDataAsync();
		}
	}
	else
	{
		UE_LOGF(LogSubmitToolP4, Error, "Perforce Connection was invalid");
	}
}

void FChangelistService::FetchChangelistDataAsync()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FChangelistService::FetchChangelistDataAsync);
	UE_LOGF(LogSubmitToolP4Debug, Log, "Fetching changelist");

	// Wait for any already running tasks
	WaitForChangelistData();

	// First, launch two "opened" and "describe" commands in parallel
	OpenedTask = SourceControlService->RunCommand(TEXT("opened"), { });
	DescribeTask = SourceControlService->RunCommand(TEXT("describe"), { "-S", CLID });

	// Then, wait for both before running an additional "where" command which
	// is required to build a complete representation of the changelist state.
	ParseChangelistTask = UE::Tasks::Launch(UE_SOURCE_LOCATION, [this]
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FChangelistService::ParseChangelistTask);

		check(OpenedTask.IsCompleted())
		check(DescribeTask.IsCompleted());

		TSharedRef<FSCCommand> OpenedCmd = OpenedTask.GetResult();
		TSharedRef<FSCCommand> DescribeCmd = DescribeTask.GetResult();

		if (!OpenedCmd->bSuccess || !DescribeCmd->bSuccess)
		{
			return FChangelistData{};
		}

		// Do a first quick parse of the fetched data in order to
		// collect the set of open and shelved depot paths in the changelist.
		TSet<FString> DepotFiles;
		for (const TMap<FString, FString>& Record : OpenedCmd->Values)
		{
			const FString* DepotFile = Record.Find(TEXT("depotFile"));
			const FString* Change = Record.Find(TEXT("change"));
			if (DepotFile && Change && *Change == CLID)
			{
				DepotFiles.Add(*DepotFile);
			}
		}
		for (const TMap<FString, FString>& Record : DescribeCmd->Values)
		{
			int32 I = 0;
			while (const FString* DepotFile = Record.Find(FString::Printf(TEXT("depotFile%d"), I)))
			{
				DepotFiles.Add(*DepotFile);
				++I;
			}
		}
		
		// Run a sync "where" query inside this task using the collected depot paths to 
		// build a mapping to local filenames
		TMap<FString, FString> DepotFileToLocalFile = RunWhereCommandSync(DepotFiles.Array());

		return FChangelistData::Parse(CLID, *DescribeCmd, *OpenedCmd, DepotFileToLocalFile);
	}, Prerequisites(OpenedTask, DescribeTask));

	// Last, launch a GT task to update the actual model data
	UpdateChangelistTask = UE::Tasks::Launch(UE_SOURCE_LOCATION, [this]
	{
		UE_LOGF(LogSubmitToolP4Debug, Log, "Fetch changelist operation succeeded.");

		if (Changelist.bIsValid)
		{
			UpdateCachedChangelistData();
		}
		else
		{
			CacheChangelistData();
		}
	}, ParseChangelistTask, LowLevelTasks::ETaskPriority::Normal, UE::Tasks::EExtendedTaskPriority::GameThreadNormalPri);
}

void FChangelistService::RevertUnchangedFiles(TFunction<void(const FSCCommand&)> OnComplete)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FChangelistService::RevertUnchangedFiles);
	UE_LOGF(LogSubmitToolP4, Log, "Reverting unchanged files from CL %ls...", *GetCLID());
	
	SourceControlService->RunCommand(TEXT("revert"), { TEXT("-a") , TEXT("-c"), GetCLID() }, FOnSCCCommandComplete::CreateLambda(
		[OnComplete, this](const FSCCommand& Cmd)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FChangelistService::RevertUnchangedFiles_Callback);
			if (Cmd.bSuccess)
			{
				for (const FSCCRecord& Record : Cmd.Values)
				{
				}
				
				UE_LOGF(LogSubmitToolP4, Log, "CL %ls Revert unchanged operation succeeded", *GetCLID());
			}
			else
			{
				UE_LOGF(LogSubmitToolP4, Error, "CL %ls Revert unchanged operation failed:", *GetCLID());
			}

			if (OnComplete.IsSet())
			{
				OnComplete(Cmd);
			}
		}
	));
}


void FChangelistService::UpdateP4CLDescriptionSynchronously()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FChangelistService::UpdateP4CLDescriptionSynchronously);
	if (!CLDescription.Equals(Changelist.Description))
	{
		UE_LOGF(LogSubmitToolP4Debug, Log, "Saving CL description %ls", *GetCLID());
		FSCCommand OutputCmd = SourceControlService->RunSyncCommand(TEXT("change"), { TEXT("-o"), GetCLID() });

		if (OutputCmd.bSuccess && OutputCmd.Values.Num() > 0)
		{
			FSCCommand ChangeCmd = FSCCommand(TEXT("change"), { TEXT("-i") });
			ChangeCmd.InputData = ParseP4ChangesOutput(OutputCmd.Values[0], CLDescription);
			SourceControlService->RunSyncCommand(ChangeCmd);

			if (ChangeCmd.bSuccess)
			{
				UE_LOGF(LogSubmitToolP4, Log, "CL %ls edit changelist operation succeeded.", *GetCLID());
				Changelist.Description = CLDescription;
			}
			else
			{
				UE_LOGF(LogSubmitToolP4, Error, "CL %ls edit changelist operation failed", *GetCLID());
			}
		}
	}
}

void FChangelistService::UpdateP4CLDescription()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FChangelistService::UpdateP4CLDescription);

	if (!CLDescription.Equals(Changelist.Description))
	{
		UE_LOGF(LogSubmitToolP4Debug, Log, "Saving CL description %ls", *GetCLID());
		
		SourceControlService->RunCommand(TEXT("change"), { TEXT("-o"), GetCLID() }, FOnSCCCommandComplete::CreateLambda([this](const FSCCommand& Cmd) {
			TRACE_CPUPROFILER_EVENT_SCOPE(FChangelistService::UpdateP4CLDescription_change_output_callback);
			if (Cmd.bSuccess && Cmd.Values.Num() > 0)
			{
				FSCCommand ChangeCmd = FSCCommand(TEXT("change"), { TEXT("-i") });
				ChangeCmd.InputData = ParseP4ChangesOutput(Cmd.Values[0], CLDescription);
				SourceControlService->RunCommand(MoveTemp(ChangeCmd), FOnSCCCommandComplete::CreateLambda([this](const FSCCommand& Cmd) {
					TRACE_CPUPROFILER_EVENT_SCOPE(FChangelistService::UpdateP4CLDescription_change_input_callback);
					if (Cmd.bSuccess)
					{
						UE_LOGF(LogSubmitToolP4, Log, "CL %ls edit changelist operation succeeded.", *GetCLID());
						Changelist.Description = CLDescription;
					}
					else
					{
						UE_LOGF(LogSubmitToolP4, Error, "CL %ls edit changelist operation failed", *GetCLID());
					}
				}));
			}
			else
			{
				UE_LOGF(LogSubmitToolP4, Error, "CL %ls edit changelist operation failed", *GetCLID());

			}
		}));		
	}
}

static ESCFileState GetLocalFileState(const FString& State)
{
	if (State == TEXT("edit"))
	{
		return ESCFileState::Edit;
	}
	if (State == TEXT("openforadd") || State == TEXT("move/add") || State == TEXT("add"))
	{
		return ESCFileState::Add;
	}
	if (State == TEXT("markfordelete") || State == TEXT("move/delete") || State == TEXT("delete"))
	{
		return ESCFileState::Delete;
	}
	if (State == TEXT("integrate") || State == TEXT("branch"))
	{
		return ESCFileState::Integrate;
	}

	// Unknown State
	ensureMsgf(false, TEXT("GetLocalFileState: Unknown perforce file state: '%s'"), *State);
	return ESCFileState::Unknown;
}

FChangelistData FChangelistData::Parse(
	const FString& CLID,
	const FSCCommand& DescribeCmd,
	const FSCCommand& OpenedCmd,
	const TMap<FString, FString>& DepotFileToLocalFile)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FChangelistData::ParseChangelistCommands);

	FChangelistData Out;

	if (!OpenedCmd.bSuccess || !DescribeCmd.bSuccess)
	{
		return Out;
	}

	bool bFoundDescription = false;

	// Opened files
	for (const TMap<FString, FString>& Record : OpenedCmd.Values)
	{
		const FString* DepotFile = Record.Find(TEXT("depotFile"));
		const FString* Change = Record.Find(TEXT("change"));
		const FString* Action = Record.Find(TEXT("action"));
		if (DepotFile && Change && Action)
		{
			if (*Change == CLID)
			{
				Out.Files.Add(MakeShared<FSCFile>(DepotFileToLocalFile.FindRef(*DepotFile), *DepotFile, GetLocalFileState(*Action)));
			}
			Out.AllOpenFiles.Add(*DepotFile, *Change);
		}
	}

	// Describe => Shelved files
	for (const TMap<FString, FString>& Record : DescribeCmd.Values)
	{
		if (const FString* Description = Record.Find(TEXT("desc")))
		{
			Out.Description = *Description;
			bFoundDescription = true;
		}
		int32 I = 0;
		while (const FString* DepotFile = Record.Find(FString::Printf(TEXT("depotFile%d"), I)))
		{
			const FString* Action = Record.Find(FString::Printf(TEXT("action%d"), I));
			if (Action)
			{
				Out.ShelvedFiles.Add(MakeShared<FSCFile>(DepotFileToLocalFile.FindRef(*DepotFile), *DepotFile, GetLocalFileState(*Action)));
			}
			++I;
		}
	}

	// Sort and compare open vs. shelved files
	Algo::StableSortBy(Out.Files, [](const FSCFileRef& File) { return File->GetDepotPath(); });
	Algo::StableSortBy(Out.ShelvedFiles, [](const FSCFileRef& File) { return File->GetDepotPath(); });
	Out.bIsShelfComplete = Algo::Compare(Out.Files, Out.ShelvedFiles,
		[](const FSCFileRef& A, const FSCFileRef& B)
		{
			return A->GetDepotPath().Equals(B->GetDepotPath(), ESearchCase::IgnoreCase) && A->GetState() == B->GetState();
		});
	
	// CL is considered valid if the describe command returned a description entry (which could still be empty)
	Out.bIsValid = bFoundDescription;

	return Out;
}

void FChangelistService::CacheChangelistData()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FChangelistService::CacheChangelistData);
	check(ParseChangelistTask.IsCompleted());
	Changelist = ParseChangelistTask.GetResult();
	if (Changelist.bIsValid)
	{
		CLDescription = Changelist.Description;
		PrintFilesAndShelvedFiles();
	}
	else
	{
		UE_LOGF(LogSubmitToolP4, Error, "Couldn't retrieve information for CL %ls", *CLID);
		FModelInterface::SetErrorState();
	}

	BroadcastReadyIsValid.Emplace(Changelist.bIsValid);
}

void FChangelistService::UpdateCachedChangelistData()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FChangelistService::DepotFileToLocalFile);
	check(ParseChangelistTask.IsCompleted());
	FChangelistData NewChangelist = ParseChangelistTask.GetResult();

	ETaskArea ChangeType = ETaskArea::None;

	if (NewChangelist.Description.TrimEnd() != Changelist.Description.TrimEnd())
	{
		UE_LOGF(LogSubmitToolP4, Log, "CL %ls Description was updated outside of Submit Tool while it was still open, Description has been updated to match P4V.", *GetCLID());
		UE_LOGF(LogSubmitToolP4Debug, Log, "\n - Original Description '%ls'\n - Submit Tool Description '%ls'\n - New Description '%ls'",
			*Changelist.Description, *CLDescription, *NewChangelist.Description);

		CLDescription = NewChangelist.Description;
		ChangeType |= ETaskArea::Changelist;
	}

	bool bNewLocalFiles = !Algo::Compare(NewChangelist.Files, Changelist.Files,
		[](const FSCFileRef& A, const FSCFileRef& B)
		{
			return A->GetFilename().Equals(B->GetFilename(), ESearchCase::IgnoreCase) && A->GetState() == B->GetState();
		});

	if (bNewLocalFiles)
	{
		UE_LOGF(LogSubmitToolP4, Log, "CL %ls files were updated outside of Submit Tool while it was open, Validation state has been reset", *GetCLID());
		ChangeType |= ETaskArea::LocalFiles;
	}

	bool bNewShelf = !Algo::Compare(NewChangelist.ShelvedFiles, Changelist.ShelvedFiles,
		[](const FSCFileRef& A, const FSCFileRef& B)
		{
			return A->GetDepotPath().Equals(B->GetDepotPath(), ESearchCase::IgnoreCase) && A->GetState() == B->GetState();
		});

	if (bNewShelf)
	{
		UE_LOGF(LogSubmitToolP4, Log, "CL %ls shelved files were updated outside of Submit Tool while it was open, Validation state has been reset", *GetCLID());
		ChangeType |= ETaskArea::ShelvedFiles;
	}

	if (bNewLocalFiles || bNewShelf)
	{
		UE_LOGF(LogSubmitToolP4Debug, Log, "Current ST File info:");
		PrintFilesAndShelvedFiles();
	}

	Changelist = NewChangelist;

	if (bNewLocalFiles || bNewShelf)
	{
		UE_LOGF(LogSubmitToolP4Debug, Log, "New P4 File info:");
		PrintFilesAndShelvedFiles();
	}

	BroadcastRefreshChangeType.Emplace(ChangeType);
}

void FChangelistService::Submit(const FString& InDescriptionAddendum, TFunction<void(const FSCCommand&)> OnComplete)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FChangelistData::Submit);
	UE_LOGF(LogSubmitToolP4, Warning, "Submit in progress for CL: %ls. Please wait...", *GetCLID());

	FString FinalDescription = WriteToString<1024>(CLDescription, InDescriptionAddendum).ToString();

	FSCCommand SubmitCommand = FSCCommand(TEXT("submit"), { TEXT("-f"), TEXT("revertunchanged"),  TEXT("-i") });


	TArray<FString> DepotPathFiles;

	Algo::Transform(Changelist.Files, DepotPathFiles, [](const FSCFileRef& InFile) { return InFile->GetDepotPath(); });

	// -i needs each description line and each depot filepath to be prepended by \t
	FinalDescription.ReplaceInline(TEXT("\n"), TEXT("\n\t"));
	SubmitCommand.InputData = FString::Printf(TEXT("Change: %s\nDescription:\n\t%s\nFiles:\n\t%s"), *CLID, *FinalDescription, *FSubmitToolCoreUtils::StringBuilderJoin<2048>(DepotPathFiles, TEXT("\n\t")));

	SourceControlService->RunCommand(MoveTemp(SubmitCommand), FOnSCCCommandComplete::CreateLambda(
		[OnComplete, this](const FSCCommand& Cmd)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FChangelistData::Submit_callback);

			if (Cmd.bSuccess)
			{
				UE_LOGF(LogSubmitToolP4, Log, "Submit operation succeeded");
			}
			else if(Cmd.bDroppedConnection)
			{ 
				UE_LOGF(LogSubmitToolP4, Warning, "Submit operation cancelled.");
			}
			else
			{
				UE_LOGF(LogSubmitToolP4, Warning, "Submit operation failed.");
			}

			if (OnComplete.IsSet())
			{
				OnComplete(Cmd);
			}

		}));
}

void FChangelistService::DeleteShelvedFiles(TFunction<void(const FSCCommand&)> OnDeleteComplete)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FChangelistService::DeleteShelvedFiles);
	UE_LOGF(LogSubmitToolP4, Log, "Removing shelved files in CL %ls...", *GetCLID());

	SourceControlService->RunCommand(TEXT("shelve"), { TEXT("-d") , TEXT("-c"), GetCLID() }, FOnSCCCommandComplete::CreateLambda(
		[OnDeleteComplete, this](const FSCCommand& Cmd)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FChangelistService::DeleteShelvedFiles_callback);

			if (Cmd.bSuccess)
			{
				Changelist.ShelvedFiles.Reset();
				UE_LOGF(LogSubmitToolP4, Log, "CL %ls delete shelf operation succeeded", *GetCLID());
			}
			else if (Cmd.bDroppedConnection)
			{
				UE_LOGF(LogSubmitToolP4, Log, "CL %ls delete shelf operation cancelled", *GetCLID());
			}
			else
			{
				UE_LOGF(LogSubmitToolP4, Error, "CL %ls delete shelf operation failed:", *GetCLID());
			}

			if (OnDeleteComplete.IsSet())
			{
				OnDeleteComplete(Cmd);
			}
		}
	));
}

void FChangelistService::ReplaceShelvedFiles(TFunction<void(const FSCCommand&)> OnComplete)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FChangelistService::ReplaceShelvedFiles);
	UE_LOGF(LogSubmitToolP4, Log, "Replacing shelved files in CL %ls...", *GetCLID());

	SourceControlService->RunCommand(TEXT("shelve"), { TEXT("-r") , TEXT("-c"), GetCLID() }, FOnSCCCommandComplete::CreateLambda(
		[OnComplete, this](const FSCCommand& Cmd)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FChangelistService::ReplaceShelvedFiles_callback);

			if (Cmd.bSuccess)
			{
				UE_LOGF(LogSubmitToolP4, Log, "CL %ls replace shelf operation succeeded", *GetCLID());
				Changelist.ShelvedFiles = ParseShelvedFilesFromP4(Cmd);

				TSharedPtr<ICacheDataService> CacheService = ServiceProvider->GetService<ICacheDataService>();

				if (CacheService.IsValid())
				{
					CacheService->SetLastShelfKnownUpToDate(GetCLID(), FDateTime::UtcNow());
				}

				Changelist.bIsShelfComplete = true;
				bIsShelfReady = true;
			}
			else if (Cmd.bDroppedConnection)
			{
				UE_LOGF(LogSubmitToolP4, Log, "CL %ls replace shelf operation cancelled", *GetCLID());
			}
			else
			{
				UE_LOGF(LogSubmitToolP4, Error, "CL %ls replace shelf operation failed", *GetCLID());
			}

			if (OnComplete.IsSet())
			{
				OnComplete(Cmd);
			}
		}
	));
}

void FChangelistService::CreateShelvedFiles(TFunction<void(const FSCCommand&)> OnComplete)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FChangelistService::CreateShelvedFiles);
	UE_LOGF(LogSubmitToolP4, Log, "Creating shelf for CL %ls...", *GetCLID());

	SourceControlService->RunCommand(TEXT("shelve"), { TEXT("-c"), GetCLID() }, FOnSCCCommandComplete::CreateLambda(
		[OnComplete, this](const FSCCommand& Cmd)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FChangelistService::CreateShelvedFiles_callback);

			if (Cmd.bSuccess)
			{
				UE_LOGF(LogSubmitToolP4, Log, "CL %ls create shelf operation succeeded", *GetCLID());
				Changelist.ShelvedFiles = ParseShelvedFilesFromP4(Cmd);

				TSharedPtr<ICacheDataService> CacheService = ServiceProvider->GetService<ICacheDataService>();
				if (CacheService.IsValid())
				{
					CacheService->SetLastShelfKnownUpToDate(GetCLID(), FDateTime::UtcNow());
				}

				Changelist.bIsShelfComplete = true;
				bIsShelfReady = true;
			}
			else if (Cmd.bDroppedConnection)
			{
				UE_LOGF(LogSubmitToolP4, Log, "CL %ls create shelf operation cancelled", *GetCLID());
			}
			else
			{
				UE_LOGF(LogSubmitToolP4, Error, "CL %ls create shelf operation failed", *GetCLID());
			}

			if (OnComplete.IsSet())
			{
				OnComplete(Cmd);
			}
		}
	));
}

void FChangelistService::UpdateShelvedFiles(TArray<FString> InFiles, TFunction<void(const FSCCommand&)> OnComplete)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FChangelistService::UpdateShelvedFiles);
	InFiles.Insert(TEXT("-f"), 0);
	InFiles.Insert(TEXT("-c"), 0);
	InFiles.Insert(CLID, 1);

	SourceControlService->RunCommand(TEXT("shelve"), MoveTemp(InFiles), FOnSCCCommandComplete::CreateLambda(
		[OnComplete, this](const FSCCommand& Cmd)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FChangelistService::UpdateShelvedFiles_callback);

			if (Cmd.bSuccess)
			{
				TSharedPtr<ICacheDataService> CacheService = ServiceProvider->GetService<ICacheDataService>();

				if (CacheService.IsValid())
				{
					CacheService->SetLastShelfKnownUpToDate(GetCLID(), FDateTime::UtcNow());
				}

				UE_LOGF(LogSubmitToolP4, Log, "CL %ls shelved files have been updated", *GetCLID());
				Changelist.bIsShelfComplete = true;
				bIsShelfReady = true;
			}
			else if (Cmd.bDroppedConnection)
			{
				UE_LOGF(LogSubmitToolP4, Log, "Update shelf operation was cancelled");
			}
			else
			{
				UE_LOGF(LogSubmitToolP4, Error, "Update shelf operation failed");
			}

			if (OnComplete != nullptr)
			{
				OnComplete(Cmd);
			}
		}));
}

bool FChangelistService::P4Tick(float DeltaTime)
{
	if (BroadcastReadyIsValid.IsSet())
	{
		CLReadyCallback.ExecuteIfBound(BroadcastReadyIsValid.GetValue());
		BroadcastReadyIsValid.Reset();
	}
	if (BroadcastRefreshChangeType.IsSet())
	{
		CLRefreshCallback.ExecuteIfBound(BroadcastRefreshChangeType.GetValue());
		BroadcastRefreshChangeType.Reset();
	}
	return true;
}

void FChangelistService::CancelP4Operations()
{
	UE_LOGF(LogSubmitToolP4, Warning, "P4 Operations cancel requested")
	SourceControlService->RequestCancel();
}

void FChangelistService::CreateCLFromDefaultCL()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FChangelistService::CreateCLFromDefaultCL);

	// If we try to open submit tool with the default cl: Update all CL Status -> create a new CL -> move files from default to the new CL -> regular flow
	UE_LOGF(LogSubmitToolP4, Log, "Default changelist is not supported by Submit Tool, creating a new CL and moving files...");
	SourceControlService->RunCommand(TEXT("change"), { TEXT("-o") }, FOnSCCCommandComplete::CreateLambda(
		[this](const FSCCommand& InCmd) {
			TRACE_CPUPROFILER_EVENT_SCOPE(FChangelistService::CreateCLFromDefaultCL_Change_Output);

			if (InCmd.bSuccess)
			{
				FSCCommand Command = FSCCommand(TEXT("change"), { TEXT("-i") });
				Command.InputData = ParseP4ChangesOutput(InCmd.Values[0], Parameters.NewChangelistMessage);

				SourceControlService->RunCommand(MoveTemp(Command), FOnSCCCommandComplete::CreateLambda(
					[this](const FSCCommand& InCmd) {
						TRACE_CPUPROFILER_EVENT_SCOPE(FChangelistService::CreateCLFromDefaultCL_Change_Input);
						if (InCmd.bSuccess)
						{
							for (const FText& Message : InCmd.Results.InfoMessages)
							{
								// Perforce message format is "Change XXXXXX created ..." it doesn't return it as a value so parse it from the message
								const FString StringMessage = Message.ToString();
								TStringBuilder<16> CLBuilder;
								for (int32 i = 7; i < StringMessage.Len() && StringMessage[i] != TCHAR(' '); ++i)
								{
									if (FChar::IsDigit(StringMessage[i]))
									{
										CLBuilder.AppendChar(StringMessage[i]);
									}
								}

								if (CLBuilder.Len() != 0)
								{
									CLID = CLBuilder.ToString();
									FConfiguration::AddOrUpdateEntry(TEXT("$(CL)"), GetCLID());
									UE_LOGF(LogSubmitToolP4, Log, "Created CL %ls from default.", *CLID);
								}
								else
								{
									UE_LOGF(LogSubmitToolP4Debug, Error, "Couldn't retrieve the created CL number.");
								}
							}

							FetchChangelistDataAsync();
						}
					}
				));
			}
		}
	));
}

void FChangelistService::PrintFilesAndShelvedFiles() const
{
	if (Changelist.Files.Num() > 0)
	{
		UE_LOGF(LogSubmitToolP4Debug, Log, "Files in CL:");
		for(const FSCFileRef& File : Changelist.Files)
		{
			UE_LOGF(LogSubmitToolP4Debug, Log, "\t%ls - %ls", *File->GetFilename(), *File->GetDepotPath());
		}
	}

	if (Changelist.ShelvedFiles.Num() > 0)
	{
		UE_LOGF(LogSubmitToolP4Debug, Log, "Shelved Files in CL:");
		for(const FSCFileRef& File : Changelist.ShelvedFiles)
		{
			UE_LOGF(LogSubmitToolP4Debug, Log, "\t%ls - %ls", *File->GetFilename(), *File->GetDepotPath());
		}
	}
}

TArray<FSCFileRef> FChangelistService::ParseShelvedFilesFromP4(const FSCCommand& InCmd)
{
	TArray<FSCFileRef> NewShelvedFiles;
	TArray<FString> WhereFiles;
	TMap<FString, ESCFileState> WhereStates;
	for (const FSCCRecord& Record : InCmd.Values)
	{
		const FString* DepotFile = Record.Find(TEXT("depotFile"));
		const FString* Action = Record.Find(TEXT("action"));
		if (DepotFile && Action)
		{
			ESCFileState State = GetLocalFileState(*Action);

			// Check if it was already in the shelf before
			FSCFileRef* FoundFile = Changelist.ShelvedFiles.FindByPredicate([&DepotFile](const FSCFileRef& InFileRef) { return InFileRef->GetDepotPath().Equals(*DepotFile, ESearchCase::IgnoreCase); });
			if (FoundFile != nullptr)
			{
				(*FoundFile)->UpdateState(State);
				NewShelvedFiles.Add(*FoundFile);
			}
			else
			{
				// Look into the local files to see if we know about locations for this one already
				FoundFile = Changelist.Files.FindByPredicate([&DepotFile](const FSCFileRef& InFileRef) { return InFileRef->GetDepotPath().Equals(*DepotFile, ESearchCase::IgnoreCase); });
				if (FoundFile != nullptr)
				{
					NewShelvedFiles.Add(MakeShared<FSCFile>((*FoundFile)->GetFilename(), *DepotFile, State));
				}
				else
				{
					// Needs a p4 where to figure out the local filepath
					WhereFiles.Add(*DepotFile);
					WhereStates.Add(*DepotFile, State);
				}
			}
		}
	}

	if (WhereFiles.Num() > 0)
	{
		TMap<FString, FString> WhereResults = RunWhereCommandSync(MoveTemp(WhereFiles));

		for (const TPair<FString, FString>& WhereEntry : WhereResults)
		{
			NewShelvedFiles.Add(MakeShared<FSCFile>(WhereEntry.Value, WhereEntry.Key, WhereStates[WhereEntry.Key]));
		}
	}

	return NewShelvedFiles;
}

TMap<FString, FString> FChangelistService::RunWhereCommandSync(TArray<FString>&& InDepotPaths)
{
	TMap<FString, FString> DepotFileToLocalFile;
	FSCCommand WhereCmd = SourceControlService->RunSyncCommand(TEXT("where"), MoveTemp(InDepotPaths));
	if (WhereCmd.bSuccess)
	{
		for (const TMap<FString, FString>& Record : WhereCmd.Values)
		{
			const FString* DepotFile = Record.Find(TEXT("depotFile"));
			const FString* Path = Record.Find(TEXT("path"));
			if (DepotFile && Path)
			{
				DepotFileToLocalFile.Add(*DepotFile, *Path);
			}
		}
	}

	return DepotFileToLocalFile;
}
void FChangelistService::WaitForChangelistData()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FChangelistService::WaitForChangelistData);
	UpdateChangelistTask.Wait();
	OpenedTask = {};
	DescribeTask = {};
	ParseChangelistTask = {};
	UpdateChangelistTask = {};
}

FString FChangelistService::ParseP4ChangesOutput(const FSCCRecord& InP4ChangeOutputRecords, const FString& InReplaceDescription)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FChangelistService::ParseP4ChangesOutput);

	TStringBuilder<1024> Buffer;
	for (const TPair<FString, FString>& RecordPair : InP4ChangeOutputRecords)
	{
		// Skip some fields that aren't required
		if (RecordPair.Key == TEXT("Date") ||
			RecordPair.Key == TEXT("Type") ||
			RecordPair.Key == TEXT("specFormatted") ||
			RecordPair.Key == TEXT("func"))
		{
			continue;
		}
		else if (RecordPair.Key == TEXT("extraTag0"))
		{
			// Changelists with shelved files will have additional tags which will be rejected in the edit
			break;
		}

		if (RecordPair.Key == TEXT("Description"))
		{
			// Description case: we will replace the current description by the new one provided
			Buffer.Append(TEXT("Description:\n"));
			{
				TArray<FString> DescLines;
				InReplaceDescription.ParseIntoArray(DescLines, TEXT("\n"), false);
				for (const FString& DescLine : DescLines)
				{
					Buffer.Append(TEXT("\t"));
					Buffer.Append(DescLine);
					Buffer.Append(TEXT("\n"));
				}
			}
			Buffer.Append(TEXT("\n"));
		}
		else if (RecordPair.Key.Contains(TEXT("Files")))
		{
			if (RecordPair.Key == TEXT("Files0"))
			{
				Buffer.Append(TEXT("Files:\n"));
			}

			Buffer.Append(TEXT("\t"));
			Buffer.Append(RecordPair.Value);
			Buffer.Append(TEXT("\n\n"));
		}
		else
		{
			// General case: just put back what is already present in the record
			Buffer.Append(RecordPair.Key);
			Buffer.Append(TEXT(":\t"));
			Buffer.Append(RecordPair.Value);
			Buffer.Append(TEXT("\n\n"));
		}
	}

	return Buffer.ToString();
}

void FChangelistService::CheckShelfIsUpToDate(TFunction<void(bool, TArray<FString>)> OnComplete)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FChangelistService::CheckShelfIsUpToDate);

	TArray<FString> FilePathsOutOfDate;

	if (!Changelist.bIsShelfComplete)
	{
		if(OnComplete)
		{
			OnComplete(false, MoveTemp(FilePathsOutOfDate));
		}
		return;
	}
	
	// If submit tool has ever known the shelf state for this CL, use that datetime and check if we know that local files have not changed since
	TSharedPtr<ICacheDataService> CacheService = ServiceProvider->GetService<ICacheDataService>();
	if (CacheService)
	{
		FDateTime LastShelf = CacheService->GetLastKnownShelfUpToDate(GetCLID());
		if (LastShelf != FDateTime::MinValue())
		{
			FFileManagerGeneric FileManager;
			for (const FSCFileRef& File : Changelist.Files)
			{
				FString Filename = File->GetFilename();
				FFileStatData FileModifiedDate = FileManager.GetStatData(*Filename);
				if (FileModifiedDate.bIsValid && FileModifiedDate.ModificationTime > LastShelf)
				{
					UE_LOGF(LogSubmitToolDebug, Log, "Modified file: %ls at %ls, Last Shelve operation: %ls", *Filename, *FileModifiedDate.ModificationTime.ToString(), *LastShelf.ToString());
					FilePathsOutOfDate.Add(Filename);
				}
			}
			
			// If local files have not changed, we can early out without running p4 diff
			if (FilePathsOutOfDate.Num() == 0)
			{
				bIsShelfReady = true;

				if (OnComplete)
				{
					OnComplete(bIsShelfReady, MoveTemp(FilePathsOutOfDate));
				}
				return;
			}
		}
	}

	if (!bDiffTaskRunning)
	{
		bDiffTaskRunning = true;
		// Real p4 diff
		UE_LOGF(LogSubmitToolP4Debug, Log, "Checking Shelf diff with local files...");
		TArray<FString> DiffArgs;
		DiffArgs.Reserve(GetFilesInCL().Num());
		for (const FSCFileRef& File : GetFilesInCL())
		{
			if (!File->IsDeleted())
			{
				DiffArgs.Emplace(*WriteToString<256>(File->GetDepotPath(), "@=", CLID));
			}
		}
		DiffArgs.Insert(TEXT("-ds"), 0);
		DiffArgs.Insert(TEXT("-Od"), 0);
		SourceControlService->RunCommand(TEXT("diff"), MoveTemp(DiffArgs), FOnSCCCommandComplete::CreateLambda(
			[OnComplete, this](const FSCCommand& Cmd)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(FChangelistService::CheckShelfIsUpToDate_DiffResult);

				TArray<FString> FilesWithChanges;
				if (Cmd.bSuccess)
				{
					for (const TMap<FString, FString>& Record : Cmd.Values)
					{
						if (Record.Contains(TEXT("clientFile")))
						{
							FilesWithChanges.Add(Record[TEXT("clientFile")]);
						}
						else
						{
							const FString Base = TEXT("clientFile");
							size_t i = 0;
							FString PathsKey = FString::Printf(TEXT("%s%d"), *Base, i);
							while (Record.Contains(PathsKey))
							{
								FilesWithChanges.Add(Record[PathsKey]);
								++i;
								PathsKey = FString::Printf(TEXT("%s%d"), *Base, i);
							}
						}
					}
				}

				bIsShelfReady = Cmd.bSuccess && FilesWithChanges.Num() == 0;
				// If we've checked that there are no diffs, register it to avoid future diffs
				if (bIsShelfReady)
				{
					ServiceProvider->GetService<ICacheDataService>()->SetLastShelfKnownUpToDate(CLID, FDateTime::UtcNow());
				}

				if (OnComplete)
				{
					OnComplete(bIsShelfReady, FilesWithChanges);
				}
				bDiffTaskRunning = false;
			}), true);
	}
}

void FChangelistService::EnsureShelfIsCurrent()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FChangelistService::EnsureShelfIsCurrent);

	bIsShelfReady = false;
	if (HasShelvedFiles())
	{
		CheckShelfIsUpToDate([this](bool bIsUpToDate, TArray<FString> InFilesOutOfDate) {
			// if shelf is not up to date, we need to refresh it
			if (!bIsUpToDate)
			{
				EDialogFactoryResult DialogResult = EDialogFactoryResult::Confirm;
				if (!FSubmitToolUserPrefs::Get()->bAutoStompShelf)
				{
					const FText TextTitle = FText::FromString(TEXT("Shelf Refresh"));
					FText TextDescription;
					if (InFilesOutOfDate.IsEmpty())
					{
						TextDescription = FText::FromString(TEXT("Your shelf does not match your local files. Do you want to recreate your shelf with the local files in the CL?"));
					}
					else
					{
						TextDescription = FText::FromString(FString::Printf(TEXT("The following files are not up to date in the shelf. They need to be re-shelved:\n\n%s"), *FString::Join(InFilesOutOfDate, TEXT("\n"))));
					}

					DialogResult = FDialogFactory::ShowDialog(
						TextTitle, 
						TextDescription,
						TArray<FString>{ TEXT("Re-shelve files"), TEXT("I will reshelve in P4V")},
						FSubmitToolUtils::BuildUserPrefCheckboxUI(FSubmitToolUserPrefs::Get()->bAutoStompShelf, FText::FromString(TEXT("Always stomp a shelf when it doesn't match local state, don't ask again"))));
				}

				if (DialogResult == EDialogFactoryResult::Confirm)
				{
					// if we get an empty list, it means the whole shelf needs to be refreshed (it has different files)
					if (InFilesOutOfDate.IsEmpty())
					{
						UE_LOGF(LogSubmitToolP4, Log, "Recreating shelf");
						ReplaceShelvedFiles();
					}
					else
					{
						// If we get a list of files out of date, we can just reshelve those instead
						UE_LOGF(LogSubmitToolP4, Log, "Refreshing shelved files that are out of date:\n%ls", *FString::Join(InFilesOutOfDate, TEXT("\n")));
						UpdateShelvedFiles(InFilesOutOfDate);
					}
				}
			}});
			return;
	}
	else
	{
		EDialogFactoryResult DialogResult = EDialogFactoryResult::Confirm;
		if (!FSubmitToolUserPrefs::Get()->bAutoCreateShelf)
		{
			const FText TextTitle = FText::FromString(TEXT("Create Shelf"));
			const FText TextDescription = FText::FromString(TEXT("Submit tool needs a shelf. Do you want to create your shelf?"));
		
			DialogResult = FDialogFactory::ShowDialog(
				TextTitle, 
				TextDescription, 
				TArray<FString>{ TEXT("Create shelf"), TEXT("Cancel")}, 
				FSubmitToolUtils::BuildUserPrefCheckboxUI(FSubmitToolUserPrefs::Get()->bAutoCreateShelf, FText::FromString(TEXT("Always create a shelf when it's empty, don't ask again"))));
		}

		if (DialogResult == EDialogFactoryResult::Confirm)
		{
			UE_LOGF(LogSubmitToolP4, Log, "Creating shelf");
			CreateShelvedFiles();
		}
	}
}
