// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Logic/Services/Interfaces/ISTSourceControlService.h"
#include "Logic/Services/Interfaces/IChangelistService.h"
#include "Containers/Ticker.h"
#include "Parameters/SubmitToolParameters.h"
#include "Tasks/Task.h"

class FSubmitToolServiceProvider;
using FSourceControlOperationRef = TSharedRef<class ISourceControlOperation, ESPMode::ThreadSafe>;


DECLARE_DELEGATE(FOnP4OperationCompleteDelegate)

struct FChangelistData
{
	FString Description;
	TArray<FSCFileRef> Files;				// <Filename, DepotPath, State>
	TArray<FSCFileRef> ShelvedFiles;		// <{}, DepotPath, State>
	TMap<FString, FString> AllOpenFiles;	// <DepotPath, Change>
	bool bIsValid = false;					// Changelist data could be fetched and parsed correctly
	bool bIsShelfComplete = false;			// OpenFilesSet == ShelvedFilesSet, content may differ

	static FChangelistData Parse(
		const FString& CLID,
		const FSCCommand& DescribeCmd,
		const FSCCommand& OpenedCmd,
		const TMap<FString, FString>& DepotFileToLocalFile);
};

class FChangelistService final : public IChangelistService
{
public:

	FChangelistService(const FGeneralParameters& InParameters, const TSharedPtr<FSubmitToolServiceProvider> InServiceProvider, const FOnChangeListReadyDelegate& InCLReadyCallback, const FOnChangelistRefreshDelegate& InCLRefreshCallback);
	~FChangelistService();

	virtual const FString GetCLID() const override
	{
		check(IsInGameThread());
		if(!CLID.IsEmpty())
		{
			return CLID;
		}

		return TEXT("Invalid");
	}

	virtual const FString& GetCLDescription() override
	{
		check(IsInGameThread());
		return CLDescription;
	}

	virtual bool SetCLDescription(const FString& newDescription, bool bNotifyEvent = false) override
	{
		check(IsInGameThread());
		FString lineEndReplaced = newDescription.Replace(TEXT("\r\n"), TEXT("\n"));

		if(CLDescription.Equals(lineEndReplaced, ESearchCase::IgnoreCase))
		{
			return false;
		}

		CLDescription = lineEndReplaced;

		if(bNotifyEvent)
		{
			OnCLDescriptionUpdated.Broadcast();
		}

		return true;
	}

	virtual const TArray<FSCFileRef>& GetFilesInCL() const override
	{
		check(IsInGameThread());
		return Changelist.Files;
	}

	virtual const TArray<FSCFileRef>& GetShelvedFilesInCL() const override
	{
		check(IsInGameThread());
		return Changelist.ShelvedFiles;
	}

	virtual bool HasShelvedFiles() const override
	{
		check(IsInGameThread());
		return Changelist.ShelvedFiles.Num() > 0;
	}

	virtual bool IsShelfComplete() const override
	{
		check(IsInGameThread());
		return Changelist.bIsShelfComplete;
	}

	virtual const TMap<FString, FString>& GetOpenFilesInChangelists() const override
	{
		check(IsInGameThread());
		return Changelist.AllOpenFiles;
	}

	virtual bool HasP4OperationsRunning() const override
	{
		check(IsInGameThread());
		return !UpdateChangelistTask.IsCompleted() || SourceControlService->IsBusy();
	}

	virtual bool IsShelfReady() const override
	{
		return bIsShelfReady;
	}

	virtual void Init() override;
	virtual void Submit(const FString& InDescriptionAddendum = TEXT(""), TFunction<void(const FSCCommand&)> OnComplete = nullptr) override;

	//void Submit(const FString& InDescriptionAddendum = TEXT(""), const FSourceControlOperationComplete& OnSubmitComplete = FSourceControlOperationComplete());
	virtual void CreateCLFromDefaultCL() override;
	virtual void FetchChangelistDataAsync() override;
	virtual void RevertUnchangedFiles(TFunction<void(const FSCCommand&)> OnComplete = nullptr) override;

	virtual void UpdateP4CLDescription() override;
	virtual void UpdateP4CLDescriptionSynchronously() override;
	virtual void DeleteShelvedFiles(TFunction<void(const FSCCommand&)> OnComplete = nullptr) override;
	virtual void ReplaceShelvedFiles(TFunction<void(const FSCCommand&)> OnComplete = nullptr) override;
	virtual void CreateShelvedFiles(TFunction<void(const FSCCommand&)> OnComplete = nullptr) override;
	virtual void UpdateShelvedFiles(TArray<FString> InFiles, TFunction<void(const FSCCommand&)> OnComplete = nullptr) override;
	virtual void CheckShelfIsUpToDate(TFunction<void(bool, TArray<FString>)> OnComplete) override;
	virtual void EnsureShelfIsCurrent() override;
	bool P4Tick(float DeltaTime);

	virtual void CancelP4Operations() override;

private:
	mutable FCriticalSection Mutex;

	FString CLID;
	FString CurrentStream;
	const FGeneralParameters& Parameters;
	const FOnChangeListReadyDelegate& CLReadyCallback;
	const FOnChangelistRefreshDelegate& CLRefreshCallback;

	const FTSTicker::FDelegateHandle TickHandle;
	const TSharedPtr<FSubmitToolServiceProvider> ServiceProvider;
	const TSharedPtr<ISTSourceControlService> SourceControlService;

	FChangelistData Changelist;
	FString CLDescription;
	bool bIsShelfReady = false;
	bool bDiffTaskRunning = false;
	TOptional<bool> BroadcastReadyIsValid;
	TOptional<ETaskArea> BroadcastRefreshChangeType;

	UE::Tasks::TTask<TSharedRef<FSCCommand>> OpenedTask;
	UE::Tasks::TTask<TSharedRef<FSCCommand>> DescribeTask;
	UE::Tasks::TTask<FChangelistData> ParseChangelistTask;
	UE::Tasks::FTask UpdateChangelistTask;

	void WaitForChangelistData();
	FString ParseP4ChangesOutput(const FSCCRecord& InP4ChangeOutputRecords, const FString& InReplaceDescription);
	void CacheChangelistData();
	void UpdateCachedChangelistData();
	void PrintFilesAndShelvedFiles() const;
	TArray<FSCFileRef> ParseShelvedFilesFromP4(const FSCCommand& InCmd);
	TMap<FString, FString> RunWhereCommandSync(TArray<FString>&& InDepotPaths);
};

Expose_TNameOf(FChangelistService);
