// Copyright Epic Games, Inc. All Rights Reserved.

#include "CredentialsService.h"

#include "SubmitToolUtils.h"
#include "SubmitToolCoreUtils.h"
#include "Logging/LogMacros.h"
#include "Logging/SubmitToolLog.h"
#include "Logic/ProcessWrapper.h"

#include "JsonObjectConverter.h"
#include "HAL/FileManager.h"
#include "HttpServerModule.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Misc/Base64.h"
#include "Tasks/Task.h"

TUniquePtr<FAES::FAESKey> FCredentialsService::Key = nullptr;


bool FHttpServerListener::StartListening()
{
	if (HttpRouter.IsValid())
	{
		return true;
	}

	FHttpServerModule& ServerModule = FHttpServerModule::Get();

	HttpRouter = ServerModule.GetHttpRouter(Port);

	if (!HttpRouter.IsValid())
	{
		UE_LOGF(LogSubmitTool, Error, "Failed to start Http Jira OAuth listener in port %d on URI %ls", Port, *URI);
		return false;
	}

	RouteHandle = HttpRouter->BindRoute(FHttpPath(URI), Verb, Callback);

	if (RouteHandle != nullptr)
	{
		ServerModule.StartAllListeners();

	}

	return RouteHandle != nullptr;
}

void FHttpServerListener::StopListening()
{
	if (HttpRouter.IsValid())
	{
		HttpRouter->UnbindRoute(RouteHandle);
		RouteHandle.Reset();
		HttpRouter.Reset();
	}
}

FCredentialsService::FCredentialsService(const FSubmitToolParameters& InParameters)
	: Parameters(InParameters)
{
	if(IsOIDCTokenEnabled())
	{
		GetOIDCToken();
		FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateRaw(this, &FCredentialsService::Tick), 5);
	}

	LoadKey();
	LoadCredentials();
}

FCredentialsService::~FCredentialsService()
{
	if(OIDCProcess.IsValid())
	{
		OIDCProcess->Stop();
	}

	GetOIDCTask.Wait();
}

UE::Tasks::TTask<void> FCredentialsService::QueueWorkForToken(TFunction<void(const FString&)> InFunction)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FCredentialsService::QueueWorkForToken);
	return UE::Tasks::Launch(UE_SOURCE_LOCATION, [this, InFunction]() {
		TRACE_CPUPROFILER_EVENT_SCOPE(FCredentialsService::DoWorkForToken);
		InFunction(OIDCToken);
		}, GetOIDCTask, LowLevelTasks::ETaskPriority::Normal, UE::Tasks::EExtendedTaskPriority::GameThreadNormalPri);
}

void FCredentialsService::LoadKey()
{
	if(!IFileManager::Get().FileExists(*GetKeyFilepath()))
	{
		GenerateKey();
	}

	if(IFileManager::Get().FileExists(*GetKeyFilepath()))
	{
		FArchive* File = IFileManager::Get().CreateFileReader(*GetKeyFilepath());
		if(File->TotalSize() < 4)
		{
			UE_LOGF(LogSubmitToolDebug, Warning, "Unexpected file size encryption key invalidated");
			File->Close();
			delete File;
			File = nullptr;
			return;
		}

		int32 Size;
		*File << Size;

		// see if the file has exactly the length we expect
		// two int32 (a size and a garbage one) + the data + one garbage int32 in between the data
		if(File->TotalSize() != 4 + 4 + Size + 4)
		{
			UE_LOGF(LogSubmitToolDebug, Warning, "Unexpected file size encryption key invalidated");
			File->Close();
			delete File;
			File = nullptr;
			return;
		}

		int32 Garbage;
		*File << Garbage;

		TArray<uint8> Bytes;
		uint8 byte;

		for(size_t i = 0; i < Size; ++i)
		{
			if(i == 2)
			{
				*File << Garbage;
			}

			*File << byte;
			Bytes.Add(byte);
		}
		File->Close();
		delete File;
		File = nullptr;

		Key = MakeUnique<FAES::FAESKey>();
		check(Bytes.Num() == sizeof(FAES::FAESKey::Key));
		FMemory::Memcpy(Key->Key, &Bytes[0], sizeof(FAES::FAESKey::Key));
	}
}

void FCredentialsService::GenerateKey()
{
	TArray<uint8> dataArray;
	for(size_t i = 0; i < FAES::FAESKey::KeySize; ++i)
	{
		int32 random = FMath::Rand();
		dataArray.Add(random);
	}

	Key = MakeUnique<FAES::FAESKey>();
	FMemory::Memcpy(Key->Key, dataArray.GetData(), sizeof(FAES::FAESKey::Key));

	FArchive* File = IFileManager::Get().CreateFileWriter(*GetKeyFilepath(), EFileWrite::FILEWRITE_EvenIfReadOnly);
	int32 Size = FAES::FAESKey::KeySize;
	*File << Size;
	int32 Garbage = FMath::Rand();
	*File << Garbage;

	for(size_t i = 0; i < Size; ++i)
	{
		if(i == 2)
		{
			Garbage = FMath::Rand();
			*File << Garbage;
		}

		*File << dataArray[i];
	}

	File->Close();
	delete File;
	File = nullptr;
}

const FString FCredentialsService::GetKeyFilepath()
{
	return FPaths::Combine(FSubmitToolCoreUtils::GetLocalAppDataPath(), TEXT("SubmitTool"), TEXT(".cache"));
}

const FString FCredentialsService::GetCredentialsFilepath() const
{
	return FPaths::Combine(FSubmitToolCoreUtils::GetLocalAppDataPath(), TEXT("SubmitTool"), TEXT("jira.dat"));
}

void FCredentialsService::GetOIDCToken()
{	
	UE_LOGF(LogSubmitTool, Log, "Obtaining new OIDCToken");

	if (!GetOIDCTask.IsValid() || GetOIDCTask.IsCompleted())
	{
		GetOIDCTask = UE::Tasks::Launch(UE_SOURCE_LOCATION,
			[this] {
				TRACE_CPUPROFILER_EVENT_SCOPE(FCredentialsService::GetOIDCToken);

				FString FullOutput;
				FOnOutputLine OutputLineProcess = FOnOutputLine::CreateLambda([&FullOutput](FString OutputLine, const EProcessOutputType& OutputType) {
					if(OutputType == EProcessOutputType::ProcessError || OutputType == EProcessOutputType::STDErr)
					{
						UE_LOGF(LogSubmitTool, Error, "%ls", *OutputLine);
					}
					else
					{
						if (OutputType == EProcessOutputType::STDOutput)
						{
							FullOutput += OutputLine;
						}

						UE_LOGF(LogSubmitToolDebug, Log, "%ls", *OutputLine);
					}
					});

				FOnCompleted OnCompleted = FOnCompleted::CreateLambda([this, &FullOutput](const int32 InExitCode) {
					ParseOIDCTokenData(FullOutput);
					});

				OIDCProcess = MakeUnique<FProcessWrapper>(TEXT("Oidc"), FConfiguration::SubstituteAndNormalizeFilename(Parameters.OAuthParameters.OAuthTokenTool), FString::Printf(TEXT("%s"), *Parameters.OAuthParameters.OAuthArgs), OnCompleted, OutputLineProcess);
				OIDCProcess->Start(true);

				if(!OIDCProcess.IsValid() || OIDCProcess->ExitCode != 0)
				{
					UE_LOGF(LogSubmitTool, Warning, "Couldn't obtain OIDC credentials");
					if(OIDCProcess.IsValid())
					{
						OIDCProcess = nullptr;
					}
					return false;
				}

				OIDCProcess = nullptr;
				return true;
			});
	}
}

bool FCredentialsService::ParseOIDCTokenData(const FString& InToken)
{
	TSharedPtr<FJsonObject> RootJsonObject;
	TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(InToken);
	FJsonSerializer::Deserialize(Reader, RootJsonObject);

	if(RootJsonObject.IsValid())
	{
		FString Expiration = RootJsonObject->GetStringField(TEXT("ExpiresAt"));
		FDateTime::ParseIso8601(*Expiration, TokenExpiration);
		UE_LOGF(LogSubmitToolDebug, Log, "OIDC Token Expiration %ls", *Expiration);
		
		OIDCToken = RootJsonObject->GetStringField(TEXT("Token"));
		UE_LOGF(LogSubmitTool, Log, "OIDC Token loaded correctly");
		return true;
	}

	UE_LOGF(LogSubmitTool, Error, "Couldn't parse OIDC Token from string: '%ls'", *InToken);

	return false;
}

constexpr int JiraCredentialDatVersion = 1;
constexpr int JiraCredentialTokenVersion = 2;
constexpr int JiraCredentialRefreshTokenVersion = 3;
void FCredentialsService::SaveCredentials() const
{
	TArray<uint8> Bytes;
	Bytes.SetNumUninitialized(LoginString.Len());
	StringToBytes(LoginString, Bytes.GetData(), LoginString.Len());

	int32 ActualLength = Bytes.Num();

	int32 NumBytesEncrypted = Align(Bytes.Num(), FAES::AESBlockSize);
	Bytes.SetNum(NumBytesEncrypted);
	FAES::EncryptData(Bytes.GetData(), Bytes.Num(), *Key);

	TArray<uint8> BytesToken = TArray<uint8>();
	FMemoryWriter Writer(BytesToken);
	if (JiraOAuthToken != nullptr)
	{
		FJiraToken::StaticStruct()->SerializeItem(Writer, JiraOAuthToken.Get(), nullptr);
	}
	int32 ActualLengthToken = BytesToken.Num();
	int32 NumBytesEncryptedToken = Align(BytesToken.Num(), FAES::AESBlockSize);
	BytesToken.SetNum(NumBytesEncryptedToken);
	FAES::EncryptData(BytesToken.GetData(), BytesToken.Num(), *Key);

	FArchive* File = IFileManager::Get().CreateFileWriter(*GetCredentialsFilepath(), EFileWrite::FILEWRITE_EvenIfReadOnly);

	if(File != nullptr)
	{
		int32 Version = JiraCredentialRefreshTokenVersion;
		*File << Version;

		*File << NumBytesEncrypted;
		*File << ActualLength;
		int32 Garbage = FMath::Rand();
		*File << Garbage;
		File->Serialize(Bytes.GetData(), Bytes.Num());
		Garbage = FMath::Rand();
		*File << Garbage;
		*File << NumBytesEncryptedToken;
		*File << ActualLengthToken;
		if (BytesToken.Num() > 0)
		{
			File->Serialize(BytesToken.GetData(), BytesToken.Num());
		}

		File->Close();
		delete File;
		File = nullptr;
	}
	else
	{
		UE_LOGF(LogSubmitTool, Warning, "Could not create file '%ls'.", *GetCredentialsFilepath());
	}
}

void FCredentialsService::LoadCredentials()
{
	if(Key.IsValid())
	{
		if(IFileManager::Get().FileExists(*GetCredentialsFilepath()))
		{
			FArchive* File = IFileManager::Get().CreateFileReader(*GetCredentialsFilepath());

			if(File != nullptr)
			{
				// Read the size
				if(File->TotalSize() < 4 + 4)
				{
					UE_LOGF(LogSubmitToolDebug, Warning, "Unexpected file size login key invalidated");
					File->Close();
					delete File;
					return;
				}

				int32 Version;
				*File << Version;

				// Check Versions here
				if(Version > JiraCredentialRefreshTokenVersion)
				{
					UE_LOGF(LogSubmitToolDebug, Warning, "Unexpected Credentials Version, aborting credentials loading.");
					File->Close();
					delete File;
					return;
				}

				int32 PaddedLength;
				int32 LengthWithoutPadding;

				*File << PaddedLength;
				*File << LengthWithoutPadding;

				int32 Garbage;
				*File << Garbage;

				TArray<uint8> DeserializedBytes;
				DeserializedBytes.SetNum(PaddedLength);
				File->Serialize(DeserializedBytes.GetData(), PaddedLength);

				FAES::DecryptData(DeserializedBytes.GetData(), DeserializedBytes.Num(), *Key);

				LoginString = BytesToString(DeserializedBytes.GetData(), LengthWithoutPadding);

				if(!GetUsername().IsEmpty() && !GetPassword().IsEmpty())
				{
					UE_LOGF(LogSubmitTool, Log, "Local Credentials loaded");
				}

				if (Version >= JiraCredentialTokenVersion)
				{
					*File << Garbage;
					// Token
					*File << PaddedLength;
					*File << LengthWithoutPadding;

					if (LengthWithoutPadding > 0)
					{
						DeserializedBytes.SetNum(PaddedLength);
						File->Serialize(DeserializedBytes.GetData(), PaddedLength);
						FAES::DecryptData(DeserializedBytes.GetData(), DeserializedBytes.Num(), *Key);

						FMemoryReader Reader(DeserializedBytes);
						FJiraToken Token;
						FJiraToken::StaticStruct()->SerializeItem(Reader, &Token, nullptr);
						if (Version < JiraCredentialRefreshTokenVersion)
						{
							Token.Refresh_Token = TEXT("");
						}
						JiraOAuthToken = MakeUnique<FJiraToken>(MoveTemp(Token));
					}
				}
				File->Close();
				delete File;
			}
			else
			{
				UE_LOGF(LogSubmitTool, Warning, "Could not read file '%ls'.", *GetCredentialsFilepath());
			}
		}
		else
		{
			UE_LOGF(LogSubmitToolDebug, Warning, "File %ls does not exists, no credentials were loaded", *GetCredentialsFilepath())
		}
	}
}

FString FCredentialsService::GetUsername() const
{
	FString DecodedString;
	if(!FBase64::Decode(LoginString, DecodedString))
	{
		UE_LOGF(LogSubmitToolDebug, Error, "Error while trying to decode Jira Login");
	}

	TArray<FString> LoginValues;
	DecodedString.ParseIntoArray(LoginValues, TEXT(":"));

	if(LoginValues.Num() == 2)
	{
		return LoginValues[0];
	}

	return FString();
}

FString FCredentialsService::GetPassword() const
{
	FString DecodedString;
	if(!FBase64::Decode(LoginString, DecodedString))
	{
		UE_LOGF(LogSubmitToolDebug, Error, "Error while trying to decode Jira Password");
	}

	TArray<FString> LoginValues;
	DecodedString.ParseIntoArray(LoginValues, TEXT(":"));

	if(LoginValues.Num() == 2)
	{
		return LoginValues[1];
	}

	return FString();
}

void FCredentialsService::SetLogin(const FString& InUsername, const FString& InPassword)
{
	int32 ChopLocation = InUsername.Find(TEXT("@"));
	FString FormattedUsername = InUsername;

	// Just grab the username if they accidentally entered their full email.
	if(ChopLocation != -1)
	{
		FormattedUsername = InUsername.LeftChop(InUsername.Len() - ChopLocation);
	}

	FString newLogin = FBase64::Encode(FormattedUsername + TEXT(":") + InPassword);
	if(newLogin != LoginString)
	{
		LoginString = newLogin;
		SaveCredentials();
	}
}

bool FCredentialsService::Tick(float DeltaTime)
{
	if(TokenExpiration != FDateTime() && TokenExpiration < FDateTime::UtcNow())
	{
		GetOIDCToken();
	}

	return true;
}
