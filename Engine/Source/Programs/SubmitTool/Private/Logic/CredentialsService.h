// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Misc/AES.h"
#include "IHttpRouter.h"
#include "HttpServerRequest.h"
#include "Parameters/SubmitToolParameters.h"
#include "Tasks/Task.h"
#include "Logic/Services/Interfaces/ICredentialsService.h"
#include "Models/JiraToken.h"

class FProcessWrapper;

class FHttpServerListener
{
public:
	FHttpServerListener(const FString& InURI, const EHttpServerRequestVerbs& InVerb, uint32 InPort, FHttpRequestHandler InCallback) : URI(InURI), Verb(InVerb), Port(InPort), Callback(InCallback)
	{}
	~FHttpServerListener()
	{
		StopListening();
	}

	bool StartListening();
	void StopListening();

	bool IsListening() const
	{
		return HttpRouter.IsValid();
	}

private:
	TSharedPtr<IHttpRouter> HttpRouter = nullptr;
	FHttpRouteHandle RouteHandle = nullptr;
	FString URI;
	EHttpServerRequestVerbs Verb;
	uint32 Port;
	FHttpRequestHandler Callback;
};

class FCredentialsService : public ICredentialsService
{
public:
	FCredentialsService(const FSubmitToolParameters& InParameters);
	virtual ~FCredentialsService();

	virtual bool HasJiraCredentials() const override
	{
		return Parameters.JiraParameters.OAuthClientId.IsEmpty() ? !LoginString.IsEmpty() : JiraOAuthToken != nullptr;
	}

	virtual void InvalidateCredentials() override
	{
		LoginString.Empty();
		JiraOAuthToken = nullptr;
	}

	virtual const FString& GetEncodedLoginString() const override
	{
		return LoginString;
	}

	virtual FString GetUsername() const override;
	virtual void SetLogin(const FString& InUsername, const FString& InPassword) override;

	virtual bool IsOIDCTokenEnabled() override
	{
		return !Parameters.OAuthParameters.OAuthTokenTool.IsEmpty();
	}

	virtual bool IsTokenReady() const override
	{
		return !OIDCToken.IsEmpty();
	}

	virtual const FString& GetToken() const override
	{
		return OIDCToken;
	}

	static const TUniquePtr<FAES::FAESKey>& GetEncryptionKey()
	{
		if(!Key.IsValid())
		{
			LoadKey();
		}

		return Key;
	}

	virtual UE::Tasks::TTask<void> QueueWorkForToken(TFunction<void(const FString&)> InFunction) override;

	virtual void SetJiraToken(FJiraToken&& InJiraToken) override
	{
		JiraOAuthToken = MakeUnique<FJiraToken>(MoveTemp(InJiraToken));
		SaveCredentials();
	}

	virtual const TUniquePtr<FJiraToken>& GetJiraToken() const override
	{
		return JiraOAuthToken;
	}

private:
	TUniquePtr<FJiraToken> JiraOAuthToken;
	static TUniquePtr<FAES::FAESKey> Key;
	static void LoadKey();
	static void GenerateKey();
	static const FString GetKeyFilepath();

	UE::Tasks::TTask<bool> GetOIDCTask;
	TUniquePtr<FProcessWrapper> OIDCProcess;
	FString OIDCToken;
	FDateTime TokenExpiration;
	const FSubmitToolParameters Parameters;

	void GetOIDCToken();
	bool ParseOIDCTokenData(const FString& InToken);
	FString GetPassword() const;
	void SaveCredentials() const;
	void LoadCredentials();

	bool Tick(float DeltaTime);

	const FString GetCredentialsFilepath() const;

	FString LoginString;
	bool bValidatedCredentials = true;
};

Expose_TNameOf(FCredentialsService);