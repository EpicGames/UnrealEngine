// Copyright Epic Games, Inc. All Rights Reserved.

#include "DialogFactory.h"
#include "View/Widgets/ConfirmDialogWidget.h"
#include "Widgets/SWindow.h"
#include "Framework/Application/SlateApplication.h"

EDialogFactoryResult FDialogFactory::ShowDialog(const FText& Title, const FText& Description,
	const TArray<FString>& AvailableButtons, const TSharedPtr<SWidget> AdditionalContent, TFunction<bool(size_t)> InIsBtnEnabled)
{	
	TSharedPtr<SWindow> Window = SNew(SWindow)
	.Title(Title)
	.SizingRule(ESizingRule::Autosized)
	.SupportsMaximize(false)
	.SupportsMinimize(false)
	.MinWidth(400)
	.MinHeight(1);
	
	EDialogFactoryResult ButtonClicked = EDialogFactoryResult::ClosedWithX;
	const FOnResult ResultsCallback = FOnResult::CreateLambda([&Window, &ButtonClicked](size_t InResult)
	{
		ButtonClicked = static_cast<EDialogFactoryResult>(InResult);
		Window->RequestDestroyWindow();
	});
	
	const TSharedPtr<SConfirmDialogWidget> ConfirmWidget = SNew(SConfirmDialogWidget)
		.DescriptionText(Description)
		.Buttons(AvailableButtons)
		.ResultCallback(ResultsCallback)
		.AdditionalContent(AdditionalContent)
		.IsBtnEnabled(InIsBtnEnabled);

	Window->SetContent(ConfirmWidget.ToSharedRef());

	FSlateApplication::Get().AddModalWindow(Window.ToSharedRef(), nullptr);

	return ButtonClicked;
}

EDialogFactoryResult FDialogFactory::ShowConfirmDialog(const FText& Title, const FText& Description, const TSharedPtr<SWidget> AdditionalContent)
{
	return ShowDialog(Title, Description, TArray<FString>{ TEXT("Confirm"), TEXT("Cancel") }, AdditionalContent);
};

EDialogFactoryResult FDialogFactory::ShowInformationDialog(const FText& Title, const FText& Description, const TSharedPtr<SWidget> AdditionalContent)
{
	return ShowDialog(Title, Description, TArray<FString>{ TEXT("Ok") }, AdditionalContent);
}

EDialogFactoryResult FDialogFactory::ShowYesNoDialog(const FText& Title, const FText& Description, const TSharedPtr<SWidget> AdditionalContent)
{
	return ShowDialog(Title, Description, TArray<FString>{ TEXT("Yes"), TEXT("No") }, AdditionalContent);
}
