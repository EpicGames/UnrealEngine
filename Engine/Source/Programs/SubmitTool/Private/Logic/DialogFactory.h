// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "View/Widgets/ConfirmDialogWidget.h"

enum class EDialogFactoryResult : size_t
{
	FirstButton = 0,
	SecondButton,
	ThirdButton,
	FourthButton,
	ClosedWithX = std::numeric_limits<size_t>::max(),
	Confirm = FirstButton,
	Cancel = SecondButton,
	Yes = FirstButton,
	No = SecondButton
};

class FDialogFactory
{
public:
	static EDialogFactoryResult ShowDialog(const FText& Title, const FText& Description, const TArray<FString>& AvailableButtons, const TSharedPtr<SWidget> AdditionalContent = nullptr, TFunction<bool(size_t)> InIsBtnEnabled = nullptr);

	/**
	 * @brief Shows a dialog screen with the Cancel and Confirm buttons
	 * @param Title The title of the dialog
	 * @param Description The description of the dialog
	 * @return The button idx that is pressed
	 */
	static EDialogFactoryResult ShowConfirmDialog(const FText& Title, const FText& Description, const TSharedPtr<SWidget> AdditionalContent = nullptr);

	/**
	 * @brief Shows a dialog screen with the Confirm button
	 * @param Title The title of the dialog
	 * @param Description The description of the dialog
	 * @return The button idx that is pressed
	 */
	static EDialogFactoryResult ShowInformationDialog(const FText& Title, const FText& Description, const TSharedPtr<SWidget> AdditionalContent = nullptr);

	/**
	 * @brief Shows a dialog screen with the Yes and No buttons
	 * @param Title The title of the dialog
	 * @param Description The description of the dialog
	 * @return The button idx that is pressed
	 */
	static EDialogFactoryResult ShowYesNoDialog(const FText& Title, const FText& Description, const TSharedPtr<SWidget> AdditionalContent = nullptr);
};
