// Copyright Epic Games, Inc. All Rights Reserved.

#include "IntegrationService.h"

#include "HAL/PlatformProcess.h"
#include "Misc/Paths.h"

#include "CommandLine/CmdLineParameters.h"
#include "Logging/SubmitToolLog.h"
#include "Logic/Services/Interfaces/IChangelistService.h"
#include "Logic/TasksService.h"
#include "Logic/Services/Interfaces/ITagService.h"
#include "Logic/JiraService.h"
#include "Logic/SwarmService.h"
#include "Logic/Services/Interfaces/ICacheDataService.h"
#include "Logic/Services/Interfaces/ISTSourceControlService.h"
#include "Logic/DialogFactory.h"
#include "Internationalization/Regex.h"

#include "Models/IntegrationOptions.h"

#include "Configuration/Configuration.h"

FIntegrationService::FIntegrationService
(const FIntegrationParameters& InParameters,
	TWeakPtr<FSubmitToolServiceProvider> InServiceProvider) :
	Parameters(InParameters),
	ServiceProvider(InServiceProvider)
{
	FString CLID;
	FCmdLineParameters::Get().GetValue(FSubmitToolCmdLine::P4ChangeList, CLID);
	for(FJiraIntegrationField& Field : Parameters.Fields)
	{
		if(Field.Name.IsEmpty())
		{
			Field.Name = FGuid::NewGuid().ToString(EGuidFormats::Digits);
		}

		if(IntegrationOptions.Contains(Field.Name))
		{
			UE_LOGF(LogSubmitTool, Error, "Integration UI has more than one Name value for item %ls, Name property must be unique.", *Field.Name);
		}
		TSharedPtr<FIntegrationOptionBase> NewField;
		switch(Field.Type)
		{
		case EFieldType::Bool:
			NewField = IntegrationOptions.Add(Field.Name, MakeShared<FIntegrationBoolOption>(Field));
			break;

		case EFieldType::Text:
		case EFieldType::MultiText:
		case EFieldType::PerforceUser:
			NewField = IntegrationOptions.Add(Field.Name, MakeShared<FIntegrationTextOption>(Field));
			break;
		case EFieldType::Combo:
			NewField = IntegrationOptions.Add(Field.Name, MakeShared<FIntegrationComboOption>(Field));
			break;

		case EFieldType::UILabel:
		case EFieldType::UISpace:
			NewField = IntegrationOptions.Add(Field.Name, MakeShared<FIntegrationEmptyOption>(Field));
			break;

		default:
			UE_LOGF(LogSubmitTool, Warning, "Unknown integration type %ls", *StaticEnum<EFieldType>()->GetNameStringByValue(static_cast<int64>(Field.Type)));
		}

		if(NewField.IsValid() && !CLID.Equals(TEXT("default")))
		{
			const FString& CachedValue = ServiceProvider.Pin()->GetService<ICacheDataService>()->GetIntegrationFieldValue(CLID, Field.Name);

			if(!CachedValue.IsEmpty())
			{
				NewField->SetValue(CachedValue);
			}
		}

	}
}

void FIntegrationService::RequestIntegration(const FOnBooleanValueChanged OnComplete) const
{
	TSharedPtr<FSwarmService> SwarmService = ServiceProvider.Pin()->GetService<FSwarmService>();
	SwarmService->FetchReview(OnGetReviewComplete::CreateLambda(
	[this, SwarmService, OnComplete](const TUniquePtr<FSwarmReview>& InReview, const FString& InErrorMessage)
	{
		TSharedPtr<IChangelistService> ChangelistService = ServiceProvider.Pin()->GetService<IChangelistService>();
		TSharedPtr<ISTSourceControlService> SourceControlService = ServiceProvider.Pin()->GetService<ISTSourceControlService>();
		TSharedPtr<FJiraService> JiraService = ServiceProvider.Pin()->GetService<FJiraService>();
		TSharedPtr<ITagService> TagService = ServiceProvider.Pin()->GetService<ITagService>();

		UE_LOGF(LogSubmitTool, Log, "Requesting Integration...");
		const FTag* JiraTag = TagService->GetTag(TEXT("#jira"));
		FString JiraTagValue;

		if(JiraTag != nullptr && JiraTag->GetValues().Num() > 0)
		{
			JiraTagValue = JiraTag->GetValues()[0];
		}

		if(!ChangelistService->GetFilesInCL().IsEmpty())
		{
			TFunction<void()> CreateSwarm = [this, JiraTagValue, OnComplete]() {
				TSharedPtr<FSwarmService> SwarmService = ServiceProvider.Pin()->GetService<FSwarmService>();
				TSharedPtr<IChangelistService> ChangelistService = ServiceProvider.Pin()->GetService<IChangelistService>();
				TSharedPtr<ISTSourceControlService> SourceControlService = ServiceProvider.Pin()->GetService<ISTSourceControlService>();
				TSharedPtr<FJiraService> JiraService = ServiceProvider.Pin()->GetService<FJiraService>();
				FString SwarmUrl;
				if (!SwarmService->GetCurrentReviewUrl(SwarmUrl))
				{
					SwarmService->CreateReview(SwarmService->GetUsersInSwarmTag(), OnCreateReviewComplete::CreateLambda([this, ChangelistService, SourceControlService, JiraService, JiraTagValue, OnComplete](bool bSuccess, const FString& Response) {
						if (bSuccess)
						{
							JiraService->GetIssueAndCreateServiceDeskRequest(JiraTagValue, ChangelistService->GetCLDescription(), Response, SourceControlService->GetCurrentStreamName(), IntegrationOptions, OnComplete);
						}
						else
						{
							UE_LOGF(LogSubmitTool, Error, "Failed to create swarm review, Integration request is cancelled");
						}
						}));
				}
				else
				{
					SwarmService->UpdateReviewDescription(FOnBooleanValueChanged::CreateLambda([this, ChangelistService, SourceControlService, JiraService, JiraTagValue, SwarmUrl, OnComplete](bool bSuccess)
						{
							if (!bSuccess)
							{
								UE_LOGF(LogSubmitTool, Warning, "Failed to update swarm review description, Integration will continue with the current swarm description");
							}

							JiraService->GetIssueAndCreateServiceDeskRequest(JiraTagValue, ChangelistService->GetCLDescription(), SwarmUrl, SourceControlService->GetCurrentStreamName(), IntegrationOptions, OnComplete);
						}), ChangelistService->GetCLDescription());
				}
			};

			if(ChangelistService->HasShelvedFiles())
			{
				EDialogFactoryResult Result = FDialogFactory::ShowConfirmDialog(FText::FromString(TEXT("Shelve files")), FText::FromString(TEXT("Submit tool will shelve your local files to update or create the swarm review.\nThis will replace the existing shelve entirely so you could lose any shelved changes that are not local, do you want to continue?")));
				if(Result == EDialogFactoryResult::Confirm)
				{
					ChangelistService->ReplaceShelvedFiles([this, CreateSwarm](const FSCCommand& InCmd)
					{
						if(InCmd.bSuccess)
						{
							CreateSwarm();
						}
						else
						{
							UE_LOGF(LogSubmitTool, Error, "Failed to replace shelve, Integration request is cancelled");
						}
					});
				}
			}
			else
			{
				ChangelistService->CreateShelvedFiles([this, CreateSwarm](const FSCCommand& InCmd)
				{
					if (InCmd.bSuccess)
					{
						CreateSwarm();
					}
					else
					{
						UE_LOGF(LogSubmitTool, Error, "Failed to shelve files, Integration request is cancelled");
					}
				});
			}
		}
		else if(!ChangelistService->GetShelvedFilesInCL().IsEmpty())
		{
			FString SwarmUrl;
			if(SwarmService->GetCurrentReviewUrl(SwarmUrl))
			{
				SwarmService->UpdateReviewDescription(FOnBooleanValueChanged::CreateLambda([this, ChangelistService, SourceControlService, JiraService, JiraTagValue, SwarmUrl, OnComplete](bool bSuccess)
					{
						if(!bSuccess)
						{
							UE_LOGF(LogSubmitTool, Error, "Failed to update swarm review description, Integration will continue with the current swarm description");
						}

						JiraService->GetIssueAndCreateServiceDeskRequest(JiraTagValue, ChangelistService->GetCLDescription(), SwarmUrl, SourceControlService->GetCurrentStreamName(), IntegrationOptions, OnComplete);
					}), ChangelistService->GetCLDescription());

			}
			else
			{
				SwarmService->CreateReview(SwarmService->GetUsersInSwarmTag(), OnCreateReviewComplete::CreateLambda([this, ChangelistService, SourceControlService, JiraService, JiraTagValue, OnComplete](bool bSuccess, const FString& Response) {
					if(bSuccess)
					{
						JiraService->GetIssueAndCreateServiceDeskRequest(JiraTagValue, ChangelistService->GetCLDescription(), Response, SourceControlService->GetCurrentStreamName(), IntegrationOptions, OnComplete);
					}
					else
					{
						UE_LOGF(LogSubmitTool, Error, "Failed to create swarm review, Integration request is cancelled");
					}
					}));
			}
		}
	}));
}

bool FIntegrationService::ValidateIntegrationOptions(bool bSilent) const
{
	bool bGlobalValidation = true;

	const FTag* JiraTag = ServiceProvider.Pin()->GetService<ITagService>()->GetTag(TEXT("#jira"));
	if(JiraTag != nullptr && JiraTag->GetValues().Num() > 0)
	{
		bool bPassesRegex = true;
		FRegexPattern Pattern = FRegexPattern(JiraTag->Definition.Validation.RegexValidation, ERegexPatternFlags::CaseInsensitive);
		for(const FString& Value : JiraTag->GetValues())
		{
			FRegexMatcher regex = FRegexMatcher(Pattern, Value);
			bPassesRegex = regex.FindNext();
			if(!bPassesRegex && !bSilent)
			{
				UE_LOGF(LogSubmitTool, Error, "Value %ls of Jira tag doesn't match the regex pattern %ls", *Value, *JiraTag->Definition.Validation.RegexValidation);
			}
		}

		if(!bPassesRegex || JiraTag->GetValues()[0].Equals("none") || JiraTag->GetValues()[0].Equals("nojira"))
		{
			UE_LOGF(LogSubmitTool, Error, "Validating Integration: Jira \"none\" and \"nojira\" are not allowed for Integrations, please specify a valid JIRA");
			bGlobalValidation = false;
		}
	}
	else
	{
		UE_LOGF(LogSubmitTool, Error, "Validating Integration: Jira is required for Integrations, please specify a valid JIRA");
		bGlobalValidation = false;
	}

	for(const TPair<FString, TSharedPtr<FIntegrationOptionBase>>& IntegrationOption : IntegrationOptions)
	{
		IntegrationOption.Value->bInvalid = false;
	}

	for(const FString& OrValidationGroup : Parameters.OneOfValidationGroups)
	{
		bool bValidGroup = false;
		TArray<FString> GroupIds;

		for(const TPair<FString, TSharedPtr<FIntegrationOptionBase>>& IntegrationOption : IntegrationOptions)
		{
			for(const FString& FieldValidationGroup : IntegrationOption.Value->FieldDefinition.ValidationGroups)
			{
				if(FieldValidationGroup.Equals(OrValidationGroup))
				{
					GroupIds.Add(IntegrationOption.Key);
					FString out;
					bValidGroup = bValidGroup || IntegrationOption.Value->GetJiraValue(out);
					break;
				}
			}

			if(bValidGroup)
			{
				break;
			}
		}

		if(!bValidGroup)
		{
			if(!bSilent)
			{
				UE_LOGF(LogSubmitTool, Error, "Validating Integration: One of these options need to have a value: %ls", *FString::Join(GroupIds, TEXT(", ")));
			}

			for(const FString& ID : GroupIds)
			{
				IntegrationOptions[ID]->bInvalid = true;
			}

			bGlobalValidation = false;
		}
	}


	for(const TPair<FString, TSharedPtr<FIntegrationOptionBase>>& IntegrationOption : IntegrationOptions)
	{
		if(IntegrationOption.Value->FieldDefinition.bRequiredValue)
		{
			bool bValidDependencies = true;

			if(IntegrationOption.Value->FieldDefinition.DependsOn.Num() > 0)
			{
				bValidDependencies = false;
				for(const FString& Dependency : IntegrationOption.Value->FieldDefinition.DependsOn)
				{
					if(IntegrationOptions.Contains(Dependency))
					{
						FString ActualValue;
						if(IntegrationOptions[Dependency]->GetJiraValue(ActualValue) || (!IntegrationOption.Value->FieldDefinition.DependsOnValue.IsEmpty() && IntegrationOption.Value->FieldDefinition.DependsOnValue.Equals(ActualValue)))
						{
							bValidDependencies = true;
							break;
						}
					}
				}
			}

			// Only Validate requirements if the dependencies are valid
			if(bValidDependencies)
			{
				FString out;
				if(!IntegrationOption.Value->GetJiraValue(out))
				{
					IntegrationOption.Value->bInvalid = true;
					bGlobalValidation = false;

					if(!bSilent)
					{
						UE_LOGF(LogSubmitTool, Error, "Validating Integration: Option %ls needs a value.", *IntegrationOption.Key);
					}
				}
			}
		}
	}

	return bGlobalValidation;
}
