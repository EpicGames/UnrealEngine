// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Parameters/SubmitToolParameters.h"
#include "Framework/SlateDelegates.h"
#include "Logic/Services/Interfaces/ISubmitToolService.h"

class FTasksService;
class FChangelistService;
class FIntegrationOptionBase;
class FTagService;
class FJiraService;
class FSwarmService;
class FSubmitToolServiceProvider;

class FIntegrationService final : public ISubmitToolService
{
public:
	FIntegrationService(const FIntegrationParameters& InParameters, TWeakPtr<FSubmitToolServiceProvider> InServiceProvider);

	void RequestIntegration(const FOnBooleanValueChanged OnComplete) const;

	const TMap<FString, TSharedPtr<FIntegrationOptionBase>>& GetIntegrationOptions() const
	{
		return IntegrationOptions;
	}

	bool ValidateIntegrationOptions(bool bSilent) const;

private:
	TMap<FString, TSharedPtr<FIntegrationOptionBase>> IntegrationOptions;

	FIntegrationParameters Parameters;
	TWeakPtr<FSubmitToolServiceProvider> ServiceProvider;
};

Expose_TNameOf(FIntegrationService);