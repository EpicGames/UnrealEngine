// Copyright Epic Games, Inc. All Rights Reserved.

#include "JiraService.h"
#include "GenericPlatform/GenericPlatformProcess.h"
#include "Logic/DialogFactory.h"
#include "Logging/SubmitToolLog.h"
#include "SubmitToolCoreUtils.h"
#include "Logic/Services/Interfaces/ITagService.h"
#include "Logic/PreflightService.h"
#include "Logic/Services/SubmitToolServiceProvider.h"
#include "Logic/Services/Interfaces/ISTSourceControlService.h"

#include "HttpModule.h"
#include "Interfaces/IHttpResponse.h"
#include "Json.h"
#include "Modules/ModuleManager.h"
#include "Parameters/SubmitToolParameters.h"
#include "JsonObjectConverter.h"
#include "Configuration/Configuration.h"


FJiraService::FJiraService(const FJiraParameters& InJiraSettings, const int32 InMaxResults, TWeakPtr<FSubmitToolServiceProvider> InServiceProvider) :
	Definition(InJiraSettings),
	MaxResults(InMaxResults),
	ServiceProvider(InServiceProvider)
{
	if(!Definition.ServerAddress.IsEmpty())
	{
		LoadJiraIssues();
		if (ServiceProvider.Pin()->GetService<ICredentialsService>()->HasJiraCredentials())
		{
			FetchJiraTickets(true, true);
		}
	}
}

FJiraService::~FJiraService()
{
	if(JiraRequest.IsValid())
	{
		JiraRequest->CancelRequest();
	}
	OnJiraIssuesRetrievedCallback.Unbind();
}

void FJiraService::ObtainOAuth(TFunction<void(bool)> InCallback)
{
	if (!Definition.OAuthClientId.IsEmpty() && !bWaitingOAuth)
	{
		if (!HttpServerListener.IsValid() || !HttpServerListener->IsListening())
		{
			bWaitingOAuth = true;
			HttpServerListener = MakeUnique<FHttpServerListener>(TEXT("/submittool/code"), EHttpServerRequestVerbs::VERB_GET, 8451, FHttpRequestHandler::CreateLambda([this, InCallback](const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete)
			{
				const FString Html = TEXT(
					"<html><body>"
					"<h2>Authentication received</h2>"
					"<p>You can return to the application.</p>"
					"</body></html>"
				);

				TUniquePtr<FHttpServerResponse> Response = FHttpServerResponse::Create(
					Html,
					TEXT("text/html; charset=utf-8")
				);

				Response->Code = EHttpServerResponseCodes::Ok;
				OnComplete(MoveTemp(Response));

				if (!Request.QueryParams.Contains(TEXT("state")) || !Request.QueryParams[TEXT("state")].Equals(UserId.ToString(EGuidFormats::Digits)))
				{
					bWaitingOAuth = false;
					UE_LOGF(LogSubmitTool, Error, "Failed to match user code for OAuth, please try to reauthenticate");
					InCallback(false);
					return false;
				}

				if (Request.QueryParams.Contains(TEXT("code")))
				{
					AuthCode = Request.QueryParams[TEXT("code")];
					UE_LOGF(LogSubmitToolDebug, Log, "Retrieved Jira Auth Code");
					ExchangeOAuthCode(InCallback);
				}
				else
				{
					bWaitingOAuth = false;
					UE_LOGF(LogSubmitTool, Error, "Could not obtain Authorization code from Atlassian.");
					InCallback(false);
				}

				return true;
			}));

			if(!HttpServerListener->StartListening())
			{
				UE_LOGF(LogSubmitTool, Error, "Failed to set up Jira OAuth listener.");
				bWaitingOAuth = false;
				InCallback(false);
				return;
			}
		}

		FString Url = FString::Format(TEXT("https://auth.atlassian.com/authorize?audience=api.atlassian.com&client_id={0}&scope=offline_access%20read%3Ajira-work%20read%3Ajira-user&redirect_uri=http%3A%2F%2F127.0.0.1%3A8451%2Fsubmittool%2Fcode&state={1}&response_type=code&prompt=consent"), { Definition.OAuthClientId, UserId.ToString(EGuidFormats::Digits) });
		FPlatformProcess::LaunchURL(*Url, nullptr, nullptr);
		UE_LOGF(LogSubmitToolDebug, Log, "Sending OAuth request for code");
	}
}

bool FJiraService::FetchJiraTickets(bool InForce, bool bInFailSilently)
{
	TSharedPtr<ICredentialsService> CredentialsService = ServiceProvider.Pin()->GetService<ICredentialsService>();
	if(CredentialsService->HasJiraCredentials() && !Definition.ServerAddress.IsEmpty() && !Definition.bDisableJira)
	{
		if (InForce || JiraIssues.Num() == 0)
		{
			QueryIssues(bInFailSilently);
			return true;
		}
	}

	return false;
}

void FJiraService::Reset()
{
	JiraIssues.Reset();
}

void FJiraService::QueryIssues(bool bInFailSilently, bool bAllowTokenRefresh)
{
	TSharedPtr<ICredentialsService> CredentialsService = ServiceProvider.Pin()->GetService<ICredentialsService>();
	if(JiraRequest.IsValid() || Definition.bDisableJira || !CredentialsService->HasJiraCredentials())
	{
		return;
	}

	FHttpModule& HttpModule = FModuleManager::GetModuleChecked<FHttpModule>("HTTP");
	JiraRequest = HttpModule.Get().CreateRequest();

	JiraRequest->OnProcessRequestComplete().BindLambda([this, bInFailSilently, bAllowTokenRefresh](FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded) { QueryIssues_HttpRequestComplete(HttpRequest, HttpResponse, bSucceeded, bInFailSilently, bAllowTokenRefresh); });

	FString Url;
	if (Definition.OAuthClientId.IsEmpty())
	{
		Url = FString::Format(TEXT("https://{0}/rest/api/2/search?maxResults={1}&jql=assignee={2}"), { Definition.ServerAddress, this->MaxResults, CredentialsService->GetUsername()});
		JiraRequest->SetHeader(TEXT("Authorization"), TEXT("Basic ") + CredentialsService->GetEncodedLoginString());
	}
	else
	{
		Url = FString::Format(TEXT("https://api.atlassian.com/ex/jira/{0}/rest/api/2/search/jql?maxResults={1}&jql=assignee%20%3D%20currentUser%28%29%20%20AND%20type%20IN%20(Bug%2C%20Task%2C%20SubTask)%20ORDER%20BY%20created&fields=key,description,summary,priority,status,issuetype,created"), { Definition.CloudID, MaxResults });
		JiraRequest->SetHeader(TEXT("Authorization"), TEXT("Bearer ") + CredentialsService->GetJiraToken()->Access_Token);
	}

	JiraRequest->SetURL(Url);
	JiraRequest->SetVerb(TEXT("GET"));
	bOngoingRequest = true;
	UE_LOGF(LogSubmitToolDebug, Log, "Sending Jira request for tickets");
	JiraRequest->ProcessRequest();
}

void FJiraService::QueryIssues_HttpRequestComplete(FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded, bool bFailSilently, bool bAllowTokenRefresh)
{
	JiraRequest = nullptr;
	bOngoingRequest = false;
	TSharedPtr<ICredentialsService> CredentialsService = ServiceProvider.Pin()->GetService<ICredentialsService>();

	if (!bSucceeded)
	{
		UE_LOGF(LogSubmitToolDebug, Error, "Unable to retrieve JIRA issues at the moment.");
		OnJiraIssuesRetrievedCallback.ExecuteIfBound(false);
		return;
	}

	if (HttpResponse.IsValid())
	{
		if (EHttpResponseCodes::IsOk(HttpResponse->GetResponseCode()))
		{
			UE_LOGF(LogSubmitToolDebug, Log, "Successfully connected to Jira");

			FString ResponseStr = HttpResponse->GetContentAsString();

			TSharedPtr<FJsonObject> RootJsonObject;
			TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(ResponseStr);
			FJsonSerializer::Deserialize(Reader, RootJsonObject);

			if (RootJsonObject.IsValid())
			{
				const TArray<TSharedPtr<FJsonValue>>* Issues;
				if (RootJsonObject->TryGetArrayField(TEXT("issues"), Issues))
				{
					UE_LOGF(LogSubmitToolDebug, Log, "Retrieved %d issues for username %ls", Issues->Num(), *CredentialsService->GetUsername());

					JiraIssues.Reset();

					for (const TSharedPtr<FJsonValue>& ArrVal : *Issues)
					{
						if (!ArrVal.IsValid())
						{
							continue;
						}

						const TSharedPtr<FJsonObject>* IssueObject;
						if (ArrVal->TryGetObject(IssueObject))
						{
							FJiraIssue Issue;
							if(ParseJsonObject(IssueObject, Issue))
							{
								JiraIssues.Add(Issue.Key, Issue);
							}
							else
							{
								FString StringValue;
								TSharedRef<TJsonWriter<TCHAR>> JsonWriter = TJsonWriterFactory<>::Create(&StringValue);
								FJsonSerializer::Serialize(IssueObject->ToSharedRef(), JsonWriter);
								UE_LOGF(LogSubmitToolDebug, Log, "Failed to parse issue %ls", *StringValue);
							}
						}
						else
						{
							UE_LOGF(LogSubmitToolDebug, Log, "Failed to parse issue object");
						}
					}

					SaveJiraIssues();
				}
			}
		}
		else
		{
			if (HttpResponse->GetResponseCode() == EHttpResponseCodes::Denied && !Definition.OAuthClientId.IsEmpty() && bAllowTokenRefresh)
			{
				UE_LOGF(LogSubmitToolDebug, Log, "Jira access token expired (401), attempting token refresh.");
				RefreshOAuthToken([this, bFailSilently](bool bRefreshSucceeded)
				{
					if (bRefreshSucceeded)
					{
						QueryIssues(bFailSilently, /*bAllowTokenRefresh=*/false);
					}
					else
					{
						UE_LOGF(LogSubmitTool, Error, "Jira token refresh failed, re-authentication required.");
						ServiceProvider.Pin()->GetService<ICredentialsService>()->InvalidateCredentials();
						OnJiraIssuesRetrievedCallback.ExecuteIfBound(false);
					}
				});
				return;
			}

			if (HttpResponse->GetResponseCode() == EHttpResponseCodes::TooManyRequests)
			{
				UE_LOGF(LogSubmitTool, Warning, "Jira request rate limited (429), please try again later.");
				OnJiraIssuesRetrievedCallback.ExecuteIfBound(false);
				return;
			}

			if (bFailSilently)
			{
				UE_LOGF(LogSubmitToolDebug, Error, "Suppressed Log: Jira Request failed with error code %d, please make sure you're logging with the right credentials. if your Okta password expired recently, make sure you log into JIRA via browser at least once.", HttpResponse->GetResponseCode());
			}
			else
			{
				UE_LOGF(LogSubmitTool, Error, "Jira Request failed with error code %d, please make sure you're logging with the right credentials. if your Okta password expired recently, make sure you log into JIRA via browser at least once.", HttpResponse->GetResponseCode());
			}
			CredentialsService->InvalidateCredentials();
		}
	}

	OnJiraIssuesRetrievedCallback.ExecuteIfBound(HttpResponse.IsValid() && EHttpResponseCodes::IsOk(HttpResponse->GetResponseCode()));
}

bool FJiraService::ParseJsonObject(const TSharedPtr<FJsonObject>* InJsonObject, FJiraIssue& OutJiraIssue) const
{
	if(InJsonObject->IsValid())
	{
		FString Key{ "" };
		FString Summary{ "" };
		FString Description{ "" };
		FString PriorityName{ "" };
		FString StatusName{ "" };
		FString IssueTypeName{ "" };

		InJsonObject->Get()->TryGetStringField(TEXT("key"), Key);

		const TSharedPtr<FJsonObject>* FieldsObject;
		if(InJsonObject->Get()->TryGetObjectField(TEXT("fields"), FieldsObject))
		{
			if(FieldsObject->IsValid())
			{
				FieldsObject->Get()->TryGetStringField(TEXT("description"), Description);
				FieldsObject->Get()->TryGetStringField(TEXT("summary"), Summary);

				const TSharedPtr<FJsonObject>* PriorityObject;
				if(FieldsObject->Get()->TryGetObjectField(TEXT("priority"), PriorityObject))
				{
					if(PriorityObject->IsValid())
					{
						PriorityObject->Get()->TryGetStringField(TEXT("name"), PriorityName);
					}
				}

				const TSharedPtr<FJsonObject>* StatusObject;
				if(FieldsObject->Get()->TryGetObjectField(TEXT("status"), StatusObject))
				{
					if(StatusObject->IsValid())
					{
						StatusObject->Get()->TryGetStringField(TEXT("name"), StatusName);
					}
				}

				const TSharedPtr<FJsonObject>* IssueTypeObject;
				if(FieldsObject->Get()->TryGetObjectField(TEXT("issuetype"), IssueTypeObject))
				{
					if(IssueTypeObject->IsValid())
					{
						IssueTypeObject->Get()->TryGetStringField(TEXT("name"), IssueTypeName);
					}
				}
			}
		}

		if(!Key.IsEmpty() && !this->JiraIssues.Contains(Key))
		{
			FString Link = Definition.ServerAddress + TEXT("/browse/") + Key;
			OutJiraIssue = FJiraIssue(Key, Summary, Link, Description, PriorityName, StatusName, IssueTypeName);
			return true;
		}
	}
	return false;
}

constexpr int JiraIssuesDatVersion = 1;
void FJiraService::SaveJiraIssues() const
{
	FArchive* File = IFileManager::Get().CreateFileWriter(*GetJiraIssuesFilepath(), EFileWrite::FILEWRITE_EvenIfReadOnly);

	if (File != nullptr)
	{
		int32 Version = JiraIssuesDatVersion;
		*File << Version;

		int32 Size = this->JiraIssues.Num();
		*File << Size;

		for (TPair<FString, FJiraIssue> Issue : this->JiraIssues)
		{
			FJiraIssue::StaticStruct()->SerializeBin(*File, &Issue.Value);
		}

		File->Close();
		delete File;
		File = nullptr;
	}
	else
	{
		UE_LOGF(LogSubmitTool, Warning, "Could not create file '%ls'.", *GetJiraIssuesFilepath());
	}
}

void FJiraService::LoadJiraIssues()
{
	// do not load the issues if there is no credentials
	if (!ServiceProvider.Pin()->GetService<ICredentialsService>()->HasJiraCredentials())
	{
		return;
	}

	if (IFileManager::Get().FileExists(*GetJiraIssuesFilepath()))
	{
		FArchive* File = IFileManager::Get().CreateFileReader(*GetJiraIssuesFilepath());

		if (File != nullptr)
		{
			this->JiraIssues.Reset();

			int32 Version;
			*File << Version;

			// Check Versions here
			if (Version != JiraIssuesDatVersion)
			{
				UE_LOGF(LogSubmitToolDebug, Warning, "Unexpected Jira Issues Version, aborting issues loading.");
				File->Close();
				delete File;
				File = nullptr;
				return;
			}

			int32 Size = 0;
			*File << Size;

			for (int32 Idx = 0; Idx < Size; Idx++)
			{
				FJiraIssue Issue;
				FJiraIssue::StaticStruct()->SerializeBin(*File, &Issue);

				if (!this->JiraIssues.Contains(Issue.Key))
				{
					this->JiraIssues.Add(Issue.Key, Issue);
				}
			}

			File->Close();
			delete File;
			File = nullptr;
		}
		else
		{
			UE_LOGF(LogSubmitTool, Warning, "Could not read file '%ls'.", *GetJiraIssuesFilepath());
		}
	}
	else
	{
		UE_LOGF(LogSubmitToolDebug, Log, "File %ls does not exists, no issues loaded", *GetJiraIssuesFilepath())
	}
}

void FJiraService::ExchangeOAuthCode(TFunction<void(bool)> InCallback)
{
	FHttpModule& HttpModule = FModuleManager::LoadModuleChecked<FHttpModule>("HTTP");
	FHttpRequestRef HttpRequest = HttpModule.Get().CreateRequest();

	HttpRequest->SetURL(TEXT("https://auth.atlassian.com/oauth/token"));

	HttpRequest->SetHeader(TEXT("Content-Type"), TEXT("application/json"));
	HttpRequest->SetVerb(TEXT("POST"));


	TSharedPtr<FJsonObject> RequestJson = MakeShared<FJsonObject>();

	RequestJson->SetStringField(TEXT("grant_type"), TEXT("authorization_code"));
	RequestJson->SetStringField(TEXT("client_id"), Definition.OAuthClientId);
	RequestJson->SetStringField(TEXT("client_secret"), Definition.OAuthClientSecret);
	RequestJson->SetStringField(TEXT("code"), AuthCode);
	RequestJson->SetStringField(TEXT("redirect_uri"), TEXT("http://127.0.0.1:8452/submittool/token"));

	FString BodyString;
	TSharedRef<TJsonWriter<TCHAR>> JsonWriter = TJsonWriterFactory<>::Create(&BodyString);
	FJsonSerializer::Serialize(RequestJson.ToSharedRef(), JsonWriter);
	HttpRequest->SetContentAsString(BodyString);

	HttpRequest->OnProcessRequestComplete().BindLambda([this, InCallback](FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded) {
		AuthCode.Empty();
		bWaitingOAuth = false;

		if (!bSucceeded)
		{			
			UE_LOGF(LogSubmitTool, Error, "Failed to exchange OAuth code for Token.")
			InCallback(false);
			return;
		}

		if (HttpResponse.IsValid())
		{
			if (EHttpResponseCodes::IsOk(HttpResponse->GetResponseCode()))
			{
				FString ResponseStr = HttpResponse->GetContentAsString();
				FJiraToken Token;
				if (FJsonObjectConverter::JsonObjectStringToUStruct<FJiraToken>(*ResponseStr, &Token))
				{
					ServiceProvider.Pin()->GetService<ICredentialsService>()->SetJiraToken(MoveTemp(Token));
					UE_LOGF(LogSubmitToolDebug, Log, "Obtained Jira Access Token correctly.");
					InCallback(true);
					return;
				}
				else
				{
					UE_LOGF(LogSubmitTool, Error, "Failed to parse Jira OAuth Token.")
					UE_LOGF(LogSubmitToolDebug, Error, "%ls", *HttpResponse->GetContentAsString());

				}
			}
			else
			{
				UE_LOGF(LogSubmitTool, Error, "Received failed %d jira response.", HttpResponse->GetResponseCode())
				UE_LOGF(LogSubmitToolDebug, Error, "%ls", *HttpResponse->GetContentAsString());
			}
		}
		else
		{
			UE_LOGF(LogSubmitTool, Error, "Invalid jira response.");
		}

		InCallback(false);
	});

	HttpRequest->ProcessRequest();
}

void FJiraService::RefreshOAuthToken(TFunction<void(bool)> InCallback)
{
	TSharedPtr<ICredentialsService> CredentialsService = ServiceProvider.Pin()->GetService<ICredentialsService>();
	const TUniquePtr<FJiraToken>& Token = CredentialsService->GetJiraToken();

	if (!Token.IsValid() || Token->Refresh_Token.IsEmpty())
	{
		UE_LOGF(LogSubmitTool, Warning, "No Jira refresh token available, re-authentication required.");
		InCallback(false);
		return;
	}

	FHttpModule& HttpModule = FModuleManager::LoadModuleChecked<FHttpModule>("HTTP");
	FHttpRequestRef HttpRequest = HttpModule.Get().CreateRequest();

	HttpRequest->SetURL(TEXT("https://auth.atlassian.com/oauth/token"));
	HttpRequest->SetHeader(TEXT("Content-Type"), TEXT("application/json"));
	HttpRequest->SetVerb(TEXT("POST"));

	TSharedPtr<FJsonObject> RequestJson = MakeShared<FJsonObject>();
	RequestJson->SetStringField(TEXT("grant_type"), TEXT("refresh_token"));
	RequestJson->SetStringField(TEXT("client_id"), Definition.OAuthClientId);
	RequestJson->SetStringField(TEXT("client_secret"), Definition.OAuthClientSecret);
	RequestJson->SetStringField(TEXT("refresh_token"), Token->Refresh_Token);

	FString BodyString;
	TSharedRef<TJsonWriter<TCHAR>> JsonWriter = TJsonWriterFactory<>::Create(&BodyString);
	FJsonSerializer::Serialize(RequestJson.ToSharedRef(), JsonWriter);
	HttpRequest->SetContentAsString(BodyString);

	HttpRequest->OnProcessRequestComplete().BindLambda([this, InCallback](FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded)
	{
		bool bSuccess = false;
		if (!bSucceeded)
		{
			UE_LOGF(LogSubmitTool, Error, "Failed to refresh Jira OAuth token.");			
		}
		else
		{
			if (HttpResponse.IsValid())
			{
				if (EHttpResponseCodes::IsOk(HttpResponse->GetResponseCode()))
				{
					FJiraToken NewToken;
					if (FJsonObjectConverter::JsonObjectStringToUStruct<FJiraToken>(*HttpResponse->GetContentAsString(), &NewToken))
					{
						ServiceProvider.Pin()->GetService<ICredentialsService>()->SetJiraToken(MoveTemp(NewToken));
						UE_LOGF(LogSubmitTool, Log, "Jira access token refreshed successfully.");
						bSuccess = true;
					}
					else
					{
						UE_LOGF(LogSubmitTool, Error, "Failed to parse refreshed Jira OAuth token.");
						UE_LOGF(LogSubmitToolDebug, Error, "%ls", *HttpResponse->GetContentAsString());
					}
				}
				else
				{
					UE_LOGF(LogSubmitTool, Error, "Jira token refresh failed with code %d.", HttpResponse->GetResponseCode());
					UE_LOGF(LogSubmitToolDebug, Error, "%ls", *HttpResponse->GetContentAsString());
				}
			}
		}

		InCallback(bSuccess);
	});

	HttpRequest->ProcessRequest();
}

const FString FJiraService::GetJiraIssuesFilepath() const
{
	return FPaths::Combine(FSubmitToolCoreUtils::GetLocalAppDataPath(), TEXT("SubmitTool"), TEXT("jira.issues.dat"));
}

void FJiraService::GetIssueAndCreateServiceDeskRequest(const FString& Key, const FString& Description, const FString& SwarmURL, const FString& InCurrentStream, const TMap<FString, TSharedPtr<FIntegrationOptionBase>>& InIntegrationOptions, const FOnBooleanValueChanged OnComplete)
{
	TSharedPtr<ICredentialsService> CredentialsService = ServiceProvider.Pin()->GetService<ICredentialsService>();

	if(Key.IsEmpty() || Key.Equals(TEXT("none"), ESearchCase::IgnoreCase) || !CredentialsService->HasJiraCredentials())
	{
		CreateServiceDeskRequest(TSharedPtr<FJsonObject>(), Description, SwarmURL, InCurrentStream, InIntegrationOptions, OnComplete);
		return;
	}

	UE_LOGF(LogSubmitTool, Log, "Requesting Information for linked Jira issue %ls", *Key);
	
	FHttpModule& HttpModule = FModuleManager::LoadModuleChecked<FHttpModule>("HTTP");
	FHttpRequestRef HttpRequest = HttpModule.Get().CreateRequest();

	HttpRequest->OnProcessRequestComplete().BindLambda([=, this](FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded) {FJiraService::GetIssueAndCreateServiceDeskRequest_HttpRequestComplete(HttpRequest, HttpResponse, bSucceeded, Description, SwarmURL, InCurrentStream, InIntegrationOptions, OnComplete); });

	FString Url;
	if (Definition.OAuthClientId.IsEmpty())
	{
		Url = FString::Format(TEXT("https://{0}/rest/api/2/issue/{1}"), { Definition.ServerAddress, Key });
		HttpRequest->SetHeader(TEXT("Authorization"), TEXT("Basic ") + CredentialsService->GetEncodedLoginString());
	}
	else
	{
		Url = FString::Format(TEXT("https://api.atlassian.com/ex/jira/{0}/rest/api/2/issue/{1}"), { Definition.CloudID, Key });
		if (CredentialsService->GetJiraToken().IsValid())
		{
			HttpRequest->SetHeader(TEXT("Authorization"), TEXT("Bearer ") + ServiceProvider.Pin()->GetService<ICredentialsService>()->GetJiraToken()->Access_Token);
		}
	}

	HttpRequest->SetURL(Url);
	HttpRequest->SetVerb(TEXT("GET"));

	HttpRequest->ProcessRequest();
}

void FJiraService::GetIssueAndCreateServiceDeskRequest_HttpRequestComplete(FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded, const FString& Description, const FString& SwarmURL, const FString& InCurrentStream, const TMap<FString, TSharedPtr<FIntegrationOptionBase>>& InIntegrationOptions, const FOnBooleanValueChanged OnComplete)
{
	TSharedPtr<FJsonObject> RootJsonObject;
	if (!bSucceeded)
	{
		if(HttpResponse.IsValid())
		{
			UE_LOGF(LogSubmitTool, Log, "Unable to retrieve Base JIRA issue information. Summary will be created with the current CL description instead. Failed with code %d", HttpResponse->GetResponseCode());
			UE_LOGF(LogSubmitToolDebug, Log, "Unable to retrieve JIRA issue information. Summary will be created with the current CL description instead. Failed with code %d\nResponse: %ls", HttpResponse->GetResponseCode(), *HttpResponse->GetContentAsString());
		}
		else
		{
			UE_LOGF(LogSubmitTool, Warning, "Unable to retrieve Base JIRA issue information. Unknown failure");
		}
	}
	else
	{
		if(HttpResponse.IsValid())
		{
			if(EHttpResponseCodes::IsOk(HttpResponse->GetResponseCode()))
			{
				FString ResponseStr = HttpResponse->GetContentAsString();
				UE_LOGF(LogSubmitToolDebug, Log, "Obtained information from Jira Issue %ls", *ResponseStr);

				TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(ResponseStr);
				FJsonSerializer::Deserialize(Reader, RootJsonObject);
			}
			else
			{
				UE_LOGF(LogSubmitTool, Warning, "Unable to retrieve Base JIRA issue information.");
				UE_LOGF(LogSubmitToolDebug, Warning, "Unable to retrieve Base JIRA issue information. Failed with code %d\nResponse: %ls", HttpResponse->GetResponseCode(), *HttpResponse->GetContentAsString());
			}
		}
	}

	// Call the function to actually create the service desk request with the required information
	CreateServiceDeskRequest(RootJsonObject, Description, SwarmURL, InCurrentStream, InIntegrationOptions, OnComplete);
}

void FJiraService::CreateServiceDeskRequest(TSharedPtr<FJsonObject> InBaseJiraJsonObject, const FString& Description, const FString& SwarmURL, const FString& InCurrentStream, const TMap<FString, TSharedPtr<FIntegrationOptionBase>>& InIntegrationOptions, const FOnBooleanValueChanged OnComplete)
{
	UE_LOGF(LogSubmitTool, Log, "Requesting creation of Jira ServiceDesk ticket...");
	FHttpModule& HttpModule = FModuleManager::LoadModuleChecked<FHttpModule>("HTTP");
	ServiceDeskRequest = HttpModule.Get().CreateRequest();

	ServiceDeskRequest->OnProcessRequestComplete().BindLambda([this, OnComplete](FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded) { CreateServiceDeskRequest_HttpRequestComplete(HttpRequest, HttpResponse, bSucceeded, OnComplete); });
	FString Url = FString::Format(TEXT("https://{0}/rest/servicedeskapi/request"), { Definition.ServerAddress });

	ServiceDeskRequest->SetURL(Url);
	ServiceDeskRequest->SetHeader(TEXT("Authorization"), FString::Format(TEXT("Basic {0}"), { Definition.ServiceDeskToken }));
	ServiceDeskRequest->SetHeader(TEXT("Content-Type"), TEXT("application/json"));
	ServiceDeskRequest->SetVerb(TEXT("POST"));
	
	TSharedPtr<FJsonObject> RequestJson = MakeShared<FJsonObject>();

	// These values should probably be put inside the .ini file
	RequestJson->SetNumberField(TEXT("serviceDeskId"), Definition.ServiceDeskID);
	RequestJson->SetNumberField(TEXT("requestTypeId"), Definition.RequestFormID);


	TSharedPtr<FJsonObject> RequestFieldValuesJson = MakeShared<FJsonObject>();

	if(InBaseJiraJsonObject.IsValid())
	{
		TSharedPtr<FJsonObject> BaseJiraFields = InBaseJiraJsonObject->GetObjectField(TEXT("fields"));
		RequestFieldValuesJson->SetStringField(TEXT("summary"), BaseJiraFields->GetStringField(TEXT("summary")));
	}
	else
	{
		FString Summary = Description.Left(50).Replace(TEXT("\n"), TEXT(" ")).Replace(TEXT("\r"), TEXT(""));
		RequestFieldValuesJson->SetStringField(TEXT("summary"), Summary);

	}

	RequestFieldValuesJson->SetStringField(TEXT("description"), Description);

	if(!SwarmURL.IsEmpty() && !Definition.SwarmUrlField.IsEmpty())
	{
		RequestFieldValuesJson->SetStringField(Definition.SwarmUrlField, SwarmURL);
	}

	if(!InCurrentStream.IsEmpty() && !Definition.StreamField.IsEmpty())
	{
		RequestFieldValuesJson->SetStringField(Definition.StreamField, InCurrentStream);
	}

	if(!Definition.PreflightField.IsEmpty())
	{		
		const FTag* PreflightTag = ServiceProvider.Pin()->GetService<ITagService>()->GetTagOfSubtype(TEXT("preflight"));
		if (PreflightTag != nullptr && PreflightTag->GetValues().Num() > 0)
		{
			FString PreflightTagValue = PreflightTag->GetValues()[0];

			if(!PreflightTagValue.IsEmpty())
			{
				if(!PreflightTagValue.Contains(TEXT("/job/")))
				{
					RequestFieldValuesJson->SetStringField(Definition.PreflightField, FString::Format(TEXT("{0}job/{1}"), { ServiceProvider.Pin()->GetService<FPreflightService>()->GetHordeServerAddress(), PreflightTagValue }));
				}
				else
				{
					RequestFieldValuesJson->SetStringField(Definition.PreflightField, PreflightTagValue);
				}
			}
		}
	}

	if (!Definition.JiraField.IsEmpty())
	{
		const FTag* JiraTag = ServiceProvider.Pin()->GetService<ITagService>()->GetTagOfType(TEXT("JiraIssue"));
		if (JiraTag != nullptr && JiraTag->GetValues().Num() > 0)
		{
			TArray<FString> URLs;
			for (const FString& TagValue : JiraTag->GetValues())
			{
				URLs.Add(FString::Format(TEXT("https://{0}/browse/{1}"), { Definition.ServerAddress, TagValue }));
			}

			RequestFieldValuesJson->SetStringField(Definition.JiraField, FString::Join(URLs, TEXT("\n")));
		}
	}

	FString Requestor = FConfiguration::Substitute(TEXT("$(USER)"));
	TSharedPtr<FUserData> LocalUserData = ServiceProvider.Pin()->GetService<ISTSourceControlService>()->GetUserDataFromCache(Requestor);
	if (LocalUserData.IsValid())
	{
		Requestor = LocalUserData.Get()->Email;
	}

	RequestFieldValuesJson->SetStringField(Definition.RequestorField, Requestor);

	for(const TPair<FString, TSharedPtr<FIntegrationOptionBase>>& Pair : InIntegrationOptions)
	{
		FString Value;
		if(!Pair.Value->GetJiraValue(Value))
		{
			continue;
		}

		TSharedPtr<FJsonObject> JiraObject = MakeShared<FJsonObject>();
		TArray<TSharedPtr<FJsonValue>> JiraArrayObject;
		JiraObject->SetStringField(TEXT("value"), Value);

		switch(Pair.Value->FieldDefinition.JiraType)
		{
		case EJiraFieldType::Object:
			RequestFieldValuesJson->SetObjectField(Pair.Value->FieldDefinition.Id, JiraObject);
			break;

		case EJiraFieldType::Array:
			const TArray<TSharedPtr<FJsonValue>>* ExistingJiraArrayObjectPtr;
			if(RequestFieldValuesJson->TryGetArrayField(Pair.Value->FieldDefinition.Id, ExistingJiraArrayObjectPtr))
			{
				JiraArrayObject = *ExistingJiraArrayObjectPtr;
			}

			JiraArrayObject.Add(MakeShared<FJsonValueObject>(JiraObject));
			RequestFieldValuesJson->SetArrayField(Pair.Value->FieldDefinition.Id, JiraArrayObject);
			break;

		case EJiraFieldType::String:
			RequestFieldValuesJson->SetStringField(Pair.Value->FieldDefinition.Id, Value);
			break;
		}
	}

	// JW: I think they decided that any jira tickets referenced in the changelist should be added as additional URLs
	RequestJson->SetObjectField(TEXT("requestFieldValues"), RequestFieldValuesJson);

	FString BodyString;
	TSharedRef<TJsonWriter<TCHAR>> JsonWriter = TJsonWriterFactory<>::Create(&BodyString);
	FJsonSerializer::Serialize(RequestJson.ToSharedRef(), JsonWriter);

	UE_LOGF(LogSubmitToolDebug, Log, "Create Jira request body:\n%ls", *BodyString);

	ServiceDeskRequest->SetContentAsString(BodyString);

	ServiceDeskRequest->ProcessRequest();
}

void FJiraService::CreateServiceDeskRequest_HttpRequestComplete(FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded, const FOnBooleanValueChanged OnComplete)
{
	if(!bSucceeded)
	{
		if(HttpResponse.IsValid())
		{
			UE_LOGF(LogSubmitTool, Error, "Unable to create JIRA service desk. Failed with code %d\nResponse: %ls", HttpResponse->GetResponseCode(), *HttpResponse->GetContentAsString());
		}
		else
		{
			UE_LOGF(LogSubmitTool, Error, "Unable to create JIRA service desk. Unknown failure");
		}
		OnComplete.ExecuteIfBound(false);
		return;
	}

	if (HttpResponse.IsValid())
	{
		if (EHttpResponseCodes::IsOk(HttpResponse->GetResponseCode()))
		{
			TSharedRef<TJsonReader<TCHAR>> Reader = TJsonReaderFactory<TCHAR>::Create(HttpResponse->GetContentAsString());
			TSharedPtr<FJsonObject> JsonObj;
			if(!FJsonSerializer::Deserialize(Reader, JsonObj))
			{
				UE_LOGF(LogSubmitTool, Error, "Unable to deserialize swarm create response");
				OnComplete.ExecuteIfBound(false);
				return;
			}

			FString CreatedTicketId = JsonObj->GetStringField(TEXT("issueKey"));
			FString WebLink;
			const TSharedPtr<FJsonObject>* Links;
			if(JsonObj->TryGetObjectField(TEXT("_links"), Links))
			{
				WebLink = Links->Get()->GetStringField(TEXT("web"));
			}

			// If the service desk request was created successfully
			UE_LOGF(LogSubmitTool, Log, "Jira service desk ticket creation was successful: %ls %ls", *CreatedTicketId, *WebLink);
			UE_LOGF(LogSubmitToolDebug, Log, "Jira service desk ticket creation was successful\n%ls", *HttpResponse->GetContentAsString());
			FDialogFactory::ShowInformationDialog(FText::FromString(TEXT("Integration Request Successful")), FText::FromString(TEXT("The Integration has sucessfully been requested!")));			
			OnComplete.ExecuteIfBound(true);
		}
		else
		{
			UE_LOGF(LogSubmitTool, Error, "Unable to create JIRA service desk. Failed with code %d\nResponse: %ls", HttpResponse->GetResponseCode(), *HttpResponse->GetContentAsString());
			FDialogFactory::ShowInformationDialog(FText::FromString(TEXT("Integration Request FAILED")), FText::FromString(TEXT("Unable to create JIRA service desk ticket.\nPlease check the logs for more info.")));
			OnComplete.ExecuteIfBound(false);
		}
	}
	else
	{
		OnComplete.ExecuteIfBound(false);
	}
}
