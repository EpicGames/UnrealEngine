// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Logic/CredentialsService.h"
#include "Models/JiraIssue.h"
#include "Models/IntegrationOptions.h"
#include "Interfaces/IHttpRequest.h"
#include "Framework/SlateDelegates.h"
#include "Logic/Services/Interfaces/ISubmitToolService.h"

DECLARE_DELEGATE_OneParam(FOnJiraIssuesRetrieved, bool /*bValidResponse*/)

enum EJiraServerRequest
{
	AuthorizationCode,
	AccessToken

};

struct FJiraParameters;
class FJsonObject;
class FJsonValue;
class FSubmitToolServiceProvider;

class FJiraService final : public ISubmitToolService
{
public:
	FJiraService() = delete;
	FJiraService(const FJiraParameters& InJiraSettings, const int32 InMaxResults, TWeakPtr<FSubmitToolServiceProvider> InServiceProvider);
	~FJiraService();
	void ObtainOAuth(TFunction<void(bool)> InCallback);

	bool FetchJiraTickets(bool InForce = false, bool bInFailSilently = false);
	void Reset();

	const TMap<FString, FJiraIssue>& GetIssues() { return this->JiraIssues; }

	FOnJiraIssuesRetrieved OnJiraIssuesRetrievedCallback;

	bool bOngoingRequest = false;
	bool IsBlockingRequestRunning()
	{
		return ServiceDeskRequest.IsValid() && ServiceDeskRequest->GetStatus() == EHttpRequestStatus::Processing;
	}

	void GetIssueAndCreateServiceDeskRequest(const FString& Key, const FString& Description, const FString& SwarmURL, const FString& InCurrentStream, const TMap<FString, TSharedPtr<FIntegrationOptionBase>>& InIntegrationOptions, const FOnBooleanValueChanged OnComplete);
	void CreateServiceDeskRequest(TSharedPtr<FJsonObject> InBaseJiraJsonObject, const FString& Description, const FString& SwarmURL, const FString& InCurrentStream, const TMap<FString, TSharedPtr<FIntegrationOptionBase>>& InIntegrationOptions, const FOnBooleanValueChanged OnComplete);
private:
	void QueryIssues(bool bInFailSilently = false, bool bAllowTokenRefresh = true);
	void QueryIssues_HttpRequestComplete(FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded, bool bFailSilently, bool bAllowTokenRefresh);
	void RefreshOAuthToken(TFunction<void(bool)> InCallback);

	bool ParseJsonObject(const TSharedPtr<FJsonObject>* InJsonObject, FJiraIssue& OutJiraIssue) const;

	const FString GetJiraIssuesFilepath() const;

	void SaveJiraIssues() const;
	void LoadJiraIssues();
	
	void ExchangeOAuthCode(TFunction<void(bool)> InCallback);

	void CreateServiceDeskRequest_HttpRequestComplete(FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded, const FOnBooleanValueChanged OnComplete);
	void GetIssueAndCreateServiceDeskRequest_HttpRequestComplete(FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded, const FString& Description, const FString& SwarmURL, const FString& InCurrentStream, const TMap<FString, TSharedPtr<FIntegrationOptionBase>>& InIntegrationOptions, const FOnBooleanValueChanged OnComplete);

private:
	FGuid UserId = FGuid::NewGuid();
	FString AuthCode;
	const FJiraParameters Definition;
	FHttpRequestPtr JiraRequest = nullptr;
	FHttpRequestPtr ServiceDeskRequest = nullptr;
	const int32 MaxResults;
	TMap<FString, FJiraIssue> JiraIssues;

	TUniquePtr<FHttpServerListener> HttpServerListener;
	bool bWaitingOAuth = false;

	TWeakPtr<FSubmitToolServiceProvider> ServiceProvider;
};

Expose_TNameOf(FJiraService);