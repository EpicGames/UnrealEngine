// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Parameters/SubmitToolParameters.h"
#include "Logic/ProcessWrapper.h"
#include "Tasks/Task.h"
#include "Logic/Services/Interfaces/ILockdownService.h"
#include "Memory/SharedBuffer.h"


DECLARE_DELEGATE_OneParam(FOnConfigFileRetrieved, bool /*bValid*/)

class FSubmitToolServiceProvider;

class FP4LockdownService : public ILockdownService
{	
public:

	FP4LockdownService() = delete;
	FP4LockdownService(const FP4LockdownParameters& InParameters, TWeakPtr<FSubmitToolServiceProvider> InServiceProvider);
	virtual FSubmitToolLockdownData ArePathsInLockdown(const TArray<FSCFileRef>& InPaths, const TSet<FString>& InDynamicFiles = TSet<FString>()) override;
	virtual bool IsBlockingOperationRunning() const override
	{
		return DownloadTask.IsValid() && !DownloadTask.IsCompleted();
	}

private:

	enum EAllowListState
	{
		Missing,
		Cached,
		Downloaded
	};

	struct FAllowListData
	{
		bool bIsEnabled = true;
		FString FileId;
		FString GroupName;
		FString Message;
		ELockdownType LockdownType = ELockdownType::None;
		FString ValidTag;
		TSet<FString, FCaseInsensitiveStringKeyFuncs> AllowListers;
		TSet<FString, FCaseInsensitiveStringKeyFuncs> Members;
		TArray<TPair<bool, FString>> Views;
	};


	struct FOverrideData : public FAllowListData
	{
		TSet<FString> Sections;
	};
	FSubmitToolLockdownData IsPathInLockdown(const FSCFileRef& InPath, const TSet<FString>& InDynamicFiles) const;
	void DownloadAllowListData();
	bool WaitForAllowListData();
	void ParseAllowListData();
	void ParseAllowListFiles(const TMap<FString, FString>& InFiles, bool bEnabled = true);

	void GetAdditionalHardlockedPaths();
	FString GetFilePath(const FString& InConfigId);

	TMap<FString, FSharedBuffer> DownloadedFiles;
	UE::Tasks::TTask<bool> DownloadTask;
	TWeakPtr<FSubmitToolServiceProvider> ServiceProvider;
	const FP4LockdownParameters& Parameters;
	TArray<FAllowListData> AllowListData;
	TArray<FOverrideData> OverrideData;
	TArray<FString> AdditionalHardlocks;
	EAllowListState AllowListState = EAllowListState::Missing;
	FCriticalSection Mutex;
};

Expose_TNameOf(FP4LockdownService);
