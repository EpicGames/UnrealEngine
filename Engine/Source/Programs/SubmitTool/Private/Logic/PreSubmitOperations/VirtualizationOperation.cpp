// Copyright Epic Games, Inc. All Rights Reserved.

#include "VirtualizationOperation.h"

#include "AnalyticsEventAttribute.h"
#include "Configuration/Configuration.h"
#include "Logic/Validators/ValidatorDefinition.h"
#include "Logic/Validators/ValidatorFactory.h"
#include "Misc/Paths.h"
#include "Misc/StringOutputDevice.h"
#include "Models/ModelInterface.h"

REGISTER_VALIDATOR_TYPE(SubmitToolParseConstants::VirtualizationToolOp, FVirtualizationOperation)

const TCHAR* LexToString(FVirtualizationOperation::EVirtualizationErrorCode ErrorCode)
{
	switch (ErrorCode)
	{
		case FVirtualizationOperation::EVirtualizationErrorCode::Success:
			return TEXT("Success");
		case FVirtualizationOperation::EVirtualizationErrorCode::NoBuildCommand:
			return TEXT("NoBuildCommand");
		case FVirtualizationOperation::EVirtualizationErrorCode::UBTNotFound:
			return TEXT("UBTNotFound");
		case FVirtualizationOperation::EVirtualizationErrorCode::UBTProcFailure:
			return TEXT("UBTProcFailure");
		case FVirtualizationOperation::EVirtualizationErrorCode::CompileFailed:
			return TEXT("CompileFailed");
		case FVirtualizationOperation::EVirtualizationErrorCode::UVTProcFailure:
			return TEXT("UVTProcFailure");
		case FVirtualizationOperation::EVirtualizationErrorCode::UVTError:
			return TEXT("UVTError");
		default:
			// Normally I'd checkNoEntry() here but I don't see this as a reason to crash the SubmitTool
			return TEXT("Unknown");
	}
}

FVirtualizationOperation::FVirtualizationOperation(const FName& InNameId, const FSubmitToolParameters& InParameters, TSharedRef<FSubmitToolServiceProvider> InServiceProvider, const FString& InDefinition) :
	FValidatorRunExecutable(InNameId, InParameters, InServiceProvider, InDefinition)
{
	ParseDefinition(InDefinition);
}

void FVirtualizationOperation::ParseDefinition(const FString& InDefinition)
{
	FStringOutputDevice Errors;
	Definition = MakeUnique<FVirtualizationToolDefinition>();
	FVirtualizationToolDefinition* ModifyableDefinition = const_cast<FVirtualizationToolDefinition*>(GetTypedDefinition<FVirtualizationToolDefinition>());
	FVirtualizationToolDefinition::StaticStruct()->ImportText(*InDefinition, ModifyableDefinition, nullptr, 0, &Errors, FVirtualizationToolDefinition::StaticStruct()->GetName());

	ModifyableDefinition->ExecutablePath = FConfiguration::SubstituteAndNormalizeFilename(ModifyableDefinition->ExecutablePath);
	ModifyableDefinition->BuildCommand = FConfiguration::SubstituteAndNormalizeFilename(ModifyableDefinition->BuildCommand);
	ModifyableDefinition->BuildCommandArgs = FConfiguration::Substitute(ModifyableDefinition->BuildCommandArgs);

	if(!Errors.IsEmpty())
	{
		UE_LOGF(LogSubmitTool, Error, "Error loading parameter file %ls", *Errors);
		FModelInterface::SetErrorState();
	}
}

bool FVirtualizationOperation::Activate()
{
	FVirtualizationToolDefinition* ModifyableDefinition = const_cast<FVirtualizationToolDefinition*>(GetTypedDefinition<FVirtualizationToolDefinition>());
	ModifyableDefinition->bValidateExecutableExists = false;
	bIsValidSetup = FValidatorRunExecutable::Activate();

	return bIsValidSetup;
}

bool FVirtualizationOperation::Validate(const FString& InCLDescription, const TArray<FSCFileRef>& InFilteredFilesInCL, const TArray<const FTag*>& InTags)
{
	const FVirtualizationToolDefinition* TypedDefinition = GetTypedDefinition<FVirtualizationToolDefinition>();
	check(TypedDefinition != nullptr);

	ErrorCode = EVirtualizationErrorCode::Success;
	bLaunchProcess = false;

	if (DoesExecutableNeedBuilding())
	{
		bCompileRequired = true;

		if (TypedDefinition->BuildCommand.IsEmpty())
		{
			LogFailure(FString::Printf(TEXT("[%s] Virtualization tool is not present locally in %s and cannot be built"), *ValidatorName, *TypedDefinition->ExecutablePath));
			ErrorCode = EVirtualizationErrorCode::NoBuildCommand;

			return false;
		}

		if (!StartBuildingTool())
		{
			LogFailure(FString::Printf(TEXT("[%s] Virtualization tool is not present locally in %s and cannot be built"), *ValidatorName, *TypedDefinition->ExecutablePath));
			return false;
		}
	}

	if (!IsBuildingTool())
	{
		bLaunchProcess = true;
	}
		
	return true;
}

void FVirtualizationOperation::StopInternalValidations()
{
	FValidatorRunExecutable::StopInternalValidations();

	if (BuildProcessHandle.IsValid())
	{
		if (FPlatformProcess::IsProcRunning(*BuildProcessHandle))
		{
			FPlatformProcess::TerminateProc(*BuildProcessHandle, true);
			Pipes.Reset();
		}
	}
}

void FVirtualizationOperation::OnProcessComplete(const FString& ProcessId, int32 ReturnCode)
{
	if (ReturnCode != 0)
	{
		ErrorCode = EVirtualizationErrorCode::UVTError;
	}

	FValidatorRunExecutable::OnProcessComplete(ProcessId, ReturnCode);
}

const TArray<FAnalyticsEventAttribute> FVirtualizationOperation::GetTelemetryAttributes() const
{
	TArray<FAnalyticsEventAttribute> Attributes = FValidatorRunExecutable::GetTelemetryAttributes();

	Attributes = AppendAnalyticsEventAttributeArray(Attributes,
		TEXT("ErrorCode"), LexToString(ErrorCode),
		TEXT("CompileRequired"), bCompileRequired,
		TEXT("CompileTime"), TotalCompileTime,
		TEXT("CompileResult"), CompileResult
	);

	return Attributes;
}

bool FVirtualizationOperation::StartBuildingTool()
{
	const FVirtualizationToolDefinition* TypedDefinition = GetTypedDefinition<FVirtualizationToolDefinition>();

	FString BuildCommand = TypedDefinition->BuildCommand;

	if (!FPaths::FileExists(BuildCommand))
	{
		LogFailure(FString::Printf(TEXT("[%s] Build File does not exist %s"),*ValidatorName, *BuildCommand));
		ErrorCode = EVirtualizationErrorCode::UBTNotFound;

		return false;
	}

	if (!Pipes.Create())
	{
		LogFailure(FString::Printf(TEXT("[%s] Error creating pipes"), *ValidatorName));
		ErrorCode = EVirtualizationErrorCode::UBTProcFailure;

		return false;
	}

	UE_LOGF(LogValidators, Log, "[%ls] Building Virtualization Tool", *ValidatorName);
	UE_LOGF(LogValidatorsResult, Log, "[%ls] Building Virtualization Tool", *ValidatorName);

	this->BuildProcessHandle = MakeUnique<FProcHandle>(
		FPlatformProcess::CreateProc(
			*BuildCommand,
			*TypedDefinition->BuildCommandArgs,
			false,
			true,
			true,
			nullptr,
			0,
			nullptr,
			Pipes.GetStdOutForProcess(),
			Pipes.GetStdInForProcess()));

	if (!this->BuildProcessHandle->IsValid())
	{
		Pipes.Reset();
		LogFailure(FString::Printf(TEXT("[%s] Error creating process %s %s."), *ValidatorName, *BuildCommand, *TypedDefinition->BuildCommandArgs));
		ErrorCode = EVirtualizationErrorCode::UBTProcFailure;

		return false;
	}

	CompileStartTime = FPlatformTime::Seconds();

	return true;
}

void FVirtualizationOperation::StartVirtualization()
{
	const FVirtualizationToolDefinition* TypedDefinition = GetTypedDefinition<FVirtualizationToolDefinition>();

	FString SubstArgs = FConfiguration::Substitute(TypedDefinition->ExecutableArguments);

	// Note that ::QueueProcess will call ::ValidationFinished on failure preventing us from setting the error code
	// before telemetry is reported. So we set the error code as a failure now then reset it back to Success if the
	// process starts correctly. This way if the process does fail to start up the correct error code will be reported.
	ErrorCode = EVirtualizationErrorCode::UVTProcFailure;

	if (QueueProcess(TEXT("#1"), TypedDefinition->ExecutablePath, SubstArgs))
	{
		// Resetting the error code to success as the process was created and execution will continue
		ErrorCode = EVirtualizationErrorCode::Success;
	}
	else
	{
		LogFailure(FString::Printf(TEXT("[%s] Error creating process %s %s."), *ValidatorName, *TypedDefinition->ExecutablePath, *SubstArgs));
	}
}

void FVirtualizationOperation::Tick(float InDeltatime)
{
	FValidatorRunExecutable::Tick(InDeltatime);

	if (BuildProcessHandle.IsValid())
	{
		FString NewOutput = OutputRemainder + FPlatformProcess::ReadPipe(Pipes.GetStdOutForReading());

		if (FPlatformProcess::IsProcRunning(*BuildProcessHandle))
		{
			int32 Position;
			NewOutput.FindLastChar('\n', Position);
			OutputRemainder = NewOutput.Mid(Position + 1);

			NewOutput.RemoveFromEnd(OutputRemainder);
			ProcessOutput(NewOutput);
		}
		else
		{
			ProcessOutput(NewOutput);

			if (!FPlatformProcess::GetProcReturnCode(*(this->BuildProcessHandle), &CompileResult))
			{
				const FVirtualizationToolDefinition* TypedDefinition = GetTypedDefinition<FVirtualizationToolDefinition>();
				LogFailure(FString::Printf(TEXT("[%s] Error accessing process result for %s."), *ValidatorName, *TypedDefinition->ExecutablePath));
				CompileResult = -1;
			}

			// cleanup
			Pipes.Reset();

			FPlatformProcess::CloseProc(*BuildProcessHandle);
			this->BuildProcessHandle = nullptr;

			TotalCompileTime = FPlatformTime::Seconds() - CompileStartTime;

			if (CompileResult == 0)
			{
				UE_LOGF(LogValidators, Log, "[%ls] Virtualization Tool built successfully - %.1f(s)", *ValidatorName, TotalCompileTime);
				UE_LOGF(LogValidatorsResult, Log, "[%ls] Virtualization built successfully - %.1f(s)", *ValidatorName, TotalCompileTime);

				bLaunchProcess = true;
			}
			else
			{
				LogFailure(FString::Printf(TEXT("[%s] Failed to build Virtualization tool (error code %d) - %.1f(s)"), *ValidatorName, CompileResult, TotalCompileTime));
				ErrorCode = EVirtualizationErrorCode::CompileFailed;
				
				ValidationFinished(false);
			}
		}
	}

	if (bLaunchProcess)
	{
		bLaunchProcess = false;
		StartVirtualization();
	}
}

void FVirtualizationOperation::ProcessOutput(const FString& InOutput)
{
	if (InOutput.IsEmpty())
	{
		return;
	}

	TArray<FString> Lines;
	const TCHAR* Separators[] = { TEXT("\n"), TEXT("\r") };
	InOutput.ParseIntoArray(Lines, Separators, UE_ARRAY_COUNT(Separators));

	for (const FString& Line : Lines)
	{
		if (    Line.Contains(" error ")
			&& !Line.Contains(" Display: ")
			&& !Line.Contains(" Warning: ")
			&& !Line.Contains(" Log: "))
		{
			UE_LOGF(LogValidators, Warning, "[%ls]: %ls", *ValidatorName, *Line);
		}
		else
		{
			UE_LOGF(LogValidators, Log, "[%ls]: %ls", *ValidatorName, *Line);
		}
	}
}
