// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Logic/ProcessPipes.h"
#include "Logic/Validators/ValidatorRunExecutable.h"

namespace SubmitToolParseConstants
{
	const FString VirtualizationToolOp = TEXT("VirtualizationTool");
}

class FVirtualizationOperation : public FValidatorRunExecutable
{
public:

public:
	FVirtualizationOperation(const FName& InNameId, const FSubmitToolParameters& InParameters, TSharedRef<FSubmitToolServiceProvider> InServiceProvider, const FString& InDefinition);
	virtual void ParseDefinition(const FString& InDefinition) override;
	virtual bool Activate() override;
	virtual bool Validate(const FString& InCLDescription, const TArray<FSCFileRef>& InFilteredFilesInCL, const TArray<const FTag*>& InTags) override;
	virtual const FString& GetValidatorTypeName() const override
	{
		return SubmitToolParseConstants::VirtualizationToolOp;
	}

	virtual void Tick(float InDeltatime) override;
	virtual void StopInternalValidations() override;

protected:
	virtual void OnProcessComplete(const FString& ProcessId, int32 ReturnCode) override;
	virtual const TArray<FAnalyticsEventAttribute> GetTelemetryAttributes() const override;

private: // methods

	bool StartBuildingTool();
	void StartVirtualization();
	void ProcessOutput(const FString& InOutput);

	bool IsBuildingTool() const
	{
		return BuildProcessHandle.IsValid();
	}

private: // members
	TUniquePtr<FProcHandle> BuildProcessHandle;
	FProcessPipes Pipes;

	bool bCompileRequired = false;
	int32 CompileResult = 0;

	double CompileStartTime = 0.0;
	double TotalCompileTime = 0.0;

	bool bLaunchProcess = false;

	FString OutputRemainder;

	enum class EVirtualizationErrorCode
	{
		Success = 0,

		NoBuildCommand,
		UBTNotFound,
		UBTProcFailure,
		CompileFailed,
		UVTProcFailure,
		UVTError
	};

	EVirtualizationErrorCode ErrorCode = EVirtualizationErrorCode::Success;

	friend const TCHAR* LexToString(FVirtualizationOperation::EVirtualizationErrorCode ErrorCode);
};
