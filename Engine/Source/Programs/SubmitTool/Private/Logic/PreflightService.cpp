// Copyright Epic Games, Inc. All Rights Reserved.

#include "PreflightService.h"
#include "GenericPlatform/GenericPlatformProcess.h"
#include "HttpModule.h"
#include "Interfaces/IHttpResponse.h"
#include "JsonObjectConverter.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "Serialization/JsonSerializer.h"
#include "Widgets/Docking/SDockTab.h"
#include "Internationalization/Regex.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Text/STextBlock.h"

#include "CommandLine/CmdLineParameters.h"
#include "Logic/Services/Interfaces/ISubmitToolExtensions.h"
#include "Configuration/Configuration.h"
#include "Logic/Services/Interfaces/ISTSourceControlService.h"
#include "Logic/Services/Interfaces/ICredentialsService.h"
#include "Logic/Services/Interfaces/ITagService.h"
#include "Logging/SubmitToolLog.h"
#include "Models/ModelInterface.h"
#include "Models/SubmitToolUserPrefs.h"
#include "Telemetry/TelemetryService.h"
#include "Parameters/SubmitToolParameters.h"
#include "SubmitToolUtils.h"

FPreflightService::FPreflightService(
	const FHordeParameters& InSettings,
	FModelInterface* InModelInterface,
	TWeakPtr<FSubmitToolServiceProvider> InServiceProvider)
	:
	Definition(InSettings),
	ServiceProvider(InServiceProvider),
	ModelInterface(InModelInterface),
	State(EPreflightServiceState::Idle),
	LastErrorMessage(TEXT(""))
{
	TickHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateRaw(this, &FPreflightService::Tick));

	PreflightTag = ServiceProvider.Pin()->GetService<ITagService>()->GetTagOfSubtype(TEXT("preflight"));
}

FPreflightService::~FPreflightService()
{
	FTSTicker::RemoveTicker(TickHandle);
	OnPreflightDataUpdated.Clear();
}

bool FPreflightService::Tick(float DeltaTime)
{
	TSharedPtr<IChangelistService> ChangelistService = ServiceProvider.Pin()->GetService<IChangelistService>();
	switch(State)
	{
		//////////////////////////////////////////////////
		case EPreflightServiceState::Idle:
			// Do nothing, wait for someone to press the "Start" preflight button
			break;

		case EPreflightServiceState::WaitingForShelf:
			if (ChangelistService->IsShelfReady())
			{
				State = EPreflightServiceState::StartPreflight;
			}
			break;

			//////////////////////////////////////////////////
		case EPreflightServiceState::StartPreflight:
			StartPreflight();
			break;

			//////////////////////////////////////////////////
		case EPreflightServiceState::Error:
			UE_LOGF(LogSubmitTool, Error, "Preflight: \"%ls\"", *LastErrorMessage);
			State = EPreflightServiceState::Idle;
			break;
	}

	return true;
}

TMap<FString, FStringFormatArg> FPreflightService::GetFormatParameters(const FString& InTemplate)
{
	RefreshStreamName();
	TMap<FString, FStringFormatArg> FormatMap =
	{
		{ TEXT("URL"), Definition.HordeServerAddress },
		{ TEXT("CLID"), ServiceProvider.Pin()->GetService<IChangelistService>()->GetCLID()}
	};

	FormatMap.Add(TEXT("Stream"), StreamName);
	FormatMap.Add(TEXT("Template"), InTemplate);
	FormatMap.Add(TEXT("AdditionalTasks"), FString());


	if (InTemplate.IsEmpty())
	{
		FPreflightTemplateDefinition Template;
		if(SelectPreflightTemplate(Template))
		{
			FormatMap[TEXT("Template")] = Template.Template;
			FormatMap[TEXT("AdditionalTasks")] = GetAdditionalTasksString(Template);
		}
		else
		{
			FormatMap[TEXT("Template")] = Definition.DefaultPreflightTemplate;
		}
	}

	return FormatMap;
}

void FPreflightService::RequestPreflight(const FString InTemplate)
{
	if(State == EPreflightServiceState::Idle)
	{
		if(!ServiceProvider.Pin()->GetService<ISTSourceControlService>()->GetClientStreams().IsEmpty())
		{
			TemplateId = InTemplate;
			UE_LOGF(LogSubmitTool, Log, "Preflight: Requesting...");
			
			TSharedPtr<IChangelistService> ChangelistService = ServiceProvider.Pin()->GetService<IChangelistService>();
			
			ChangelistService->EnsureShelfIsCurrent();
			State = EPreflightServiceState::WaitingForShelf;
		}
		else
		{
			UE_LOGF(LogSubmitTool, Error, "Couldn't retrieve stream name in this p4 client. Submit tool can't start a preflight, see previous errors.");
		}
	}
	else
	{
		// Do nothing, we're already busy trying to start a preflight
	}
}

void FPreflightService::QueueFetch(bool bRequeue, float InSeconds)
{
	FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda([this, bRequeue](float DeltaTime) { FetchPreflightInfo(bRequeue); return false; }), InSeconds);
}

void FPreflightService::Requeue()
{
	float WaitTime = Definition.FetchPreflightEachSeconds;

	if(HordePreflights.IsValid())
	{
		for(const FPreflightData& PFData : HordePreflights->PreflightList)
		{
			if(PFData.CachedResults.State != EPreflightState::Completed)
			{
				WaitTime = Definition.FetchPreflightEachSecondsWhenInProgress;
				break;
			}
		}
	}

	for(const TPair<FString, FPreflightData>& Pair : UnlinkedHordePreflights)
	{
		if(Pair.Value.CachedResults.State != EPreflightState::Completed)
		{
			WaitTime = Definition.FetchPreflightEachSecondsWhenInProgress;
			break;
		}
	}

	QueueFetch(true, WaitTime);
}

void FPreflightService::FetchPreflightInfo(bool bRequeue, const FString& InOAuthToken)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPreflightService::FetchPreflightInfo);

	TSharedPtr<ICredentialsService> CredentialsService = ServiceProvider.Pin()->GetService<ICredentialsService>();
	if(Definition.HordeServerAddress.IsEmpty() || !CredentialsService->IsOIDCTokenEnabled() || FModelInterface::GetState() == ESubmitToolAppState::Finished)
	{
		bHordeInformationReady = true;
		return;
	}

	TSharedPtr<IChangelistService> ChangelistService = ServiceProvider.Pin()->GetService<IChangelistService>();

	// Don't bother with the default changelist
	if(ChangelistService->GetCLID() == TEXT("default"))
	{
		if(bRequeue)
		{
			QueueFetch(bRequeue, Definition.FetchPreflightEachSeconds);
		}

		bHordeInformationReady = true;
		return;
	}

	const FString& OIDCToken = CredentialsService->IsTokenReady() ? CredentialsService->GetToken() : InOAuthToken;

	if(!OIDCToken.IsEmpty())
	{
		if(!LinkedPFRequest.IsValid())
		{
			LinkedPFRequest = FHttpModule::Get().CreateRequest();

			FString FetchPreflightUrl = FString::Format(*Definition.FindPreflightURLFormat, GetFormatParameters());
			LinkedPFRequest->SetURL(FetchPreflightUrl);
			LinkedPFRequest->SetVerb(TEXT("GET"));
		}
		else if(LinkedPFRequest->GetStatus() == EHttpRequestStatus::Processing)
		{
			// if it's still Processing, do not try to request again.
			return;
		}

		// ensure the token is the most up to date
		LinkedPFRequest->SetHeader(TEXT("Authorization"), FString::Printf(TEXT("bearer %s"), *OIDCToken));

		if(LinkedPFRequest->OnProcessRequestComplete().IsBound())
		{
			LinkedPFRequest->OnProcessRequestComplete().Unbind();
		}

		LinkedPFRequest->OnProcessRequestComplete().BindLambda([this, bRequeue, OIDCToken](FHttpRequestPtr Request, FHttpResponsePtr HttpResponse, bool bConnectedSuccessfully)
			{
				if(!bConnectedSuccessfully)
				{
					if(HttpResponse.IsValid())
					{
						UE_LOGF(LogSubmitTool, Warning, "Unable to connect to horde. Connection error %d", HttpResponse->GetResponseCode());
						UE_LOGF(LogSubmitToolDebug, Warning, "Unable to connect to horde. Connection error\nResponse: %ls", *HttpResponse->GetContentAsString());
					}
					else
					{
						UE_LOGF(LogSubmitTool, Warning, "Unable to connect to horde. Connection error, no response.");
					}

					bHordeInformationReady = true;
					OnHordeConnectionFailed.Broadcast();
					return;
				}

				if(HttpResponse.IsValid())
				{
					//UE_LOGF(LogSubmitToolDebug, Verbose, "Fetch Preflight Response: %ls", *HttpResponse->GetContentAsString());
					if(EHttpResponseCodes::IsOk(HttpResponse->GetResponseCode()))
					{
						TUniquePtr<FPreflightList> NewHordePreflights = MakeUnique<FPreflightList>();
						FJsonObjectConverter::JsonObjectStringToUStruct<FPreflightList>(FString::Printf(TEXT("{\"PreflightList\" : %s}"), *HttpResponse->GetContentAsString()), NewHordePreflights.Get());
						NewHordePreflights->Initialize();

						if(PreflightTag != nullptr && !Definition.FindSinglePreflightURLFormat.IsEmpty())
						{
							for(FString PreflightId : PreflightTag->GetValues())
							{
								if(PreflightId.Equals(TEXT("skip")) || PreflightId.Equals(TEXT("none")))
								{
									continue;
								}

								if(PreflightId.Contains(TEXT("/")))
								{
									int32 SlashIdx;
									PreflightId.FindLastChar(TCHAR('/'), SlashIdx);
									PreflightId = PreflightId.RightChop(SlashIdx + 1);
								}

								PreflightId.TrimStartAndEndInline();

								FRegexPattern Pattern = FRegexPattern(TEXT("(?:[0-9]|[a-f]){24}"), ERegexPatternFlags::CaseInsensitive);
								FRegexMatcher regex = FRegexMatcher(Pattern, PreflightId);
								bool match = regex.FindNext();
								if(match)
								{
									const FPreflightData* FoundData = NewHordePreflights->PreflightList.FindByPredicate([&PreflightId](const FPreflightData& InData) { return InData.ID == PreflightId; });
									if(FoundData == nullptr)
									{
										FetchUnlinkedPreflight(PreflightId, bRequeue, OIDCToken);
									}
								}
							}
						}

						if(!HordePreflights.IsValid() || *NewHordePreflights != *HordePreflights)
						{
							UE_LOGF(LogSubmitToolDebug, Verbose, "Newer Preflight information received");

							// Only log when there's a different number of preflights
							if(!HordePreflights.IsValid() || HordePreflights->PreflightList.Num() != NewHordePreflights->PreflightList.Num())
							{
								UE_LOGF(LogSubmitTool, Log, "Retrieved %d preflights for CL %ls", NewHordePreflights->PreflightList.Num(), *ServiceProvider.Pin()->GetService<IChangelistService>()->GetCLID());
							}

							HordePreflights = MoveTemp(NewHordePreflights);

							if(PreflightTag != nullptr)
							{
								bool bCLDescriptionModified = false;

								if(HordePreflights->PreflightList.Num() != 0)
								{
									FString CurrentTagValue = PreflightTag->GetValuesText();
									
									if(!bStopAskingTagUpdate && !CurrentTagValue.Contains(HordePreflights->PreflightList[0].ID))
									{
										if (FSubmitToolUserPrefs::Get()->bAutoUpdatePreflight)
										{
											ModelInterface->SetTagValues(*PreflightTag, HordePreflights->PreflightList[0].ID);
											UE_LOGF(LogSubmitTool, Log, "Tag %ls has been updated with the latest associated preflight %lsjob/%ls", *PreflightTag->Definition.GetTagId(), *Definition.HordeServerAddress, *HordePreflights->PreflightList[0].ID)
											bCLDescriptionModified = true;
										}
										else
										{
											EDialogFactoryResult Result = EDialogFactoryResult::FirstButton;
											if (!CurrentTagValue.IsEmpty())
											{
												Result = ShowUpdatePreflightTagDialog();
											}

											if(Result == EDialogFactoryResult::FirstButton)
											{
												// Set the latest one as the tag value
												ModelInterface->SetTagValues(*PreflightTag, HordePreflights->PreflightList[0].ID);
												UE_LOGF(LogSubmitTool, Log, "Tag %ls has been updated with the latest associated preflight %lsjob/%ls", *PreflightTag->Definition.GetTagId(), *Definition.HordeServerAddress, *HordePreflights->PreflightList[0].ID)
												bCLDescriptionModified = true;
											}
											else
											{
												bStopAskingTagUpdate = true;
											}
										}

									}
								}

								if(bCLDescriptionModified)
								{
									ModelInterface->ValidateCLDescription();
								}
							}
						}
					}
					else
					{
						UE_LOGF(LogSubmitTool, Warning, "Could not retrieve preflights, Http code %d.", HttpResponse->GetResponseCode());
						UE_LOGF(LogSubmitToolDebug, Error, "Fetch preflight failed. Response %ls", *HttpResponse->GetContentAsString());
					}
				}
				else
				{
					UE_LOGF(LogSubmitTool, Warning, "Unable to fetch preflights. Failed unknown response");
					UE_LOGF(LogSubmitToolDebug, Warning, "Unable to fetch preflights. Failed unknown response");
				}

				if(ActiveUnlinkedRequests == 0)
				{
					bHordeInformationReady = true;
					if(OnPreflightDataUpdated.IsBound() && HordePreflights.IsValid())
					{
						OnPreflightDataUpdated.Broadcast(HordePreflights, UnlinkedHordePreflights);
					}

					if(bRequeue)
					{
						Requeue();
					}
				}
			});

		FTimespan TimeSinceLast = FDateTime::UtcNow() - LastRequest;
		if(bRequeue || TimeSinceLast.GetTotalSeconds() > 3)
		{
			LastRequest = FDateTime::UtcNow();
			UE_LOGF(LogSubmitToolDebug, Log, "Fetching preflights for CL %ls. URL: %ls", *ChangelistService->GetCLID(), *LinkedPFRequest->GetURL())
			LinkedPFRequest->ProcessRequest();
		}
	}
	else
	{
		CredentialsService->QueueWorkForToken([this, bRequeue](const FString& InToken)
			{
				if(!InToken.IsEmpty())
				{
					FetchPreflightInfo(bRequeue, InToken);
				}
				else
				{
					bHordeInformationReady = true;
					UE_LOGF(LogSubmitTool, Warning, "Couldn't obtain OAuth token login, communication with Horde is not possible.");
				}
			});
	}
}

void FPreflightService::FetchUnlinkedPreflight(const FString& InPreflightId, bool bRequeue, const FString& InOAuthToken)
{
	if(InOAuthToken.IsEmpty())
	{
		bHordeInformationReady = true;
		return;
	}

	FHttpRequestPtr& UnlinkedPFRequest = UnlinkedPFRequests.FindOrAdd(InPreflightId);

	if(!UnlinkedPFRequest.IsValid())
	{
		UnlinkedPFRequest = FHttpModule::Get().CreateRequest();

		FStringFormatNamedArguments ReplaceStringArgs = GetFormatParameters();
		ReplaceStringArgs.Add(TEXT("PreflightId"), InPreflightId);


		FString FetchPreflightUrl = FString::Format(*Definition.FindSinglePreflightURLFormat, ReplaceStringArgs);
		UnlinkedPFRequest->SetURL(FetchPreflightUrl);
		UnlinkedPFRequest->SetVerb(TEXT("GET"));
	}
	else if(UnlinkedPFRequest->GetStatus() == EHttpRequestStatus::Processing)
	{
		// if it's still Processing, do not try to request again.
		return;
	}

	UnlinkedPFRequest->SetHeader(TEXT("Authorization"), FString::Printf(TEXT("bearer %s"), *InOAuthToken));

	if(UnlinkedPFRequest->OnProcessRequestComplete().IsBound())
	{
		UnlinkedPFRequest->OnProcessRequestComplete().Unbind();
	}

	ActiveUnlinkedRequests++;
	UnlinkedPFRequest->OnProcessRequestComplete().BindLambda([this, bRequeue, InOAuthToken, InPreflightId](FHttpRequestPtr Request, FHttpResponsePtr HttpResponse, bool bConnectedSuccessfully)
		{
			ActiveUnlinkedRequests--;
			if(!bConnectedSuccessfully)
			{
				if(HttpResponse.IsValid())
				{
					UE_LOGF(LogSubmitTool, Warning, "Unable to connect to horde. Connection error %d", HttpResponse->GetResponseCode());
					UE_LOGF(LogSubmitToolDebug, Warning, "Unable to connect to horde. Connection error\nResponse: %ls", *HttpResponse->GetContentAsString());
				}
				else
				{
					UE_LOGF(LogSubmitTool, Warning, "Unable to connect to horde. Connection error, no response.");
				}

				if (ActiveUnlinkedRequests == 0)
				{
					bHordeInformationReady = true;
				}
				return;
			}

			if(HttpResponse.IsValid())
			{
				UE_LOGF(LogSubmitToolDebug, Verbose, "Fetch Single Preflight Response: %ls", *HttpResponse->GetContentAsString());
				if(EHttpResponseCodes::IsOk(HttpResponse->GetResponseCode()))
				{
					FPreflightData ReceivedPreflightInfo;
					FJsonObjectConverter::JsonObjectStringToUStruct<FPreflightData>(*HttpResponse->GetContentAsString(), &ReceivedPreflightInfo);
					ReceivedPreflightInfo.RecalculateCachedResults();

					if(!UnlinkedHordePreflights.Contains(InPreflightId) || UnlinkedHordePreflights[InPreflightId] != ReceivedPreflightInfo)
					{
						UE_LOGF(LogSubmitToolDebug, Verbose, "Newer %ls Preflight information received", *InPreflightId);

						// Only log when the preflight is new
						if(!UnlinkedHordePreflights.Contains(InPreflightId))
						{
							UE_LOGF(LogSubmitTool, Log, "Retrieved information from preflight %ls", *InPreflightId);
							UnlinkedHordePreflights.Add(InPreflightId, ReceivedPreflightInfo);
						}
						else
						{
							UnlinkedHordePreflights[InPreflightId] = MoveTemp(ReceivedPreflightInfo);
						}

					}
				}
				else
				{
					UE_LOGF(LogSubmitTool, Warning, "Could not retrieve preflights, Http code %d.", HttpResponse->GetResponseCode());
					UE_LOGF(LogSubmitToolDebug, Error, "Fetch preflight failed. Response %ls", *HttpResponse->GetContentAsString());
				}
			}
			else
			{
				UE_LOGF(LogSubmitTool, Warning, "Unable to fetch preflights. Failed unknown response");
				UE_LOGF(LogSubmitToolDebug, Warning, "Unable to fetch preflights. Failed unknown response");
			}

			if(ActiveUnlinkedRequests == 0)
			{
				bHordeInformationReady = true;
				if(OnPreflightDataUpdated.IsBound())
				{
					OnPreflightDataUpdated.Broadcast(HordePreflights, UnlinkedHordePreflights);
				}

				if(bRequeue)
				{
					Requeue();
				}
			}
		});

	UnlinkedPFRequest->ProcessRequest();
}

void FPreflightService::StartPreflight()
{
	const TArray<FSCFileRef>& ShelvedFiles = ServiceProvider.Pin()->GetService<IChangelistService>()->GetShelvedFilesInCL();

	if (ShelvedFiles.IsEmpty())
	{
		LastErrorMessage = TEXT("Shelve is empty or it couldn't be retrieved from p4, can't request preflight");
		State = EPreflightServiceState::Error;
		return;
	}

	FString StartPreflightUrl = FString::Format(*Definition.StartPreflightURLFormat, GetFormatParameters(TemplateId));
	TemplateId = FString();

	// If for some reason, our preflight settings are missing, this will be empty, let's not popup a browser with nothing in it
	if(!StartPreflightUrl.IsEmpty())
	{
		UE_LOGF(LogSubmitTool, Log, "Preflight: Starting preflight with URL: \"%ls\"", *StartPreflightUrl);

		FTelemetryService::Get()->CustomEvent(TEXT("SubmitTool.PreflightLaunched"), MakeAnalyticsEventAttributeArray(
			TEXT("PreflightURL"), StartPreflightUrl,
			TEXT("Stream"), StreamName
		));

		FPlatformProcess::LaunchURL(*StartPreflightUrl, nullptr, nullptr);
		State = EPreflightServiceState::Idle;

		// Do a Fetch in 5, 10 and 30 s to try and capture the triggered preflight
		QueueFetch(false, 5.f);
		QueueFetch(false, 10.f);
		QueueFetch(false, 30.f);
	}
	else
	{
		LastErrorMessage = TEXT("Missing INI preflight settings");
		State = EPreflightServiceState::Error;
	}
}

TArray<const FPreflightData*> FPreflightService::GetTaggedPreflights() const
{
	TArray<const FPreflightData*> FoundPreflights;
	if (PreflightTag == nullptr)
	{
		return FoundPreflights;
	}

	const TArray<FString>& PreflightValues = PreflightTag->GetValues();

	for (FString PreflightId : PreflightValues)
	{
		if (PreflightId.Contains(TEXT("/")))
		{
			int32 SlashIdx;
			PreflightId.FindLastChar(TCHAR('/'), SlashIdx);
			PreflightId = PreflightId.RightChop(SlashIdx + 1);
		}
		PreflightId.TrimStartAndEndInline();

		FRegexPattern Pattern = FRegexPattern(TEXT("(?:[0-9]|[a-f]){24}"), ERegexPatternFlags::CaseInsensitive);
		FRegexMatcher regex = FRegexMatcher(Pattern, PreflightId);
		if (regex.FindNext())
		{
			const FPreflightData* FoundData = nullptr;
			if (HordePreflights.IsValid())
			{
				FoundData = HordePreflights->PreflightList.FindByPredicate([&PreflightId](const FPreflightData& InData) { return InData.ID == PreflightId; });
				// Look into the linked preflights and if we don't find it check the unlinked preflights
				if (FoundData != nullptr)
				{
					FoundPreflights.Add(FoundData);
				}
			}

			if(FoundData == nullptr)
			{
				FoundData = UnlinkedHordePreflights.Find(PreflightId);
				if (FoundData)
				{
					FoundPreflights.Add(FoundData);
				}
			}
		}
	}

	return FoundPreflights;
}

bool FPreflightService::SelectPreflightTemplate(FPreflightTemplateDefinition& OutTemplate) const
{
	TSharedPtr<IChangelistService> ChangelistService = ServiceProvider.Pin()->GetService<IChangelistService>();

	// Get the list of files in the changelist
	const TArray<FSCFileRef>& FilesInCl = bCheckShelveInstead ? ChangelistService->GetShelvedFilesInCL() : ChangelistService->GetFilesInCL();

	if (ServiceProvider.Pin()->HasService<ISubmitToolExtensions>())
	{
		TOptional<FPreflightTemplateDefinition> Result = ServiceProvider.Pin()->GetService<ISubmitToolExtensions>()->SelectPreflightTemplate(FilesInCl, ServiceProvider.Pin(), Definition);
		if (Result.IsSet())
		{
			OutTemplate = Result.GetValue();
			return true;
		}
	}

	// Loop through each definition to see if the files are in the path then check extension
	for(const FPreflightTemplateDefinition& Def : Definition.Definitions)
	{
		FString RegexPat = Def.RegexPath.Replace(TEXT("$(StreamRoot)"), *StreamName, ESearchCase::IgnoreCase);
		FRegexPattern Pattern = FRegexPattern(RegexPat, ERegexPatternFlags::CaseInsensitive);
		for(const FSCFileRef& File : FilesInCl)
		{
			FRegexMatcher regex = FRegexMatcher(Pattern, File->GetDepotPath());
			if(regex.FindNext())
			{
				OutTemplate = Def;
				return true;
			}
		}
	}

	return false;
}

FString FPreflightService::GetAdditionalTasksString(const FPreflightTemplateDefinition& InTemplate) const
{
	TStringBuilder<256> AdditionalTaskStrBuilder;
	const FString BaseString = TEXT("&id-additional-tasks.");
	const FString EndString = TEXT("=true");
	TSharedPtr<IChangelistService> ChangelistService = ServiceProvider.Pin()->GetService<IChangelistService>();

	const TArray<FSCFileRef>& FilesInCl = bCheckShelveInstead ? ChangelistService->GetShelvedFilesInCL() : ChangelistService->GetFilesInCL();
	for(const FPreflightAdditionalTask& AdditionalTask : InTemplate.AdditionalTasks)
	{
		FString RegexPat = AdditionalTask.RegexPath.Replace(TEXT("$(StreamRoot)"), *StreamName, ESearchCase::IgnoreCase);
		FRegexPattern Pattern = FRegexPattern(RegexPat, ERegexPatternFlags::CaseInsensitive);
		for(const FSCFileRef& File : FilesInCl)
		{
			FRegexMatcher regex = FRegexMatcher(Pattern, File->GetDepotPath());
			if(regex.FindNext())
			{
				AdditionalTaskStrBuilder.Append(BaseString);
				AdditionalTaskStrBuilder.Append(AdditionalTask.TaskId);
				AdditionalTaskStrBuilder.Append(EndString);
			}
		}
	}

	return AdditionalTaskStrBuilder.ToString();
}

void FPreflightService::RefreshStreamName()
{
	TSharedPtr<ISTSourceControlService> SCCService = ServiceProvider.Pin()->GetService<ISTSourceControlService>();
	StreamName = SCCService->GetRootStreamName();

	const TArray<FSCFileRef>& ShelvedFiles = ServiceProvider.Pin()->GetService<IChangelistService>()->GetShelvedFilesInCL();

	if (ShelvedFiles.Num() != 0)
	{
		FString CommonPath = ShelvedFiles[0]->GetDepotPath();
		const FString& LastPath = ShelvedFiles.Last()->GetDepotPath();

		for (size_t i = 0; i < FMath::Min(CommonPath.Len(), LastPath.Len()); ++i)
		{
			if (CommonPath[i] != LastPath[i])
			{
				CommonPath = CommonPath.Left(i);
				break;
			}
		}

		if (!CommonPath.Equals(TEXT("//")))
		{
			int32 NextSlash = CommonPath.Find(TEXT("/"), ESearchCase::IgnoreCase, ESearchDir::FromStart, 2);
			if (NextSlash > 2)
			{
				const FString Depot = CommonPath.Mid(2, NextSlash - 2);
				int32 StreamDepth = SCCService->GetDepotStreamDepth(Depot);

				for (int32 i = 0; i < StreamDepth; ++i)
				{
					NextSlash = CommonPath.Find(TEXT("/"), ESearchCase::IgnoreCase, ESearchDir::FromStart, NextSlash + 1);
				}

				if (NextSlash != -1)
				{
					const FSCCStream* FoundStream = SCCService->GetSCCStream(CommonPath.Left(NextSlash));

					if (FoundStream != nullptr)
					{
						StreamName = FoundStream->Name;
					}
				}
			}
		}
	}
}

EDialogFactoryResult FPreflightService::ShowUpdatePreflightTagDialog() const
{
	if(ModelInterface->GetMainTab().IsValid() && ModelInterface->GetMainTab().Pin()->GetParentWindow().IsValid())
	{
		ModelInterface->GetMainTab().Pin()->GetParentWindow()->DrawAttention(FWindowDrawAttentionParameters());
	}

	const FText TextTitle = FText::FromString(FString::Printf(TEXT("Preflight CL %s: Newer preflight available"), *ServiceProvider.Pin()->GetService<IChangelistService>()->GetCLID()));
	const FText TextDescription = FText::FromString(FString::Printf(TEXT("There is a newer preflight for this changelist:\n<a id=\"browser\" style=\"Hyperlink\" href=\"%sjob/%s\">%s - %s</>\n\nDo you want to update the #preflight tag?"), *Definition.HordeServerAddress, *HordePreflights->PreflightList[0].ID, *HordePreflights->PreflightList[0].Name, *HordePreflights->PreflightList[0].ID));

	return FDialogFactory::ShowDialog(TextTitle, TextDescription, TArray<FString>{ TEXT("Update Tag Value"), TEXT("Cancel") }, FSubmitToolUtils::BuildUserPrefCheckboxUI(FSubmitToolUserPrefs::Get()->bAutoUpdatePreflight, FText::FromString(TEXT("Always update, Don't ask again"))));
}
