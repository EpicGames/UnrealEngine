// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/Ticker.h"
#include "Logic/Services/Interfaces/IChangelistService.h"
#include "Logic/ProcessWrapper.h"
#include "Logic/DialogFactory.h"
#include "Parameters/SubmitToolParameters.h"
#include "Models/PreflightData.h"
#include "Logic/Services/Interfaces/ISubmitToolService.h"

class FSubmitToolServiceProvider;
class FModelInterface;
class IHttpRequest;

enum class EPreflightServiceState
{
	Idle,
	WaitingForShelf,
	StartPreflight,
	Error,
};


class FPreflightService final : public ISubmitToolService
{
public:
	FPreflightService() = delete;
	FPreflightService(
		const FHordeParameters& InSettings,
		FModelInterface* InModelInterface,
		TWeakPtr<FSubmitToolServiceProvider> InServiceProvider);
	~FPreflightService();

	void RequestPreflight(const FString InTemplate = TEXT(""));
	void FetchPreflightInfo(bool bRequeue = false, const FString& InOAuthToken = TEXT(""));

	EPreflightServiceState GetState() const
	{
		return State;
	}
	bool IsRequestInProgress() const
	{
		return State != EPreflightServiceState::Idle;
	}
	const TUniquePtr<FPreflightList>& GetPreflightData() const
	{
		return HordePreflights;
	}
	const TMap<FString, FPreflightData>& GetUnlinkedPreflights() const
	{
		return UnlinkedHordePreflights;
	}
	const FString& GetHordeServerAddress() const
	{
		return Definition.HordeServerAddress;
	}
	const FString& GetDefaultPreflightTemplate() const
	{
		return Definition.DefaultPreflightTemplate;
	}
	bool GetIsHordeInformationReady() const
	{
		return bHordeInformationReady;
	}

	TArray<const FPreflightData*> GetTaggedPreflights() const;
	TMap<FString, FStringFormatArg> GetFormatParameters(const FString& InTemplate = FString());

	bool SelectPreflightTemplate(FPreflightTemplateDefinition& OutTemplate) const;

	FOnPreflightDataUpdated OnPreflightDataUpdated;
	FSimpleMulticastDelegate OnHordeConnectionFailed;

private:
	void QueueFetch(bool bRequeue, float InSeconds);

	void Requeue();

	void StartPreflight();
	FString GetAdditionalTasksString(const FPreflightTemplateDefinition& InTemplate) const;
	void RefreshStreamName();

	void FetchUnlinkedPreflight(const FString& InPreflightId, bool bRequeue, const FString& InOAuthToken);

	EDialogFactoryResult ShowUpdatePreflightTagDialog() const;
	bool Tick(float DeltaTime);

	// Definitions from the ini
	const FHordeParameters Definition;

	// services we depend on
	TWeakPtr<FSubmitToolServiceProvider> ServiceProvider;
	FTag* PreflightTag;
	FModelInterface* ModelInterface;

	// local data
	bool bCheckShelveInstead = false;
	bool bStopAskingTagUpdate = false;
	bool bHordeInformationReady = false;
	FDateTime LastRequest = FDateTime::MinValue();
	EPreflightServiceState State;
	FTSTicker::FDelegateHandle TickHandle;
	FString LastErrorMessage;
	TUniquePtr<FPreflightList> HordePreflights;
	TMap<FString, FPreflightData> UnlinkedHordePreflights;

	FString StreamName;
	FString TemplateId;

	// Fetch Preflight
	TSharedPtr<IHttpRequest> LinkedPFRequest = nullptr;
	TMap<FString, TSharedPtr<IHttpRequest>> UnlinkedPFRequests;
	int8 ActiveUnlinkedRequests = 0;
};

Expose_TNameOf(FPreflightService);