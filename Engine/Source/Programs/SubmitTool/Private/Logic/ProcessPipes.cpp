// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProcessPipes.h"

#include "HAL/PlatformProcess.h"

bool FProcessPipes::Create()
{
	checkf(!IsValid(), TEXT("Cannot call FProcessPipes::Create on a valid object"));

	if (!FPlatformProcess::CreatePipe(StdOutReadPipe, StdOutWritePipe, false))
	{
		return false;
	}

	if (!FPlatformProcess::CreatePipe(StdInReadPipe, StdInWritePipe, true))
	{
		// Cleanup StdOut pipes before returning
		FPlatformProcess::ClosePipe(StdOutReadPipe, StdOutWritePipe);

		StdOutReadPipe = nullptr;
		StdOutWritePipe = nullptr;

		return false;
	}

	if (!FPlatformProcess::CreatePipe(StdErrReadPipe, StdErrWritePipe, false))
	{
		// Cleanup StdOut & StdIn pipes before returning
		FPlatformProcess::ClosePipe(StdOutReadPipe, StdOutWritePipe);
		FPlatformProcess::ClosePipe(StdInReadPipe, StdInWritePipe);

		StdInReadPipe = nullptr;
		StdInWritePipe = nullptr;
		StdOutReadPipe = nullptr;
		StdOutWritePipe = nullptr;

		return false;
	}

	return true;
}

void FProcessPipes::Reset()
{
	if (IsValid())
	{
		FPlatformProcess::ClosePipe(StdOutReadPipe, StdOutWritePipe);

		StdOutReadPipe = nullptr;
		StdOutWritePipe = nullptr;

		FPlatformProcess::ClosePipe(StdInReadPipe, StdInWritePipe);

		StdInReadPipe = nullptr;
		StdInWritePipe = nullptr;


		FPlatformProcess::ClosePipe(StdErrReadPipe, StdErrWritePipe);

		StdErrReadPipe = nullptr;
		StdErrWritePipe = nullptr;
	}
}

bool FProcessPipes::IsValid() const
{
	// Technically we only need to test one pointer but might as well check them
	// all for added safety.
	return	StdInReadPipe != nullptr &&
			StdInWritePipe != nullptr &&
			StdOutWritePipe != nullptr &&
			StdOutReadPipe != nullptr&&
			StdErrWritePipe != nullptr &&
			StdErrReadPipe != nullptr;
}