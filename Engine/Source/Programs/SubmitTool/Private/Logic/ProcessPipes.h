// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/** Wrapper around stdin, stderr and stdout pipes created by a call to FPlatformProcess::CreatePipe */
struct FProcessPipes
{
	FProcessPipes() = default;

	~FProcessPipes()
	{
		Reset();
	}

	bool Create();

	void Reset();

	bool IsValid() const;

	void* GetStdInForProcess() const
	{
		return StdInReadPipe;
	}

	void* GetStdInForWriting() const
	{
		return StdInWritePipe;
	}

	void* GetStdOutForProcess() const
	{
		return StdOutWritePipe;
	}

	void* GetStdOutForReading() const
	{
		return StdOutReadPipe;
	}

	void* GetStdErrForProcess() const
	{
		return StdErrWritePipe;
	}

	void* GetStdErrForReading() const
	{
		return StdErrReadPipe;
	}

private:

	void* StdOutReadPipe = nullptr;
	void* StdOutWritePipe = nullptr;

	void* StdErrReadPipe = nullptr;
	void* StdErrWritePipe = nullptr;

	void* StdInReadPipe = nullptr;
	void* StdInWritePipe = nullptr;

};
