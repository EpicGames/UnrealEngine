// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProcessWrapper.h"

#include "GenericPlatform/GenericPlatformTime.h"
#include "HAL/PlatformProcess.h"
#include "Logging/SubmitToolLog.h"
#include "Misc/DateTime.h"
#include "ProcessPipes.h"

FProcessWrapper::FProcessWrapper(const FString& InProcessName, const FString& InPath, const FString& InArgs, const FOnCompleted& InOnCompleted, const FOnOutputLine& InOnOutputLine, const FString& InWorkingDir, const bool InbLaunchHidden, const bool bLaunchReallyHidden, const bool InbLaunchDetached, bool InbOwnsProcessLifetime)
	: ProcessName(InProcessName)
	, Path(InPath)
	, Args(InArgs)
	, WorkingDir(InWorkingDir)
	, bLaunchesHidden(InbLaunchHidden)
	, bLaunchesReallyHidden(bLaunchReallyHidden)
	, bLaunchDetached(InbLaunchDetached)
	, bOwnsProcessLifetime(InbOwnsProcessLifetime)
	, Pipes(MakeUnique<FProcessPipes>())
	, OnCompleted(InOnCompleted)
	, OnOutputLine(InOnOutputLine)
{}

FProcessWrapper::~FProcessWrapper()
{
	if (bOwnsProcessLifetime)
	{
		Stop();
	}
	else
	{
		Cleanup();
	}
}

bool FProcessWrapper::Start(bool bWaitForExit)
{
	if(IsRunning())
	{
		OutputLine(FString::Format(TEXT("Process %s already running, ignored start request"), { *ProcessName }), EProcessOutputType::ProcessError);
		return false;
	}
	
	OutputRemainder.Reset();
	StdErrOutputRemainder.Reset();
	ExecutingTime = 0;

	if (!Pipes->Create())
	{
		OutputLine(FString::Format(TEXT("Error creating pipes in process {0}"), { *ProcessName }), EProcessOutputType::ProcessError);
		ExitCode = 11;
		bIsComplete = true;
		return false;
	}

	bStarted = true;
	OutputLine(FString::Format(TEXT("Running process {0}: {1} {2}"), { *ProcessName, *Path, *Args }), EProcessOutputType::ProcessInfo);

	ProcessHandle = MakeUnique<FProcHandle>(FPlatformProcess::CreateProc
	(
		*Path,
		*Args,
		bLaunchDetached,
		bLaunchesHidden,
		bLaunchesReallyHidden,
		nullptr,
		0,
		WorkingDir.Len() ? *WorkingDir : nullptr,
		Pipes->GetStdOutForProcess(),
		Pipes->GetStdInForProcess(),
		Pipes->GetStdErrForProcess())
	);

	if (!ProcessHandle->IsValid())
	{
		OutputLine(FString::Format(TEXT("Error creating process {0}"), { *ProcessName }), EProcessOutputType::ProcessError);
		ExitCode = 11;
		bIsComplete = true;
		return false;
	}

	
	if(bWaitForExit)
	{
		FDateTime Before = FDateTime::UtcNow();
		FPlatformProcess::WaitForProc(*ProcessHandle);
		FTimespan Timespan = FDateTime::UtcNow() - Before;
		OnTick(Timespan.GetTotalSeconds());
		return true;
	}
	else
	{
		TickerHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateRaw(this, &FProcessWrapper::OnTick));
		return true;
	}
}

void FProcessWrapper::Stop()
{
	if(IsRunning())
	{
		OutputLine(FString::Format(TEXT("Process {0} was stopped"), { *ProcessName }), EProcessOutputType::ProcessInfo);
		FPlatformProcess::TerminateProc(*ProcessHandle, true);
	}

	ExitCode = 10;
	bIsComplete = true;
	Cleanup();
}

bool FProcessWrapper::IsRunning() const
{
	return ProcessHandle.IsValid() && FPlatformProcess::IsProcRunning(*ProcessHandle);
}

bool FProcessWrapper::OnTick(float Delta)
{
	if(!ProcessHandle.IsValid())
	{
		return false;
	}

	bool bIsProcessRunning = FPlatformProcess::IsProcRunning(*ProcessHandle);
	ExecutingTime += Delta;

	// Flush output	only if process is finished
	ReadOutput(!bIsProcessRunning);

	if(!bIsProcessRunning)
	{
		FPlatformProcess::GetProcReturnCode(*ProcessHandle, &ExitCode);
		bIsComplete = true;

		OutputLine(FString::Printf(TEXT("Completed running process %s. Process took %s and exited with code %d"), *ProcessName, *FGenericPlatformTime::PrettyTime(ExecutingTime), ExitCode), EProcessOutputType::ProcessInfo);

		Cleanup();

		if(OnCompleted.IsBound())
		{
			OnCompleted.ExecuteIfBound(ExitCode);
		}
	}

	// Requeue Tick
	return bIsProcessRunning;
}

void FProcessWrapper::Cleanup()
{
	Pipes->Reset();
	ProcessHandle = nullptr;

	FTSTicker::RemoveTicker(TickerHandle);
	TickerHandle = nullptr;
}

void FProcessWrapper::ReadOutput(bool FlushOutput)
{
	if(!OnOutputLine.IsBound())
	{
		return;
	}

	//STDOut
	ProcessLines(FlushOutput, false);

	//STDErr
	ProcessLines(FlushOutput, true);
}

void FProcessWrapper::ProcessLines(bool bFlushOutput, bool bIsStdErr)
{
	FString OutputLines;
	if (bIsStdErr)
	{
		OutputLines = StdErrOutputRemainder + FPlatformProcess::ReadPipe(Pipes->GetStdErrForReading());
	}
	else
	{
		OutputLines = OutputRemainder + FPlatformProcess::ReadPipe(Pipes->GetStdOutForReading());
	}

	if (!bFlushOutput)
	{
		FString& OutputRemainderRef = bIsStdErr ? StdErrOutputRemainder : OutputRemainder;
		if (!OutputLines.Contains(TEXT("\n")))
		{
			// If we're not flushing and we only have a partial line, exit here and keep everything as remainder
			OutputRemainderRef = OutputLines;
			return;
		}

		// Find the last line, which may be truncated and store it in the output remainder for the next processline
		int32 Position;
		OutputLines.FindLastChar('\n', Position);
		OutputRemainderRef = OutputLines.Mid(Position + 1);
		OutputLines.RemoveFromEnd(OutputRemainderRef);
	}
	else
	{
		// if we've flushed, empty previous output remainder
		FString& OutputRemainderRef = bIsStdErr ? StdErrOutputRemainder : OutputRemainder;
		OutputRemainderRef.Empty();
	}

	// Parse into lines and fire line events
	TArray<FString> Lines;
	const TCHAR* Separators[] = { TEXT("\n"), TEXT("\r") };
	OutputLines.ParseIntoArray(Lines, Separators, UE_ARRAY_COUNT(Separators));

	for (FString& Line : Lines)
	{
		OutputLine(MoveTemp(Line), bIsStdErr ? EProcessOutputType::STDErr : EProcessOutputType::STDOutput);
	}
}

void FProcessWrapper::OutputLine(FString&& OutputLine, const EProcessOutputType& OutputType)
{
	OnOutputLine.ExecuteIfBound(MoveTemp(OutputLine), OutputType);
}
