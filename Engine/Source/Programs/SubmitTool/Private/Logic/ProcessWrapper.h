// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Containers/Ticker.h"
#include "Logic/ProcessPipes.h"

enum EProcessOutputType
{
	STDOutput,
	STDErr,
	ProcessInfo,
	ProcessError,
};

DECLARE_DELEGATE_TwoParams(FOnOutputLine, FString /*Output*/, const EProcessOutputType&)
DECLARE_DELEGATE_OneParam(FOnCompleted, const int32 /*Return code*/)

class FProcessWrapper
{
public:
	FProcessWrapper(const FString& InProcessName, const FString& InPath, const FString& InArgs, const FOnCompleted& InOnCompleted = FOnCompleted(), const FOnOutputLine& InOnOutputLine = FOnOutputLine(), const FString& InWorkingDir = FString(), const bool bLaunchHidden = true, const bool bLaunchReallyHidden = true, const bool bLaunchDetached = false, bool InbOwnsProcessLifetime = true);
	~FProcessWrapper();
	bool Start(bool bWaitForExit = false);
	void Stop();
	bool IsRunning() const;

	const FString& GetProcessName() const { return ProcessName; }
	const FString& GetExecutable() const { return Path; }
	const FString& GetArgs() const { return Args; }

	int32 ExitCode;
	float ExecutingTime = 0;
	bool bIsComplete = false;
	bool bStarted = false;
private:
	bool OnTick(float Delta);
	void Cleanup();
	void ReadOutput(bool FlushOutput);
	void ProcessLines(bool bFlushOutput, bool bIsStdErr);
	void OutputLine(FString&& Output, const EProcessOutputType& OutputType);
	
	FString ProcessName;
	FString Path;
	FString Args;
	FString WorkingDir;
	const bool bLaunchesHidden;
	const bool bLaunchesReallyHidden;
	const bool bLaunchDetached;
	const bool bOwnsProcessLifetime;

	TUniquePtr<FProcHandle> ProcessHandle;
	TUniquePtr<FProcessPipes> Pipes;
	FString OutputRemainder;
	FString StdErrOutputRemainder;

	FTSTicker::FDelegateHandle TickerHandle;

	const FOnCompleted OnCompleted = nullptr;
	const FOnOutputLine OnOutputLine = nullptr;
};
