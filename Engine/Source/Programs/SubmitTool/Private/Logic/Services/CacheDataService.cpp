// Copyright Epic Games, Inc. All Rights Reserved.

#include "CacheDataService.h"
#include "HAL/FileManagerGeneric.h"
#include "JsonObjectConverter.h"
#include "Misc/Paths.h"

#include "Logging/SubmitToolLog.h"
#include "Parameters/SubmitToolParameters.h"

FCacheDataService::FCacheDataService(const FGeneralParameters& InGeneralParameters) : Parameters(InGeneralParameters)
{
	LoadFromFile(FConfiguration::SubstituteAndNormalizeFilename(Parameters.CacheFile));
	CleanOldData();
}

FCacheDataService::~FCacheDataService()
{
}

bool FCacheDataService::GetChangelistCacheData(const FString& InCLId, FChangelistCacheData& OutCLData) const
{
	OutCLData = CacheData->CLCacheData.FindOrAdd(InCLId);
	return true;
}

const FDateTime FCacheDataService::GetLastValidationDate(const FString& InCLId, const FName& InValidatorId, const FString& InValidatorConfig, const FString& InFilePath) const
{
	FChangelistCacheData& Data = CacheData->CLCacheData.FindOrAdd(InCLId);
	Data.LastAccessed = FDateTime::UtcNow();
	return Data.GetLastValidationDate(InFilePath, InValidatorId, InValidatorConfig);
}

void FCacheDataService::UpdateLastValidationForFiles(const FString& InCLId, const FName& InValidatorId, const FString& InValidatorConfig, const TArray<FSCFileRef>& InFiles, const FDateTime& InNewTimestamp)
{
	FChangelistCacheData& Data = CacheData->CLCacheData.FindOrAdd(InCLId);
	Data.LastAccessed = FDateTime::UtcNow();

	for(const FSCFileRef& File : InFiles)
	{
		FString Filename = File->GetFilename();
		FPaths::NormalizeFilename(Filename);

		FValidationRecord& ValidationRecord = Data.LastFileValidations.FindOrAdd(Filename).LastValidationDateTimes.FindOrAdd(InValidatorId);
		ValidationRecord.ValidatorConfig = InValidatorConfig;
		ValidationRecord.LastValidationSuccess = InNewTimestamp;
	}
}

FString FCacheDataService::GetIntegrationFieldValue(const FString& InCLId, const FString& InIntegrationFieldId) const
{
	FChangelistCacheData& Data = CacheData->CLCacheData.FindOrAdd(InCLId);
	Data.LastAccessed = FDateTime::UtcNow();	
	return Data.IntegrationFields.Contains(InIntegrationFieldId) ? Data.IntegrationFields[InIntegrationFieldId] : FString();
}

void FCacheDataService::SetIntegrationFieldValue(const FString& InCLId, const FString& InIntegrationFieldId, const FString& InValue)
{
	FChangelistCacheData& Data = CacheData->CLCacheData.FindOrAdd(InCLId);
	Data.LastAccessed = FDateTime::UtcNow();
	Data.IntegrationFields.FindOrAdd(InIntegrationFieldId) = InValue;
}

void FCacheDataService::SaveCacheToDisk()
{
	CleanOldData();
	SaveToFile(FConfiguration::SubstituteAndNormalizeFilename(Parameters.CacheFile));
}

FDateTime FCacheDataService::GetLastKnownShelfUpToDate(const FString& InCLId) const
{
	FChangelistCacheData& Data = CacheData->CLCacheData.FindOrAdd(InCLId);
	Data.LastAccessed = FDateTime::UtcNow();
	return Data.LastShelfIsUpToDate;
}

void FCacheDataService::SetLastShelfKnownUpToDate(const FString& InCLId, const FDateTime& InDateTime)
{
	FChangelistCacheData& Data = CacheData->CLCacheData.FindOrAdd(InCLId);
	Data.LastAccessed = FDateTime::UtcNow();
	Data.LastShelfIsUpToDate = InDateTime;
}

void FCacheDataService::LoadFromFile(const FString& InFilePath)
{
	CacheData = MakeUnique<FCacheFile>();
	if(FPaths::FileExists(InFilePath))
	{
		FString InText;
		FArchive* File = IFileManager::Get().CreateFileReader(*InFilePath, EFileRead::FILEREAD_None);
		if (File == nullptr)
		{
			UE_LOGF(LogSubmitTool, Log, "Error opening cache data file %ls", *InFilePath);
			return;
		}

		*File << InText;
		File->Close();
		delete File;
		File = nullptr;

		FString OutputText;
		FText Errors;

		if(!FJsonObjectConverter::JsonObjectStringToUStruct(InText, CacheData.Get(), 0, 0, false, &Errors))
		{
			UE_LOGF(LogSubmitTool, Log, "Error loading cache data file %ls", *Errors.ToString());
		}
		else
		{
			UE_LOGF(LogSubmitToolDebug, Verbose, "Loaded Cache Data from %ls:\n%ls", *InFilePath, *InText);
		}
	}
	else
	{
		UE_LOGF(LogSubmitToolDebug, Log, "No cache data available %ls.", *InFilePath);
	}
}

void FCacheDataService::SaveToFile(const FString& InFilePath) const
{
	FString OutputText;
	FJsonObjectConverter::UStructToJsonObjectString(*CacheData, OutputText);

	FArchive* File = IFileManager::Get().CreateFileWriter(*InFilePath, EFileWrite::FILEWRITE_EvenIfReadOnly);
	if (File == nullptr)
	{
		UE_LOGF(LogSubmitTool, Log, "Error opening cache data file for writing %ls", *InFilePath);
		return;
	}

	*File << OutputText;
	File->Close();
	delete File;
	File = nullptr;
	UE_LOGF(LogSubmitToolDebug, Verbose, "Saved Cache Data to %ls:\n%ls", *InFilePath, *OutputText);
}

void FCacheDataService::CleanOldData()
{
	TArray<FString> OldData;
	for(const TPair<FString, FChangelistCacheData>& Pair : CacheData->CLCacheData)
	{
		if((FDateTime::UtcNow() - Pair.Value.LastAccessed).GetTotalHours() > Parameters.InvalidateCacheHours)
		{
			OldData.Add(Pair.Key);
		}
	}

	for(const FString& Key : OldData)
	{
		CacheData->CLCacheData.Remove(Key);
	}
}
