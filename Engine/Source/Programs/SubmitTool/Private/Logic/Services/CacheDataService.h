// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Models/CacheData.h"
#include "Logic/Services/Interfaces/ICacheDataService.h"

struct FGeneralParameters;

class FNoOpCacheDataService final : public ICacheDataService
{
public:
	virtual bool GetChangelistCacheData(const FString& InCLId, FChangelistCacheData& OutCLData) const override { return false; }

	virtual const FDateTime GetLastValidationDate(const FString& InCLId, const FName& InValidatorId, const FString& InValidatorConfig, const FString& InDepotPath) const override { return FDateTime::MinValue(); }
	virtual void UpdateLastValidationForFiles(const FString& InCLId, const FName& InValidatorId, const FString& InValidatorConfig, const TArray<FSCFileRef>& InFiles, const FDateTime& InNewTimestamp) override {}

	virtual FString GetIntegrationFieldValue(const FString& InCLId, const FString& InIntegrationFieldId) const override { return FString(); }
	virtual void SetIntegrationFieldValue(const FString& InCLId, const FString& InIntegrationFieldId, const FString& InValue) { }
	virtual FDateTime GetLastKnownShelfUpToDate(const FString& InCLId) const { return FDateTime::MinValue(); }
	virtual void SetLastShelfKnownUpToDate(const FString& InCLId, const FDateTime& InDateTime) { }

	virtual void SaveCacheToDisk() { }

};

class FCacheDataService final : public ICacheDataService
{
public:
	FCacheDataService(const FGeneralParameters& InGeneralParameters);
	~FCacheDataService();
	
	virtual bool GetChangelistCacheData(const FString& InCLId, FChangelistCacheData& OutCLData) const override;

	virtual const FDateTime GetLastValidationDate(const FString& InCLId, const FName& InValidatorId, const FString& InValidatorConfig, const FString& InDepotPath) const override;
	virtual void UpdateLastValidationForFiles(const FString& InCLId, const FName& InValidatorId, const FString& InValidatorConfig, const TArray<FSCFileRef>& InFiles, const FDateTime& InNewTimestamp) override;

	virtual FString GetIntegrationFieldValue(const FString& InCLId, const FString& InIntegrationFieldId) const override;
	virtual void SetIntegrationFieldValue(const FString& InCLId, const FString& InIntegrationFieldId, const FString& InValue);
	virtual FDateTime GetLastKnownShelfUpToDate(const FString& InCLId) const;
	virtual void SetLastShelfKnownUpToDate(const FString& InCLId, const FDateTime& InDateTime);

	virtual void SaveCacheToDisk();


private:
	void LoadFromFile(const FString& InFilePath);
	void SaveToFile(const FString& InFilePath) const;
	void CleanOldData();

	const FGeneralParameters& Parameters;
	TUniquePtr<FCacheFile> CacheData;
};
