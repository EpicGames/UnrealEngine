// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Memory/SharedBuffer.h"
#include "Internationalization/Internationalization.h"
#include "Logic/Services/Interfaces/ISTSourceControlService.h"

THIRD_PARTY_INCLUDES_START
#include <p4/clientapi.h>
THIRD_PARTY_INCLUDES_END

#include "PerforceData.generated.h"

#define FROM_TCHAR(InText, bIsUnicodeServer) (bIsUnicodeServer ? TCHAR_TO_UTF8(InText) : TCHAR_TO_ANSI(InText))
#define TO_TCHAR(InText, bIsUnicodeServer) (bIsUnicodeServer ? UTF8_TO_TCHAR(InText) : ANSI_TO_TCHAR(InText))

#define LOCTEXT_NAMESPACE "SubmitToolPerforce"

UENUM()
enum class EP4ClientUserFlags : uint64
{
	None = 0,
	/** The server uses unicode */
	UnicodeServer = 1 << 0,
	/** Binary data returned by commands should be collected in the DataBuffer member */
	CollectData = 1 << 1,
	UseZTag = 1 << 2,
	UseClient = 1 << 3,
	UseUser = 1 << 4,
	SilentErrors = 1 << 5,
};

ENUM_CLASS_FLAGS(EP4ClientUserFlags)

/**
 * A utility class to make it easier to gather a depot file from perforce when running
 * p4 print.
 */
class FP4File
{
public:
	FP4File() = default;
	~FP4File() = default;

	/**
	 * Start gathering the file in the given record. If the record is missing data
	 * then the gather will not begin. The calling code should check for this and
	 * raise errors or warnings accordingly.
	 * This class does not actually do any perforce work itself, and relies on a
	 * ClientUser to actually provide the data as it is downloaded.
	 */
	void Initialize(const TMap<FString, FString>& Record)
	{
		if(Record.Find(TEXT("fileSize")) != nullptr && Record.Find(TEXT("depotFile")) != nullptr)
		{
			const FString SizeAsString = Record[TEXT("fileSize")];
			DepotFilePath = Record[TEXT("depotFile")];

			int64 FileSize = FCString::Atoi64(*SizeAsString);

			Data = FUniqueBuffer::Alloc(FileSize);
			Offset = 0;
		}
	}

	/** Returns true if the FP4DepotFile was set up correctly and can gather a file. */
	bool IsValid() const
	{
		return !Data.IsNull();
	}

	/** Returns true if all of the files data has been acquired. */
	bool IsFileComplete() const
	{
		return Offset == (int64)Data.GetSize();
	}

	/** Returns the number of bytes in the file that have not yet been acquired. */
	int64 GetRemainingBytes() const
	{
		return (int64)Data.GetSize() - Offset;
	}

	/** Returns the depot path of the file we are gathering */
	const FString& GetDepotPath() const
	{
		return DepotFilePath;
	}

	/**
	 * Returns the currently acquired file data and then invalidates the FP4DepotFile.
	 * It is up to the caller to ensure that the entire file has been acquired or to
	 * decide if a partially acquired file is okay.
	 */
	FSharedBuffer Release()
	{
		Offset = INDEX_NONE;

		DepotFilePath.Reset();

		return Data.MoveToShared();
	}

	/* Used to reset the FP4DepotFile if an error is encountered */
	void Reset()
	{
		Offset = INDEX_NONE;

		DepotFilePath.Reset();

		Data.Reset();
	}

	/**
	 * Called when new data for the file has been downloaded and we can add it to the
	 * data that we have already required.
	 *
	 * @param DataPtr		The pointer to the downloaded data
	 * @param DataLength	The size of the downloaded data in bytes
	 * @return				True if the FP4DepotFile is valid and there was enough space
	 *						for the downloaded data. False if the FP4DepotFile is invalid
	 *						or if there is not enough space.
	 */
	bool OnDataDownloaded(const char* DataPtr, int DataLength)
	{
		if(DataLength <= GetRemainingBytes())
		{
			FMemory::Memcpy((char*)Data.GetData() + Offset, DataPtr, DataLength);
			Offset += DataLength;

			return true;
		}
		else
		{
			return false;
		}
	}

private:

	/** The path of the file in the perforce depot */
	FString DepotFilePath;

	/** The buffer containing the file data, allocated up front. */
	FUniqueBuffer Data;
	/** Tracks where the next set on downloaded data should be placed in the buffer. */
	int64 Offset = INDEX_NONE;
};

/** Custom ClientUser class for handling results and errors from Perforce commands */
class FSTClientUser : public ClientUser
{
public:

	FSTClientUser(FSCCommand& InCommand, EP4ClientUserFlags InFlags)
		: ClientUser()
		, Flags(InFlags)
		, Command(InCommand)
	{

	}


	/** Called by P4API on "change -i" command to fill OutBuffer. */
	virtual void InputData(StrBuf* OutBuffer, Error* OutError) override
	{
		if (!Command.InputData.IsEmpty())
		{
			OutBuffer->Append(FROM_TCHAR(*Command.InputData, IsUnicodeServer()));
		}
	}


	/**  Called by P4API when the results from running a command are ready. */
	virtual void OutputStat(StrDict* VarList) override
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FSTClientUser::OutputStat);

		TMap<FString, FString> Record = {};
		StrRef Var, Value;

		// Iterate over each variable and add to records
		for(int32 Index = 0; VarList->GetVar(Index, Var, Value); Index++)
		{
			Record.Add(TO_TCHAR(Var.Text(), IsUnicodeServer()), TO_TCHAR(Value.Text(), IsUnicodeServer()));
		}

		if(IsCollectingData())
		{
			if(File.IsValid())
			{
				FText Message = FText::Format(LOCTEXT("P4Client_GatheringUnfinished", "Started gathering depot file '{0}' before the previous file finished!"),
					FText::FromString(File.GetDepotPath()));

				Command.Results.ErrorMessages.Add(MoveTemp(Message));
			}

			File.Initialize(Record);
		}

		Command.Values.Add(Record);
	}

	/** Called by P4API when it output a chunk of text data from a file (commonly via P4 Print) */
	virtual void OutputText(const char* DataPtr, int DataLength) override
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FSTClientUser::OutputText);

		if(IsCollectingData())
		{
			if(File.OnDataDownloaded(DataPtr, DataLength))
			{
				if(File.IsFileComplete())
				{
					const FString Filename = File.GetDepotPath();
					Files.Add(Filename, File.Release());
				}
			}
			else
			{
				FText Message = FText::Format(
					LOCTEXT("P4Client_TextCollectionFailed", "Collecting text data requires {0} bytes but the buffer only has {1} bytes remaining: {2}"),
					DataLength,
					File.GetRemainingBytes(),
					FText::FromString(File.GetDepotPath()));

				Command.Results.ErrorMessages.Add(MoveTemp(Message));

				File.Reset();
			}
		}
		else
		{
			ClientUser::OutputText(DataPtr, DataLength);
		}
	}

	/**  Called by P4API when it output a chunk of binary data from a file (commonly via P4 Print) */
	void OutputBinary(const char* DataPtr, int DataLength)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FSTClientUser::OutputBinary);

		if(IsCollectingData())
		{
			// For binary files we get a zero size call once the file is completed so we wait for that
			// rather than checking FP4DepotFile::isFileComplete after every transfer
			if(DataLength == 0)
			{
				if(File.IsFileComplete())
				{
					const FString Filename = File.GetDepotPath();
					Files.Add(Filename, File.Release());
				}
				else
				{
					FText Message = FText::Format(
						LOCTEXT("P4Client_IncompleteFIle", "Collecting binary data completed but missing {0} bytes: {1}"),
						File.GetRemainingBytes(),
						FText::FromString(File.GetDepotPath()));

					Command.Results.ErrorMessages.Add(MoveTemp(Message));

					File.Reset();
				}
			}
			else if(!File.OnDataDownloaded(DataPtr, DataLength))
			{
				FText Message = FText::Format(
					LOCTEXT("P4Client_BinaryCollectionFailed", "Collecting binary data requires {0} bytes but the buffer only has {1} bytes remaining: {2}"),
					DataLength,
					File.GetRemainingBytes(),
					FText::FromString(File.GetDepotPath()));

				Command.Results.ErrorMessages.Add(MoveTemp(Message));

				File.Reset();
			}
		}
		else
		{
			ClientUser::OutputText(DataPtr, DataLength);
		}
	}

	virtual void Message(Error* err) override
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FSTClientUser::Message);

		StrBuf Buffer;
		err->Fmt(Buffer, EF_PLAIN);

		FString Message(TO_TCHAR(Buffer.Text(), IsUnicodeServer()));

		// Previously we used ::HandleError which would have \n at the end of each line.
		// For now we should add that to maintain compatibility with existing code.
		if(!Message.EndsWith(TEXT("\n")))
		{
			Message.Append(TEXT("\n"));
		}

		if(err->GetSeverity() <= ErrorSeverity::E_INFO)
		{
			Command.Results.InfoMessages.Add(FText::FromString(MoveTemp(Message)));
		}
		else
		{
			Command.Results.ErrorMessages.Add(FText::FromString(MoveTemp(Message)));
		}
	}

	virtual void OutputInfo(char Indent, const char* InInfo) override
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FSTClientUser::OutputInfo);

		// We don't expect this to ever be called (info messages should come
		// via ClientUser::Message) but implemented just to be safe.

		Command.Results.InfoMessages.Add(FText::FromString(FString(TO_TCHAR(InInfo, IsUnicodeServer()))));
	}

	virtual void OutputError(const char* errBuf) override
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FSTClientUser::OutputError);

		// In general we expect errors to be passed to use via ClientUser::Message but some
		// errors raised by the p4 cpp api can call ::HandleError or ::OutputError directly.
		// Since the default implementation of ::HandleError calls ::OutputError we only need
		// to implement this method to make sure we capture all of the errors being passed in
		// this way.

		Command.Results.ErrorMessages.Add(FText::FromString(FString(TO_TCHAR(errBuf, IsUnicodeServer()))));
	}

	inline bool IsUnicodeServer() const
	{
		return EnumHasAnyFlags(Flags, EP4ClientUserFlags::UnicodeServer);
	}

	inline bool IsCollectingData() const
	{
		return EnumHasAnyFlags(Flags, EP4ClientUserFlags::CollectData);
	}

	/** Returns DataBuffer as a FSharedBuffer, note that once called DataBuffer will be empty */
	inline TMap<FString, FSharedBuffer> ReleaseData()
	{
		return MoveTemp(Files);
	}

	EP4ClientUserFlags Flags;

	FSCCommand& Command;

private:
	TMap<FString, FSharedBuffer> Files;

	FP4File File;
};

#undef LOCTEXT_NAMESPACE
