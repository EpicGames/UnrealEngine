// Copyright Epic Games, Inc. All Rights Reserved.

#include "SubmitToolPerforce.h"
#include "Logic/ProcessWrapper.h"
#include "SubmitToolCoreUtils.h"
#include "Configuration/Configuration.h"
#include "Logging/SubmitToolLog.h"
#include "Logic/CredentialsService.h"
#include "Async/Async.h"

#include "CommandLine/CmdLineParameters.h"
#include "Misc/Base64.h"
#include "Misc/Paths.h"
#include "HAL/FileManager.h"

constexpr size_t MaxRecentUsers = 12;
constexpr size_t RecentUsersDatVersion = 1;
constexpr size_t MaxRecentGroups = 12;
constexpr size_t RecentGroupsDatVersion = 1;
constexpr size_t MaxConnections = 7;
constexpr size_t MaxConnectionsAttempts = 50; // 50*0.2f => ~10s
constexpr float ConnectionAttemptWaitTime = 0.2f;


FP4Connection::FP4Connection(FClientApiWrapper& InConnection)
	: Connection(InConnection)
{
	Connection.bIsReady = false;
}

FP4Connection::~FP4Connection()
{
	Connection.bIsReady = true;
}

FConnectionPool::~FConnectionPool()
{
	for(TUniquePtr<FClientApiWrapper>& ConWrapper : P4Connections)
	{
		ConWrapper->bWantsCancel = true;
		Error P4Error;
		ConWrapper->Connection->Final(&P4Error);
		if(P4Error.Test())
		{
			StrBuf ErrorMsg;
			P4Error.Fmt(&ErrorMsg);
			UE_LOGF(LogSubmitToolP4, Error, "P4ERROR: Invalid connection to server.");
			UE_LOGF(LogSubmitToolP4, Error, "%ls", ANSI_TO_TCHAR(ErrorMsg.Text()));
		}
	}
}

TUniquePtr<FP4Connection> FConnectionPool::GetAvailableConnection()
{
	FScopeLock Lock(&Mutex);

	int32 ClosingConnections = 0;
	int32 I = 0;
	for (TArray<TUniquePtr<FClientApiWrapper>>::TIterator It = P4Connections.CreateIterator(); It; ++It, ++I)
	{
		FClientApiWrapper* ConWrapper = It->Get();
		if (ConWrapper->Connection->Dropped() == 1)
		{
			UE_LOGF(LogSubmitToolP4Debug, Log, "Connection #%d dropped.", I);
			It.RemoveCurrent();
		}
		else if (ConWrapper->bWantsCancel)
		{
			UE_LOGF(LogSubmitToolP4Debug, Log, "Connection #%d is cancelling.", I);
			ClosingConnections++;
		}
		else if (ConWrapper->bIsReady)
		{
			return MakeUnique<FP4Connection>(*ConWrapper);
		}
	}

	const int32 NumConnections = P4Connections.Num() - ClosingConnections;
	if (NumConnections < MaxConnections)
	{
		UE_LOGF(LogSubmitToolP4Debug, Log, "Creating new p4 connection %d/%d.", NumConnections + 1, MaxConnections);
		int32 NewIdx = CreateConnection();
		if (NewIdx != -1)
		{
			return MakeUnique<FP4Connection>(*P4Connections[NewIdx]);
		}
	}

	return nullptr;
}

void FConnectionPool::CancelCurrentRequests()
{
	for (int32 I = 0; I < P4Connections.Num(); ++I)
	{
		const TUniquePtr<FClientApiWrapper>& Connection = P4Connections[I];
		if (!Connection->bIsReady)
		{
			UE_LOGF(LogSubmitToolP4Debug, Log, "Cancelling connection #%d.", I);
			Connection->bWantsCancel = true;
		}
	}
}

int32 FConnectionPool::CreateConnection()
{
	if(bConnectionFailed)
	{
		return -1;
	}

	TUniquePtr<ClientApi> P4Client = MakeUnique<ClientApi>();
	FString Port;
	FCmdLineParameters::Get().GetValue(FSubmitToolCmdLine::P4Server, Port);
	P4Client->SetPort(TCHAR_TO_ANSI(*Port));

	Error P4Error;
	P4Client->Init(&P4Error);
	if(P4Error.Test())
	{
		StrBuf ErrorMsg;
		P4Error.Fmt(&ErrorMsg);
		UE_LOGF(LogSubmitToolP4, Error, "P4ERROR: Invalid connection to server.");
		UE_LOGF(LogSubmitToolP4, Error, "%ls", ANSI_TO_TCHAR(ErrorMsg.Text()));
		bConnectionFailed = true;
		return -1;
	}

	FScopeLock Lock(&Mutex);
	return P4Connections.Add(MakeUnique<FClientApiWrapper>(MoveTemp(P4Client)));
}

FSubmitToolPerforce::FSubmitToolPerforce()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FSubmitToolPerforce::FSubmitToolPerforce);
	InitialStreamTask = GetStream();
}

FSubmitToolPerforce::~FSubmitToolPerforce()
{
	OnUsersGetCallbacks.Clear();
	OnGroupsGetCallbacks.Clear();

	FTSTicker::RemoveTicker(TickHandle);
}

void FSubmitToolPerforce::InitializeParameters(const FSubmitToolParameters& InParameters)
{
	GroupsToExclude = InParameters.GeneralParameters.GroupsToExclude;
}

UE::Tasks::TTask<TSharedRef<FSCCommand>> FSubmitToolPerforce::GetStream(const FString& InStream)
{
	TArray<FString> Args = { TEXT("-o") };

	if(!InStream.IsEmpty())
	{
		Args.Add(InStream);
	}

	return RunCommand(TEXT("stream"), MoveTemp(Args));
}

void FSubmitToolPerforce::ParseStreamCommand(const FSCCommand& InCompletedCommand, const FString& InRequestedStream, bool bParseHierarchy)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FSubmitToolPerforce::ParseStreamCommand);
	if (InCompletedCommand.bSuccess && InCompletedCommand.Values.Num() > 0 && InCompletedCommand.Values[0].Contains(TEXT("Stream")))
	{
		TUniquePtr<FSCCStream> Stream = MakeUnique<FSCCStream>(InCompletedCommand.Values[0][TEXT("Stream")], InCompletedCommand.Values[0][TEXT("Parent")], InCompletedCommand.Values[0][TEXT("Type")]);

		const FString Base = TEXT("Paths");
		size_t i = 1;
		FString PathsKey = FString::Printf(TEXT("%s%d"), *Base, i);
		while (InCompletedCommand.Values[0].Contains(PathsKey))
		{
			FString Value = InCompletedCommand.Values[0][PathsKey];
			if (Value.StartsWith(TEXT("import")))
			{
				int32 Idx = Value.Find(TEXT("//"));
				if (Idx != INDEX_NONE)
				{
					Value = Value.RightChop(Value.Find(TEXT("//")));
				}

				while (!Value.EndsWith(TEXT("/")))
				{
					Value.RemoveAt(Value.Len() - 1);
				}

				Stream->AdditionalImportPaths.Add(Value);
			}

			++i;
			PathsKey = FString::Printf(TEXT("%s%d"), *Base, i);
		}

		const FString Parent = Stream->Parent;

		if (!Streams.Contains(Stream->Name))
		{
			if (bParseHierarchy && !StreamHierarchy.ContainsByPredicate([&Stream](const FSCCStream* InStream) { return InStream->Name.Equals(Stream->Name, ESearchCase::IgnoreCase); }))
			{
				StreamHierarchy.Insert(Stream.Get(), 0);
			}

			Streams.Add(Stream->Name, MoveTemp(Stream));
		}

		if (bParseHierarchy && !Parent.IsEmpty() && !Parent.Equals(TEXT("none")))
		{
			ParseStreamCommand(RunSyncCommand(TEXT("stream"), { TEXT("-o"), Parent }), Parent, bParseHierarchy);
		}
	}
	else
	{
		TUniquePtr<FSCCStream> Stream = MakeUnique<FSCCStream>(TEXT("Invalid"), FString(), TEXT("Invalid"));

		if (!InRequestedStream.IsEmpty())
		{
			if (!Streams.Contains(InRequestedStream))
			{
				Streams.Add(InRequestedStream, MoveTemp(Stream));
			}
		}
		else if (!Streams.Contains(TEXT("Invalid")))
		{
			StreamHierarchy.Add(Stream.Get());
			Streams.Add(TEXT("Invalid"), MoveTemp(Stream));
		}
	}
}


void FSubmitToolPerforce::GetUsers(const FOnUsersGet::FDelegate& Callback)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FSubmitToolPerforce::GetUsers);

	if (CachedUsersArray.Num())
	{
		Callback.ExecuteIfBound(CachedUsersArray);
		return;
	}

	OnUsersGetCallbacks.Add(Callback);

	if (!bUserTaskIsRunning)
	{
		bUserTaskIsRunning = true;
		RunCommand(TEXT("users"), {}, FOnSCCCommandComplete::CreateLambda(
			[this](const FSCCommand& Cmd)
			{
				for (const TMap<FString, FString>& Record : Cmd.Values)
				{
					if (Record.Contains("User"))
					{
						TSharedPtr<FUserData> User = MakeShared<FUserData>(Record[TEXT("User")], Record[TEXT("FullName")], Record[TEXT("Email")]);
						CachedUsersArray.Add(User);
						CachedUsers.Add(User->Username, User);
					}
				}

				LoadRecentUsers();

				OnUsersGetCallbacks.Broadcast(CachedUsersArray);
				OnUsersGetCallbacks.Clear();
				bUserTaskIsRunning = false;
			}
		));
	}
}

void FSubmitToolPerforce::LoadRecentUsers()
{
	const TUniquePtr<FAES::FAESKey>& Key = FCredentialsService::GetEncryptionKey();

	if(!Key.IsValid())
	{
		return;
	}

	FString RecentUsersString;

	const FString FilePath = GetRecentUsersFilepath();
	if(IFileManager::Get().FileExists(*FilePath))
	{
		FArchive* File = IFileManager::Get().CreateFileReader(*FilePath);

		if(File != nullptr)
		{
			int32 Version;
			*File << Version;

			// Check Versions here
			if(Version != RecentUsersDatVersion)
			{
				UE_LOGF(LogSubmitToolDebug, Warning, "Unexpected Recent Users Version, aborting issues loading.");
				File->Close();
				delete File;
				File = nullptr;
				return;
			}

			int32 PaddedLength;
			int32 LengthWithoutPadding;

			*File << PaddedLength;
			*File << LengthWithoutPadding;

			TArray<uint8> DeserializedBytes;
			DeserializedBytes.SetNum(PaddedLength);
			File->Serialize(DeserializedBytes.GetData(), PaddedLength);

			FAES::DecryptData(DeserializedBytes.GetData(), DeserializedBytes.Num(), *Key);

			RecentUsersString = BytesToString(DeserializedBytes.GetData(), LengthWithoutPadding);

			File->Close();
			delete File;
			File = nullptr;
		}
		else
		{
			UE_LOGF(LogSubmitTool, Warning, "Could not read file '%ls'.", *FilePath);
		}
	}
	else
	{
		UE_LOGF(LogSubmitToolDebug, Log, "File %ls does not exists, no recent users were loaded", *FilePath);
	}

	TArray<FString> Usernames;
	RecentUsersString.ParseIntoArray(Usernames, TEXT(";"));

	RecentUsers.Empty();
	for(FString& Username : Usernames)
	{
		if(const TSharedPtr<FUserData>* User = CachedUsers.Find(Username))
		{
			RecentUsers.Add(*User);
		}
	}
}

TSharedPtr<FUserData> FSubmitToolPerforce::GetUserDataFromCache(const FString& Username) const
{
	if(CachedUsers.Contains(Username))
	{
		return CachedUsers[Username];
	}
	return nullptr;
}

void FSubmitToolPerforce::SaveRecentUsers()
{
	const FString RecentUsersString = FString::JoinBy(RecentUsers, TEXT(";"), [](const TSharedPtr<FUserData>& User) { return User->Username; });
	const TUniquePtr<FAES::FAESKey>& Key = FCredentialsService::GetEncryptionKey();

	const FString FilePath = GetRecentUsersFilepath();
	FArchive* File = IFileManager::Get().CreateFileWriter(*FilePath, EFileWrite::FILEWRITE_EvenIfReadOnly);

	if(File == nullptr)
	{
		UE_LOGF(LogSubmitTool, Warning, "Could not create file '%ls'.", *FilePath);
		return;
	}

	TArray<uint8> Bytes;
	Bytes.SetNumUninitialized(RecentUsersString.Len());
	StringToBytes(RecentUsersString, Bytes.GetData(), RecentUsersString.Len());

	int32 ActualLength = Bytes.Num();

	int32 NumBytesEncrypted = Align(Bytes.Num(), FAES::AESBlockSize);
	Bytes.SetNum(NumBytesEncrypted);
	FAES::EncryptData(Bytes.GetData(), Bytes.Num(), *Key);

	int32 Version = RecentUsersDatVersion;
	*File << Version;

	*File << NumBytesEncrypted;
	*File << ActualLength;

	File->Serialize(Bytes.GetData(), Bytes.Num());

	File->Close();
	delete File;
	File = nullptr;
}

const TArray<TSharedPtr<FUserData>>& FSubmitToolPerforce::GetRecentUsers() const
{
	return RecentUsers;
}

void FSubmitToolPerforce::AddRecentUser(TSharedPtr<FUserData>& User)
{
	if(RecentUsers.Contains(User))
	{
		// Remove so we can push the user to the top
		RecentUsers.Remove(User);
	}

	if(RecentUsers.Num() >= MaxRecentUsers)
	{
		RecentUsers.RemoveAt(MaxRecentUsers - 1);
	}

	RecentUsers.EmplaceAt(0, User);
	SaveRecentUsers();
}

const FString FSubmitToolPerforce::GetRecentUsersFilepath() const
{
	return FPaths::Combine(FSubmitToolCoreUtils::GetLocalAppDataPath(), TEXT("SubmitTool"), TEXT("recent_users.dat"));
}

void FSubmitToolPerforce::GetGroups(const FOnGroupsGet::FDelegate& Callback)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FSubmitToolPerforce::GetGroups);

	if (!CachedGroupsArray.IsEmpty())
	{
		Callback.ExecuteIfBound(CachedGroupsArray);
		return;
	}

	OnGroupsGetCallbacks.Add(Callback);

	if(!bGroupTaskIsRunning)
	{
		bGroupTaskIsRunning = true;
		UE::Tasks::Launch(UE_SOURCE_LOCATION,
			[this] {
				TRACE_CPUPROFILER_EVENT_SCOPE(FSubmitToolPerforce::GetGroups);
				TUniquePtr<FP4Connection> Connection = Connections.GetAvailableConnection();
				
				size_t Attempts = 0;
				while(!Connection.IsValid())
				{
					TRACE_CPUPROFILER_EVENT_SCOPE(WaitForConnection);
					if(Attempts >= MaxConnectionsAttempts)
					{
						bGroupTaskIsRunning = false;
						UE::Tasks::Launch(UE_SOURCE_LOCATION, [this]() mutable
						{
							LoadRecentGroups();
							bGroupTaskIsRunning = false;
							OnGroupsGetCallbacks.Broadcast(CachedGroupsArray);
							OnGroupsGetCallbacks.Clear();
						}, LowLevelTasks::ETaskPriority::Normal, UE::Tasks::EExtendedTaskPriority::GameThreadNormalPri);
						return false;
					}

					FPlatformProcess::Sleep(ConnectionAttemptWaitTime);
					Connection = Connections.GetAvailableConnection();

					++Attempts;
				}

				ClientApi& P4Client = Connection->GetConnection();

				FString UserName;
				FCmdLineParameters::Get().GetValue(FSubmitToolCmdLine::P4User, UserName);
				P4Client.SetUser(TCHAR_TO_ANSI(*UserName));

				FString Client;
				FCmdLineParameters::Get().GetValue(FSubmitToolCmdLine::P4Client, Client);
				P4Client.SetClient(TCHAR_TO_ANSI(*Client));

				P4Client.SetProtocol("tag", "");


				class FGroupsClientUser : public FSTClientUser
				{
					public:
					FGroupsClientUser(FSCCommand& InCommand, EP4ClientUserFlags InFlags, TArray<TSharedPtr<FString>>& OutGroupsArray, const TArray<FString>& InGroupsToExclude)
					: FSTClientUser(InCommand, InFlags), GroupsArray(OutGroupsArray), ExcludedGroups(InGroupsToExclude)
					{}

					virtual void OutputStat(StrDict* VarList) override
					{
						TRACE_CPUPROFILER_EVENT_SCOPE(FGroupsClientUser::OutputStat);
						StrRef Var, Value;

						FString Group;
						int32 FoundKeys = 0;
						bool bIsSubGroup = false;
						// Iterate over each variable and add to records
						for (int32 Index = 0; FoundKeys < 2 && VarList->GetVar(Index, Var, Value); Index++)
						{
							if (Var == "isSubGroup")
							{
								bIsSubGroup = Value != "0";
								++FoundKeys;
							}
							else if (Var == "group")
							{
								Group = TO_TCHAR(Value.Text(), IsUnicodeServer());
								++FoundKeys;
							}
						}

						if (bIsSubGroup || Group.IsEmpty())
						{
							return;
						}

						if (GroupsArray.ContainsByPredicate([&Group](const TSharedPtr<FString>& InStr) { return InStr->Equals(Group, ESearchCase::IgnoreCase); }))
						{
							return;
						}

						for (const FString& Filter : ExcludedGroups)
						{
							if (Group.StartsWith(Filter))
							{
								return;
							}
						}

						GroupsArray.Add(MakeShared<FString>(Group));
					}

					TArray<TSharedPtr<FString>>& GroupsArray;
					const TArray<FString> ExcludedGroups;
				};

				TArray<TSharedPtr<FString>> GroupsResults;

				FSCCommand Command;
				FGroupsClientUser P4User(Command, bIsUnicodeServer ? EP4ClientUserFlags::UnicodeServer : EP4ClientUserFlags::None, GroupsResults, GroupsToExclude);

				{
					TRACE_CPUPROFILER_EVENT_SCOPE(FSubmitToolPerforce::RunP4Command_groups);
					UE_LOGF(LogSubmitToolP4Debug, Log, "Running command: p4 -p %ls -u %ls -c %ls -ztag groups", TO_TCHAR(P4Client.GetPort().Text(), bIsUnicodeServer), *UserName, *Client);
					P4Client.Run("groups", &P4User);
				}

				for(const FText& Msg : Command.Results.InfoMessages)
				{
					UE_LOGF(LogSubmitToolP4Debug, Verbose, "p4 groups: %ls", *Msg.ToString());
				}

				if(Command.Results.HasErrors())
				{
					for(const FText& Error : Command.Results.ErrorMessages)
					{
						UE_LOGF(LogSubmitToolP4, Error, "p4 groups: %ls", *Error.ToString());
					}
				}

				UE::Tasks::Launch(UE_SOURCE_LOCATION, [this, GroupsResults = MoveTemp(GroupsResults)]() mutable
				{
					TRACE_CPUPROFILER_EVENT_SCOPE(FSubmitToolPerforce::GetGroups_Broadcast);
					CachedGroupsArray = MoveTemp(GroupsResults);
					LoadRecentGroups();
					bGroupTaskIsRunning = false;
					OnGroupsGetCallbacks.Broadcast(CachedGroupsArray);
					OnGroupsGetCallbacks.Clear();
				}, LowLevelTasks::ETaskPriority::Normal, UE::Tasks::EExtendedTaskPriority::GameThreadNormalPri);

				return !Command.Results.HasErrors();
			});
	}
}

void FSubmitToolPerforce::LoadRecentGroups()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FSubmitToolPerforce::LoadRecentGroups);
	const TUniquePtr<FAES::FAESKey>& Key = FCredentialsService::GetEncryptionKey();

	if(!Key.IsValid())
	{
		return;
	}

	FString RecentGroupsString;

	const FString FilePath = GetRecentGroupsFilepath();
	if(IFileManager::Get().FileExists(*FilePath))
	{
		FArchive* File = IFileManager::Get().CreateFileReader(*FilePath);

		if(File != nullptr)
		{
			int32 Version;
			*File << Version;

			// Check Versions here
			if(Version != RecentGroupsDatVersion)
			{
				UE_LOGF(LogSubmitToolDebug, Warning, "Unexpected Recent Groups Version, aborting issues loading.");
				File->Close();
				delete File;
				File = nullptr;
				return;
			}

			int32 PaddedLength;
			int32 LengthWithoutPadding;

			*File << PaddedLength;
			*File << LengthWithoutPadding;

			TArray<uint8> DeserializedBytes;
			DeserializedBytes.SetNum(PaddedLength);
			File->Serialize(DeserializedBytes.GetData(), PaddedLength);

			FAES::DecryptData(DeserializedBytes.GetData(), DeserializedBytes.Num(), *Key);

			RecentGroupsString = BytesToString(DeserializedBytes.GetData(), LengthWithoutPadding);

			File->Close();
			delete File;
			File = nullptr;
		}
		else
		{
			UE_LOGF(LogSubmitTool, Warning, "Could not read file '%ls'.", *FilePath);
		}
	}
	else
	{
		UE_LOGF(LogSubmitToolDebug, Log, "File %ls does not exists, no recent groups were loaded", *FilePath);
	}

	TArray<FString> GroupNames;
	RecentGroupsString.ParseIntoArray(GroupNames, TEXT(";"));

	RecentGroups.Empty();
	for(FString& Name : GroupNames)
	{
		if(const TSharedPtr<FString>* Group = CachedGroupsArray.FindByPredicate([&Name](TSharedPtr<FString>& InStr) { return InStr->Equals(Name, ESearchCase::IgnoreCase); }))
		{
			RecentGroups.Add(*Group);
		}
	}
}

void FSubmitToolPerforce::SaveRecentGroups()
{
	const FString RecentGroupsString = FString::JoinBy(RecentGroups, TEXT(";"), [](const TSharedPtr<FString>& Group) { return *Group; });
	const TUniquePtr<FAES::FAESKey>& Key = FCredentialsService::GetEncryptionKey();

	const FString FilePath = GetRecentGroupsFilepath();
	FArchive* File = IFileManager::Get().CreateFileWriter(*FilePath, EFileWrite::FILEWRITE_EvenIfReadOnly);

	if(File == nullptr)
	{
		UE_LOGF(LogSubmitTool, Warning, "Could not create file '%ls'.", *FilePath);
		return;
	}

	TArray<uint8> Bytes;
	Bytes.SetNumUninitialized(RecentGroupsString.Len());
	StringToBytes(RecentGroupsString, Bytes.GetData(), RecentGroupsString.Len());

	int32 ActualLength = Bytes.Num();

	int32 NumBytesEncrypted = Align(Bytes.Num(), FAES::AESBlockSize);
	Bytes.SetNum(NumBytesEncrypted);
	FAES::EncryptData(Bytes.GetData(), Bytes.Num(), *Key);

	int32 Version = RecentGroupsDatVersion;
	*File << Version;

	*File << NumBytesEncrypted;
	*File << ActualLength;

	File->Serialize(Bytes.GetData(), Bytes.Num());

	File->Close();
	delete File;
	File = nullptr;
}

const TArray<TSharedPtr<FString>>& FSubmitToolPerforce::GetRecentGroups() const
{
	return RecentGroups;
}

void FSubmitToolPerforce::AddRecentGroup(TSharedPtr<FString>& Group)
{
	if(RecentGroups.Contains(Group))
	{
		// Remove so we can push the user to the top
		RecentGroups.Remove(Group);
	}

	if(RecentGroups.Num() >= MaxRecentGroups)
	{
		RecentGroups.RemoveAt(MaxRecentGroups - 1);
	}

	RecentGroups.EmplaceAt(0, Group);
	SaveRecentGroups();
}

const FString FSubmitToolPerforce::GetRecentGroupsFilepath() const
{
	return FPaths::Combine(FSubmitToolCoreUtils::GetLocalAppDataPath(), TEXT("SubmitTool"), TEXT("recent_groups.dat"));
}

UE::Tasks::TTask<bool> FSubmitToolPerforce::DownloadFiles(TArray<FString>&& FilePaths, TMap<FString, FSharedBuffer>& OutFileBuffers, bool bSilentErrors)
{
	FilePaths.Insert(TEXT("-q"), 0);
	TSharedRef<FSCCommand> Command = MakeShared<FSCCommand>(TEXT("print"), MoveTemp(FilePaths));
	Command->bCollectData = true;
	Command->bSilentErrors = bSilentErrors;
	return UE::Tasks::Launch(UE_SOURCE_LOCATION, [this, Command, &OutFileBuffers]
	{
		if (RunP4Command(*Command, DefaultFlags | EP4ClientUserFlags::CollectData) || Command->Data.Num() != 0)
		{
			OutFileBuffers = MoveTemp(Command->Data);
			return true;
		}
		return false;
	});
}

void FSubmitToolPerforce::GetUsersAndGroups(const FOnUsersAndGroupsGet::FDelegate& Callback)
{
	OnUsersAndGroupsGetCallbacks.Add(Callback);

	auto OnUsersAndGroupsReady = [this]() {
		if(bUsersReady && bGroupsReady)
		{
			OnUsersAndGroupsGetCallbacks.Broadcast(CachedUsersArray, CachedGroupsArray);
			OnUsersAndGroupsGetCallbacks.Clear();

		}
	};

	GetUsers(FOnUsersGet::FDelegate::CreateLambda([OnUsersAndGroupsReady, this](TArray<TSharedPtr<FUserData>>&) { bUsersReady = true; OnUsersAndGroupsReady(); }));
	GetGroups(FOnGroupsGet::FDelegate::CreateLambda([OnUsersAndGroupsReady, this](TArray<TSharedPtr<FString>>&) { bGroupsReady = true; OnUsersAndGroupsReady(); }));
}

bool FSubmitToolPerforce::RunCommandInternal(FSCCommand& Command)
{
	EP4ClientUserFlags Flags = DefaultFlags;
	if (Command.bSilentErrors)
	{
		Flags |= EP4ClientUserFlags::SilentErrors;
	}

	return RunP4Command(Command, Flags);
}

const TArray<FSCCStream*>& FSubmitToolPerforce::GetClientStreams() const
{
	return StreamHierarchy;
}

const FSCCStream* FSubmitToolPerforce::GetSCCStream(const FString& InStreamName)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FSubmitToolPerforce::GetSCCStream);
	for (int32 I = 0; I < 2; ++I)
	{
		if (TUniquePtr<FSCCStream>* Stream = Streams.Find(InStreamName))
		{
			return Stream->Get();
		}
		if (I == 0)
		{
			TSharedRef<FSCCommand> GetStreamResult = GetStream(InStreamName).GetResult();
			ParseStreamCommand(*GetStreamResult, InStreamName, false);
		}
	}
	return nullptr;
}

void FSubmitToolPerforce::WaitForInitialStreamTask()
{
	if (InitialStreamTask.IsValid())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FSubmitToolPerforce::WaitForInitialStreamTask);
		ParseStreamCommand(*InitialStreamTask.GetResult());
		InitialStreamTask = {};
	}
}

const FString FSubmitToolPerforce::GetRootStreamName()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FSubmitToolPerforce::GetRootStreamName);
	WaitForInitialStreamTask();
	for (int32 I = 0; I < 2; ++I)
	{
		if (StreamHierarchy.Num() > 0)
		{
			return StreamHierarchy[0]->Name;
		}
		if (I == 0)
		{
			TSharedRef<FSCCommand> GetStreamResult = GetStream().GetResult();
			ParseStreamCommand(*GetStreamResult);
		}
	}
	return FString();
}

const FString FSubmitToolPerforce::GetCurrentStreamName()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FSubmitToolPerforce::GetCurrentStreamName);
	WaitForInitialStreamTask();
	for (int32 I = 0; I < 2; ++I)
	{
		if (StreamHierarchy.Num() > 0)
		{
			return StreamHierarchy.Last()->Name;
		}
		if (I == 0)
		{
			TSharedRef<FSCCommand> GetStreamResult = GetStream().GetResult();
			ParseStreamCommand(*GetStreamResult);
		}
	}
	return FString();
}

int32 FSubmitToolPerforce::GetDepotStreamDepth(const FString& InDepotName)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FSubmitToolPerforce::GetDepotStreamDepth);
	for (int32 I = 0; I < 2; ++I)
	{
		if (int32* StreamDepth = DepotStreamDepths.Find(InDepotName))
		{
			return *StreamDepth;
		}
		if (I == 0)
		{
			TSharedRef<FSCCommand> Cmd = RunCommand(TEXT("depot"), { TEXT("-o"), InDepotName }).GetResult();

			if (Cmd->bSuccess && Cmd->Values.Num() > 0)
			{
				if (const FString* StreamDepth = Cmd->Values[0].Find(TEXT("StreamDepth")))
				{
					int32 Depth = 0;
					for (int32 j = 2; j < StreamDepth->Len(); ++j)
					{
						if ((*StreamDepth)[j] == TCHAR('/'))
						{
							Depth++;
						}
					}

					DepotStreamDepths.Add(InDepotName, Depth);
				}
			}
		}
	}
	return 0;
}

const FAuthTicket& FSubmitToolPerforce::GetAuthTicket()
{
	if(!P4Ticket.IsValid())
	{
		TUniquePtr<FP4Connection> Connection = Connections.GetAvailableConnection();
		const StrPtr& Username = Connection->GetConnection().GetUser();
		const StrPtr& Ticket = Connection->GetConnection().GetPassword();
		P4Ticket = FAuthTicket(TO_TCHAR(Username.Text(), bIsUnicodeServer), TO_TCHAR(Ticket.Text(), bIsUnicodeServer));
	}

	return P4Ticket;
}

void FSubmitToolPerforce::RequestCancel()
{
	Connections.CancelCurrentRequests();
}

bool FSubmitToolPerforce::IsBusy() const
{
	return ActiveCommands != 0;
}

bool FSubmitToolPerforce::RunP4Command(FSCCommand& Command, EP4ClientUserFlags CmdFlags)
{
	TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("FSubmitToolPerforce::RunP4Command_%s"), *Command.Command));

	CmdFlags |= bIsUnicodeServer ? EP4ClientUserFlags::UnicodeServer : EP4ClientUserFlags::None;

	TUniquePtr<FP4Connection> Connection = Connections.GetAvailableConnection();

	size_t Attempts = 0;
	while(!Connection.IsValid())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(WaitForConnection);
		if(Attempts >= MaxConnectionsAttempts)
		{
			return false;
		}

		FPlatformProcess::Sleep(ConnectionAttemptWaitTime);
		Connection = Connections.GetAvailableConnection();

		++Attempts;
	}
	ActiveCommands++;

	ClientApi& P4Client = Connection->GetConnection();
	TStringBuilder<128> ShortCommand;
	TStringBuilder<512> FullCommand;
	ShortCommand.Append(TEXT("p4 "));
	FullCommand.Append(TEXT("p4 -p "));
	FullCommand.Append(TO_TCHAR(P4Client.GetPort().Text(), bIsUnicodeServer));

	if(EnumHasAnyFlags(CmdFlags, EP4ClientUserFlags::UseUser))
	{
		FString UserName;
		FCmdLineParameters::Get().GetValue(FSubmitToolCmdLine::P4User, UserName);
		P4Client.SetUser(TCHAR_TO_ANSI(*UserName));

		FullCommand.Append(TEXT(" -u ") + UserName);
	}

	if(EnumHasAnyFlags(CmdFlags, EP4ClientUserFlags::UseClient))
	{
		FString Client;
		FCmdLineParameters::Get().GetValue(FSubmitToolCmdLine::P4Client, Client);
		P4Client.SetClient(TCHAR_TO_ANSI(*Client));

		FullCommand.Append(TEXT(" -c ") + Client);
	}

	if(EnumHasAnyFlags(CmdFlags, EP4ClientUserFlags::UseZTag))
	{
		P4Client.SetProtocol("tag", "");
		FullCommand.Append(TEXT(" -ztag "));
	}

	ShortCommand.Append(Command.Command);
	FullCommand.Append(Command.Command);

	int32 ArgC = Command.Args.Num();
	UTF8CHAR** ArgV = new UTF8CHAR * [ArgC];
	for(int32 Index = 0; Index < ArgC; Index++)
	{
		if(bIsUnicodeServer)
		{
			FTCHARToUTF8 UTF8String(*Command.Args[Index]);
			ArgV[Index] = new UTF8CHAR[UTF8String.Length() + 1];
			FMemory::Memcpy(ArgV[Index], UTF8String.Get(), UTF8String.Length() + 1);
		}
		else
		{
			ArgV[Index] = new UTF8CHAR[Command.Args[Index].Len() + 1];
			FMemory::Memcpy(ArgV[Index], TCHAR_TO_ANSI(*Command.Args[Index]), Command.Args[Index].Len() + 1);
		}

		ShortCommand.Append(TEXT(" "));
		ShortCommand.Append(Command.Args[Index]);
		FullCommand.Append(TEXT(" "));
		FullCommand.Append(Command.Args[Index]);
	}

	P4Client.SetArgv(ArgC, (char**)ArgV);
	FSTClientUser P4User(Command, CmdFlags);

	UE_LOGF(LogSubmitToolP4Debug, Log, "Running command: %ls", *FullCommand);
	P4Client.Run(FROM_TCHAR(*Command.Command, bIsUnicodeServer), &P4User);

	if (P4Client.Dropped())
	{
		UE_LOGF(LogSubmitToolP4Debug, Log, "P4 connection dropped: %ls", *Command.Command);
		Command.bDroppedConnection = true;
	}

	for(const FText& Msg : Command.Results.InfoMessages)
	{
		UE_LOGF(LogSubmitToolP4Debug, Log, "%ls: %ls", *FullCommand, *Msg.ToString());
	}

	if(Command.Results.HasErrors())
	{
		for(const FText& Error : Command.Results.ErrorMessages)
		{
			if (EnumHasAnyFlags(CmdFlags, EP4ClientUserFlags::SilentErrors))
			{
				UE_LOGF(LogSubmitToolP4Debug, Error, "%ls: %ls", *FullCommand, *Error.ToString());
			}
			else
			{
				UE_LOGF(LogSubmitToolP4Debug, Error, "%ls: %ls", *FullCommand, *Error.ToString());
				UE_LOGF(LogSubmitToolP4, Error, "%ls: %ls", *ShortCommand, *Error.ToString());
			}
		}
	}

	Command.Data = P4User.ReleaseData();

	// Free arguments
	for(int32 Index = 0; Index < ArgC; Index++)
	{
		delete[] ArgV[Index];
	}
	delete[] ArgV;


	ActiveCommands--;
	return !Command.Results.HasErrors() && !Command.bDroppedConnection;
}
