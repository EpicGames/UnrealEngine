// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Logic/Services/Interfaces/ISTSourceControlService.h"
#include "PerforceData.h"
#include "Containers/Ticker.h"
#include "Containers/LockFreeFixedSizeAllocator.h"

class FProcessWrapper;
class ISourceControlProvider;

DECLARE_DELEGATE_RetVal(bool, FOnIsCancelled);

class FSTP4KeepAlive : public KeepAlive
{
public:
	FSTP4KeepAlive(FOnIsCancelled InIsCancelled)
		: IsCancelled(InIsCancelled)
	{}

	/** Called when the perforce connection wants to know if it should stay connected */
	virtual int IsAlive() override
	{
		if (IsCancelled.IsBound() && IsCancelled.Execute())
		{
			return 0;
		}

		return 1;
	}

	FOnIsCancelled IsCancelled;
};

struct FClientApiWrapper
{
	FClientApiWrapper(TUniquePtr<ClientApi>&& InConnection)
	: bIsReady(true), KeepAlive(FOnIsCancelled::CreateLambda([this] { return bWantsCancel.load(); })), Connection(MoveTemp(InConnection))
	{
		Connection->SetBreak(&KeepAlive);
	}

	std::atomic<bool> bIsReady = true;
	FSTP4KeepAlive KeepAlive;
	TUniquePtr<ClientApi> Connection;
	std::atomic<bool> bWantsCancel = false;
};

struct FP4Connection
{
public:
	FP4Connection(FClientApiWrapper& InConnection);
	~FP4Connection();

	ClientApi& GetConnection() const { return *Connection.Connection; }

private:
	FClientApiWrapper& Connection;
};

class FConnectionPool
{
public:
	FConnectionPool() = default;
	~FConnectionPool();
	TUniquePtr<FP4Connection> GetAvailableConnection();	
	void CancelCurrentRequests();
private:
	int32 CreateConnection();
	std::atomic<bool> bConnectionFailed = false;
	TArray<TUniquePtr<FClientApiWrapper>> P4Connections;
	FCriticalSection Mutex;
};

class FSubmitToolPerforce final : public ISTSourceControlService
{
public:
	FSubmitToolPerforce();
	~FSubmitToolPerforce();


	virtual void InitializeParameters(const FSubmitToolParameters& InParameters);
	virtual void GetUsers(const FOnUsersGet::FDelegate& Callback) override;
	virtual void GetGroups(const FOnGroupsGet::FDelegate& Callback) override;
	virtual void GetUsersAndGroups(const FOnUsersAndGroupsGet::FDelegate& Callback) override;

	virtual UE::Tasks::TTask<bool> DownloadFiles(TArray<FString>&& FilePaths, TMap<FString, FSharedBuffer>& OutFileBuffers, bool bSilentErrors = false) override;

	virtual const TArray<TSharedPtr<FUserData>>& GetRecentUsers() const override;
	virtual void AddRecentUser(TSharedPtr<FUserData>& User) override;
	virtual const TArray<TSharedPtr<FString>>& GetRecentGroups() const override;
	virtual void AddRecentGroup(TSharedPtr<FString>& Group) override;

	virtual TSharedPtr<FUserData> GetUserDataFromCache(const FString& Username) const override;

	virtual const TArray<FSCCStream*>& GetClientStreams() const override;
	virtual const FSCCStream* GetSCCStream(const FString& InStreamName) override;
	virtual const FString GetRootStreamName() override;
	virtual const FString GetCurrentStreamName() override;
	virtual int32 GetDepotStreamDepth(const FString& InDepotName) override;
	virtual const FAuthTicket& GetAuthTicket() override;
	virtual void RequestCancel() override;
	virtual bool IsBusy() const override;
			
protected:
	virtual bool RunCommandInternal(FSCCommand& Command) override;

private:
	bool RunP4Command(FSCCommand& Command, EP4ClientUserFlags CmdFlags = DefaultFlags);

	UE::Tasks::TTask<TSharedRef<FSCCommand>> GetStream(const FString& InStream = FString());
	void ParseStreamCommand(const FSCCommand& InCompletedCommand, const FString& InRequestedStream = FString(), bool bParseHierarchy = true);
	void WaitForInitialStreamTask();

	TArray<FString> GroupsToExclude;
	TMap<FString, TSharedPtr<FUserData>> CachedUsers;
	TArray<TSharedPtr<FUserData>> CachedUsersArray;
	TArray<TSharedPtr<FString>> CachedGroupsArray;
	UE::Tasks::TTask<TSharedRef<FSCCommand>> InitialStreamTask;
	bool bUserTaskIsRunning = false;
	std::atomic<bool> bGroupTaskIsRunning = false;
	bool bIsUnicodeServer = false;
	bool bUsersReady = false;
	bool bGroupsReady = false;
	FOnUsersGet OnUsersGetCallbacks;
	FOnGroupsGet OnGroupsGetCallbacks;
	FOnUsersAndGroupsGet OnUsersAndGroupsGetCallbacks;
	std::atomic<uint64> ActiveCommands = 0;
	
	FTSTicker::FDelegateHandle TickHandle;

	void LoadRecentUsers();
	void SaveRecentUsers();
	const FString GetRecentUsersFilepath() const;
	TArray<TSharedPtr<FUserData>> RecentUsers;
	void LoadRecentGroups();
	void SaveRecentGroups();
	const FString GetRecentGroupsFilepath() const;
	TArray<TSharedPtr<FString>> RecentGroups;

	TMap<FString, TUniquePtr<FSCCStream>> Streams;
	TMap<FString, int32> DepotStreamDepths;

	TArray<FSCCStream*> StreamHierarchy;
	FAuthTicket P4Ticket;
	
	FConnectionPool Connections;

	static constexpr EP4ClientUserFlags DefaultFlags = EP4ClientUserFlags::UseClient | EP4ClientUserFlags::UseUser | EP4ClientUserFlags::UseZTag;
};
