// Copyright Epic Games, Inc. All Rights Reserved.

#include "SwarmService.h"

#include "Interfaces/IHttpResponse.h"
#include "Logic/Services/SubmitToolServiceProvider.h"
#include "Logic/Services/Interfaces/ISTSourceControlService.h"
#include "Logic/Services/Interfaces/ITagService.h"
#include "Logic/Services/Interfaces/IChangelistService.h"
#include "Serialization/JsonSerializer.h"
#include "JsonObjectConverter.h"

#include "Logging/SubmitToolLog.h"

FSwarmService::FSwarmService(TWeakPtr<FSubmitToolServiceProvider> InServiceProvider) :
	ServiceProvider(InServiceProvider)
{
	GetSwarmURLTask = ServiceProvider.Pin()->GetService<ISTSourceControlService>()->RunCommand(TEXT("property"), { "-l", "-n", "P4.Swarm.URL" });
	ServiceProvider.Pin()->GetService<ISTSourceControlService>()->GetUsersAndGroups(FOnUsersAndGroupsGet::FDelegate::CreateLambda([this](TArray<TSharedPtr<FUserData>>& InUsers, TArray<TSharedPtr<FString>>& InGroups) {
		Users = &InUsers;
		Groups = &InGroups;
	}));
}

void FSwarmService::FetchReview(const OnGetReviewComplete& OnComplete)
{
	FString Changelist = ServiceProvider.Pin()->GetService<IChangelistService>()->GetCLID();

	if (Changelist.IsEmpty())
	{
		return;
	}

	const TSharedRef<IHttpRequest> HttpRequest = FHttpModule::Get().CreateRequest();
	HttpRequest->SetHeader(TEXT("Content-Type"), TEXT("application/json; charset=utf-8"));
	HttpRequest->SetHeader(TEXT("Authorization"), *ServiceProvider.Pin()->GetService<ISTSourceControlService>()->GetAuthTicket().ToString());

	HttpRequest->SetURL(FString::Format(TEXT("{0}?change={1}&max={2}"),
		{
			ReviewsURL(),
			Changelist, // get the review for a specific CL
			1 // we only want a single review.
		}));
	HttpRequest->SetVerb(TEXT("GET"));

	HttpRequest->OnProcessRequestComplete().BindLambda([OnComplete, this](FHttpRequestPtr Request, FHttpResponsePtr Response, bool bConnectedSuccessfully)
		{
			if (!bConnectedSuccessfully || !Response.IsValid())
			{
				OnComplete.ExecuteIfBound(Review, TEXT("Connection Failed"));
				return;
			}

			UE_LOGF(LogSubmitToolDebug, Log, "Fetch review Response: %ls", *Response->GetContentAsString());

			if(EHttpResponseCodes::IsOk(Response->GetResponseCode()))
			{
				FSwarmReviewCollection ReviewCollection;
				if(FJsonObjectConverter::JsonObjectStringToUStruct<FSwarmReviewCollection>(Response->GetContentAsString(), &ReviewCollection, 0, 0))
				{
					if(ReviewCollection.Reviews.Num() > 0)
					{
						Review = MakeUnique<FSwarmReview>(ReviewCollection.Reviews[0]);
						OnComplete.ExecuteIfBound(Review, {});
					}
					else
					{
						OnComplete.ExecuteIfBound(Review, TEXT("No available reviews."));
					}
				}
				else
				{
					OnComplete.ExecuteIfBound(Review, TEXT("Could not parse the response json."));
				}
			}
			else
			{
				UE_LOGF(LogSubmitTool, Error, "Could not communicate with swarm due to error %d.\n%ls", Response->GetResponseCode(), *Response->GetContentAsString());
				OnComplete.ExecuteIfBound(Review, FString::Printf(TEXT("Error code %d."), Response->GetResponseCode()));
			}
		});

	HttpRequest->ProcessRequest();
}

void FSwarmService::CreateReview(const TArray<FString>& InReviewers, const OnCreateReviewComplete& OnComplete)
{
	FString Changelist = ServiceProvider.Pin()->GetService<IChangelistService>()->GetCLID();

	if (Changelist.IsEmpty())
	{
		OnComplete.ExecuteIfBound(false, FString());
		return;
	}

	if(CreateSwarmRequest.IsValid())
	{
		CreateSwarmRequest->CancelRequest();
	}

	CreateSwarmRequest = FHttpModule::Get().CreateRequest();
	CreateSwarmRequest->SetHeader(TEXT("Content-Type"), TEXT("application/json; charset=utf-8"));
	CreateSwarmRequest->SetHeader(TEXT("Authorization"), *ServiceProvider.Pin()->GetService<ISTSourceControlService>()->GetAuthTicket().ToString());

	CreateSwarmRequest->SetURL(ReviewsURL());
	CreateSwarmRequest->SetVerb(TEXT("POST"));

	TSharedRef<FJsonObject> RequestJson = MakeShared<FJsonObject>();
	RequestJson->SetNumberField(TEXT("change"), FCString::Atoi(*Changelist));

	TArray<TSharedPtr<FJsonValue>> ReviewersObject;
	TArray<TSharedPtr<FJsonValue>> GroupsObject;

	for (const FString& value : InReviewers)
	{
		if (Users != nullptr)
		{
			TSharedPtr<FUserData>* FoundUser = Users->FindByPredicate([value = value.TrimChar(TCHAR('@'))](TSharedPtr<FUserData>& InUserData) { return InUserData->Username == value; });
			if (FoundUser != nullptr)
			{
				ReviewersObject.Add(MakeShared<FJsonValueString>((*FoundUser)->Username));
				continue;
			}
		}

		if (Groups != nullptr)
		{
			TSharedPtr<FString>* FoundGroup = Groups->FindByPredicate([value = value.TrimChar(TCHAR('@'))](TSharedPtr<FString>& InGroupData) { return *InGroupData == value; });
			if (FoundGroup != nullptr)
			{
				TSharedPtr<FJsonObject> GroupObject = MakeShared<FJsonObject>();
				GroupObject->SetStringField(TEXT("name"), *(*FoundGroup));
				GroupsObject.Add(MakeShared<FJsonValueObject>(GroupObject));
			}
		}
	}

	if (!ReviewersObject.IsEmpty())
	{
		RequestJson->SetArrayField(TEXT("reviewers"), ReviewersObject);
	}

	if(!GroupsObject.IsEmpty())
	{
		RequestJson->SetArrayField(TEXT("reviewerGroups"), GroupsObject);
	}

	RequestJson->SetStringField(TEXT("description"), ServiceProvider.Pin()->GetService<IChangelistService>()->GetCLDescription());

	FString BodyString;
	TSharedRef<TJsonWriter<TCHAR>> JsonWriter = TJsonWriterFactory<>::Create(&BodyString);
	FJsonSerializer::Serialize(RequestJson, JsonWriter);
	UE_LOGF(LogSubmitToolDebug, Log, "Create Swarm request body:\n%ls", *BodyString);
	CreateSwarmRequest->SetContentAsString(BodyString);

	CreateSwarmRequest->OnProcessRequestComplete().BindLambda([OnComplete, this](FHttpRequestPtr Request, FHttpResponsePtr HttpResponse, bool bConnectedSuccessfully)
		{
			if(!bConnectedSuccessfully)
			{
				if(HttpResponse.IsValid())
				{
					UE_LOGF(LogSubmitTool, Warning, "Unable to create swarm review. Connection error %d - %ls.", HttpResponse->GetResponseCode(), *EHttpResponseCodes::GetDescription(static_cast<EHttpResponseCodes::Type>(HttpResponse->GetResponseCode())).ToString());
					UE_LOGF(LogSubmitToolDebug, Warning, "Unable to create swarm review. Connection error\nResponse: %ls", *HttpResponse->GetContentAsString());
				}
				else
				{
					UE_LOGF(LogSubmitTool, Warning, "Unable to create swarm review. Connection error, no response.");
				}
				OnComplete.ExecuteIfBound(false, FString());
				return;
			}
			if(HttpResponse.IsValid())
			{
				UE_LOGF(LogSubmitToolDebug, Log, "Create review Response: %ls", *HttpResponse->GetContentAsString());
				if(EHttpResponseCodes::IsOk(HttpResponse->GetResponseCode()))
				{
					TSharedRef<TJsonReader<TCHAR>> Reader = TJsonReaderFactory<TCHAR>::Create(HttpResponse->GetContentAsString());
					TSharedPtr<FJsonObject> JsonObj;
					if(!FJsonSerializer::Deserialize(Reader, JsonObj))
					{
						UE_LOGF(LogSubmitTool, Error, "Unable to deserialize swarm create response");
						return;
					}

					FSwarmReview ReviewObj;
					if (FJsonObjectConverter::JsonObjectToUStruct<FSwarmReview>(JsonObj->GetObjectField(TEXT("review")).ToSharedRef(), &ReviewObj))
					{
						Review = MakeUnique<FSwarmReview>(ReviewObj);
					}
					FString ReviewId = JsonObj->GetObjectField(TEXT("review"))->GetStringField(TEXT("id"));
					FString ReviewURL = BuildReviewURL(ReviewId);
					OnComplete.ExecuteIfBound(true, ReviewURL);
				}
				else
				{
					UE_LOGF(LogSubmitTool, Error, "Could not create a swarm review due to error %d - %ls.", HttpResponse->GetResponseCode(), *EHttpResponseCodes::GetDescription(static_cast<EHttpResponseCodes::Type>(HttpResponse->GetResponseCode())).ToString());
					OnComplete.ExecuteIfBound(false, FString());
				}
			}
			else
			{
				UE_LOGF(LogSubmitTool, Warning, "Unable to create swarm review. Failed with unknown connection error");
				OnComplete.ExecuteIfBound(false, FString());
			}
		});

	UE_LOGF(LogSubmitToolP4, Log, "Creating swarm review");
	CreateSwarmRequest->ProcessRequest();
}


void FSwarmService::UpdateReviewDescription(const TDelegate<void(bool)>& OnComplete, const FString& InDescription)
{
	if (!Review.IsValid())
	{
		UE_LOGF(LogSubmitTool, Warning, "Tried to update swarm review but Swarm API is not available or there is no review for this CL");
		OnComplete.ExecuteIfBound(false);
		return;
	}

	if(UpdateSwarmRequest.IsValid())
	{
		UpdateSwarmRequest->CancelRequest();
	}

	UpdateSwarmRequest = FHttpModule::Get().CreateRequest();
	UpdateSwarmRequest->SetHeader(TEXT("Content-Type"), TEXT("application/json; charset=utf-8"));
	UpdateSwarmRequest->SetHeader(TEXT("Authorization"), *ServiceProvider.Pin()->GetService<ISTSourceControlService>()->GetAuthTicket().ToString());

	FString URL = ReviewsURL() / FString::FromInt(Review->Id);
	UpdateSwarmRequest->SetURL(URL);
	UpdateSwarmRequest->SetVerb(TEXT("PATCH"));

	TSharedRef<FJsonObject> RequestJson = MakeShared<FJsonObject>();
	RequestJson->SetStringField(TEXT("description"), InDescription);

	FString BodyString;
	TSharedRef<TJsonWriter<TCHAR>> JsonWriter = TJsonWriterFactory<>::Create(&BodyString);
	FJsonSerializer::Serialize(RequestJson, JsonWriter);
	UE_LOGF(LogSubmitToolDebug, Log, "Update Swarm request body:\n%ls", *BodyString);
	UpdateSwarmRequest->SetContentAsString(BodyString);

	UpdateSwarmRequest->OnProcessRequestComplete().BindLambda([OnComplete, this](FHttpRequestPtr Request, FHttpResponsePtr HttpResponse, bool bConnectedSuccessfully)
		{
			if(!bConnectedSuccessfully || !HttpResponse.IsValid())
			{
				if(HttpResponse.IsValid())
				{
					UE_LOGF(LogSubmitTool, Warning, "Unable to update swarm review. Connection error %d - %ls.", HttpResponse->GetResponseCode(), *EHttpResponseCodes::GetDescription(static_cast<EHttpResponseCodes::Type>(HttpResponse->GetResponseCode())).ToString());
					UE_LOGF(LogSubmitToolDebug, Warning, "Unable to update swarm review. Connection error %d\nResponse: %ls", HttpResponse->GetResponseCode(), *HttpResponse->GetContentAsString());
				}
				else
				{
					UE_LOGF(LogSubmitTool, Warning, "Unable to update swarm review. Connection error, no response.");
				}

				OnComplete.ExecuteIfBound(false);
				return;
			}
			else
			{
				UE_LOGF(LogSubmitToolDebug, Log, "Update review response: %ls", *HttpResponse->GetContentAsString());
				if(EHttpResponseCodes::IsOk(HttpResponse->GetResponseCode()))
				{
					UE_LOGF(LogSubmitTool, Log, "Swarm description updated successfully");
					OnComplete.ExecuteIfBound(true);
				}
				else
				{
					UE_LOGF(LogSubmitTool, Error, "Could not update swarm description due to error %d - %ls.", HttpResponse->GetResponseCode(), *EHttpResponseCodes::GetDescription(static_cast<EHttpResponseCodes::Type>(HttpResponse->GetResponseCode())).ToString());
					OnComplete.ExecuteIfBound(false);
				}
			}
		});

	UE_LOGF(LogSubmitToolP4, Log, "Updating swarm review description");
	UpdateSwarmRequest->ProcessRequest();

}

FString FSwarmService::ReviewsURL()
{
	if (GetSwarmURLTask.IsValid())
	{
		TSharedRef<FSCCommand> Cmd = GetSwarmURLTask.GetResult();
		if (Cmd->bSuccess && Cmd->Values.Num() > 0)
		{
			if (const FString* Value = Cmd->Values[0].Find(TEXT("value")))
			{
				SwarmURL = *Value;
			}
		}
		GetSwarmURLTask = {};
	}
	return SwarmURL.IsEmpty() ? FString{} : SwarmURL / TEXT("api/v9/reviews");
}

const TArray<FString> FSwarmService::GetUsersInSwarmTag() const
{
	TArray<FString> Reviewers;
	
	for (const FTag* Tag : ServiceProvider.Pin()->GetService<ITagService>()->GetTagsArray())
	{
		if (Tag->Definition.InputSubType.Equals(TEXT("Swarm"), ESearchCase::IgnoreCase))
		{
			return Tag->GetValues();
		}
	}
	return TArray<FString>();
}
