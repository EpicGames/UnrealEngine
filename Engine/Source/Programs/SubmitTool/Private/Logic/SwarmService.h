// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Models/SwarmReview.h"
#include "Logic/Services/Interfaces/ISubmitToolService.h"
#include "Logic/Services/Interfaces/ISTSourceControlService.h"

#include "HttpModule.h"

class FSubmitToolServiceProvider;

DECLARE_DELEGATE_TwoParams(OnGetReviewComplete, const TUniquePtr<FSwarmReview>&, const FString&)
DECLARE_DELEGATE_TwoParams(OnCreateReviewComplete, bool, const FString&)

class FSwarmService : public ISubmitToolService
{
public:
	FSwarmService(TWeakPtr<FSubmitToolServiceProvider> InServiceProvider);
	void FetchReview(const OnGetReviewComplete& OnComplete);

	void CreateReview(const TArray<FString>& InReviewers, const OnCreateReviewComplete& OnComplete);

	void UpdateReviewDescription(const TDelegate<void(bool)>& OnComplete, const FString& InDescription);


	const TUniquePtr<FSwarmReview>& GetReview() const { return Review; }
	bool GetCurrentReviewUrl(FString& OutUrl) const
	{
		if(Review.IsValid())
		{
			OutUrl = BuildReviewURL(FString::FromInt(Review->Id));
		}

		return Review.IsValid();
	}

	FString BuildReviewURL(const FString& InReviewId) const
	{
		return FString::Printf(TEXT("%s/reviews/%s"), *SwarmURL, *InReviewId);
	}

	void CancelOperations()
	{
		if(CreateSwarmRequest.IsValid())
		{
			CreateSwarmRequest->CancelRequest();
		}
	}

	bool IsRequestRunning() const
	{
		return (CreateSwarmRequest.IsValid() && CreateSwarmRequest->GetStatus() == EHttpRequestStatus::Processing)
			|| (UpdateSwarmRequest.IsValid() && UpdateSwarmRequest->GetStatus() == EHttpRequestStatus::Processing);
	}

	const TArray<FString> GetUsersInSwarmTag() const;

private:
	FString ReviewsURL();

private:
	TSharedPtr<IHttpRequest> CreateSwarmRequest;
	TSharedPtr<IHttpRequest> UpdateSwarmRequest;

	TWeakPtr<FSubmitToolServiceProvider> ServiceProvider;
	UE::Tasks::TTask<TSharedRef<FSCCommand>> GetSwarmURLTask;
	FString SwarmURL;
	TUniquePtr<FSwarmReview> Review = nullptr;

	TArray<TSharedPtr<FUserData>>* Users = nullptr;
	TArray<TSharedPtr<FString>>* Groups = nullptr;
};

Expose_TNameOf(FSwarmService);
