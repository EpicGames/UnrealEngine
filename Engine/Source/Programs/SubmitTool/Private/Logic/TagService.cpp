// Copyright Epic Games, Inc. All Rights Reserved.

#include "TagService.h"
#include "Models/ModelInterface.h"
#include "Logging/SubmitToolLog.h"


FTagService::FTagService(const FSubmitToolParameters& InParameters, TSharedPtr<IChangelistService> CLService) :
	Parameters(InParameters),
	ChangelistService(CLService)
{
	RegisterTags();
}

void FTagService::RegisterTags()
{
	for(const FTagDefinition& Tag : Parameters.AvailableTags)
	{
		FTag TagObj = FTag(Tag);
		TagObj.OnTagUpdated.AddLambda([this](const FTag& tag) {
			if(OnTagUpdated.IsBound())
			{
				OnTagUpdated.Broadcast(tag);
			}});

		RegisteredTags.Add(Tag.GetTagId(), TagObj);
	}
}

void FTagService::ParseCLDescription()
{
	for(TPair<FString, FTag>& kvp : RegisteredTags)
	{
		kvp.Value.ParseTag(GetCLDescription());
	}
}

void FTagService::ReApplyTags()
{
	TArray<FTag*> TagsToReApply;

	for (TPair<FString, FTag>& TagPair : RegisteredTags)
	{
		if (TagPair.Value.StartPos != std::numeric_limits<size_t>::max() && !GetCLDescription().Contains(TagPair.Value.Definition.TagId))
		{
			TagPair.Value.StartPos = std::numeric_limits<size_t>::max();
			TagPair.Value.LastSize = std::numeric_limits<size_t>::max();
			TagPair.Value.SetTagState(ETagState::Unchecked);
			TagsToReApply.Add(&TagPair.Value);
		}
	}

	for (FTag* Tag : TagsToReApply)
	{
		ApplyTag(*Tag);
	}

	// Parse to ensure we're matching the current description
	ParseCLDescription();
}

void FTagService::ApplyTag(const FString& tagID)
{
	FTag* tag = GetTag(tagID);
	if(tag != nullptr)
	{
		ApplyTag(*tag);
	}
}

void FTagService::ApplyTag(FTag& Tag)
{
	UE_LOGF(LogSubmitToolDebug, Log, "Trying to apply tag: %ls", *Tag.Definition.GetTagId());

	if(Tag.StartPos != std::numeric_limits<size_t>::max())
	{
		UE_LOGF(LogSubmitTool, Error, "Tried to apply an already applied Tag: %ls. Something has gone wrong.", *Tag.Definition.GetTagId());
	}
	else
	{
		FString& CLDescription = GetCLDescription();
		CLDescription.TrimEndInline();

		bool bFirstTag = true;
		for(const TPair<FString, FTag>& Pair : RegisteredTags)
		{
			if(Pair.Value.IsEnabled())
			{
				bFirstTag = false;
				break;
			}
		}

		if(bFirstTag)
		{
			CLDescription.AppendChar(TCHAR('\n'));
		}

		Tag.StartPos = CLDescription.Len();
		const FString& FullTag = Tag.GetFullTag();
		Tag.bIsDirty = false;

		UE_LOGF(LogSubmitToolDebug, Log, "Tag applied: %ls", *FullTag.TrimChar(TEXT('\n')));

		CLDescription.Append(FullTag);
		Tag.LastSize = CLDescription.Len() - Tag.StartPos;

		if(OnTagUpdated.IsBound())
		{
			OnTagUpdated.Broadcast(Tag);
		}
	}
}

void FTagService::UpdateTagsInCL()
{
	for(TPair<FString, FTag>& Tag : RegisteredTags)
	{
		if(Tag.Value.bIsDirty)
		{
			FString& CLDescription = GetCLDescription();
			size_t PreviousSize = Tag.Value.LastSize;

			const FString& NewTag = Tag.Value.GetFullTag();

			CLDescription.RemoveAt(Tag.Value.StartPos, PreviousSize);
			CLDescription.InsertAt(Tag.Value.StartPos, NewTag);

			Tag.Value.LastSize = NewTag.Len();
			Tag.Value.bIsDirty = false;

			UpdateTagsPositions(Tag.Value.StartPos, NewTag.Len() - PreviousSize);
		}
	}
}

void FTagService::RemoveTag(const FString& tagID)
{
	FTag* tag = GetTag(tagID);
	if(tag != nullptr)
	{
		RemoveTag(*tag);
	}
}

void FTagService::RemoveTag(FTag& Tag)
{
	if(Tag.IsEnabled())
	{
		UE_LOGF(LogSubmitToolDebug, Log, "Removing tag: %ls", *Tag.Definition.GetTagId());

		GetCLDescription().RemoveAt(Tag.StartPos, Tag.LastSize);
		UpdateTagsPositions(Tag.StartPos, -static_cast<int32>(Tag.LastSize));

		Tag.bIsDirty = false;
		Tag.StartPos = std::numeric_limits<size_t>::max();
		Tag.LastSize = std::numeric_limits<size_t>::max();
		Tag.SetTagState(ETagState::Unchecked);

		if(OnTagUpdated.IsBound())
		{
			OnTagUpdated.Broadcast(Tag);
		}
	}
}

void FTagService::SetTagValues(const FString& TagID, const FString& Values)
{
	FTag* tag = GetTag(TagID);
	if(tag != nullptr)
	{
		SetTagValues(*tag, Values);
	}
}

void FTagService::SetTagValues(FTag& tag, const FString& values)
{
	tag.SetValues(values);

	if(!tag.IsEnabled())
	{
		ApplyTag(tag);
	}
	else
	{
		UpdateTagsInCL();
	}
}

void FTagService::SetTagValues(FTag& tag, const TArray<FString>& values)
{
	tag.SetValues(values);

	if(!tag.IsEnabled())
	{
		ApplyTag(tag);
	}
	else
	{
		UpdateTagsInCL();
	}
}

void FTagService::UpdateTagsPositions(size_t ChangePos, int32 Delta)
{
	if(Delta != 0)
	{
		for(TPair<FString, FTag>& tag : RegisteredTags)
		{
			if(tag.Value.StartPos != std::numeric_limits<size_t>::max() && tag.Value.StartPos > ChangePos)
			{
				tag.Value.StartPos += Delta;
			}
		}
	}
}

const TArray<const FTag*>& FTagService::GetTagsArray() const
{
	if(CachedTags.Num() == 0)
	{
		for(const TPair<FString, FTag>& tag : RegisteredTags)
		{
			CachedTags.Add(&tag.Value);
		}
	}

	return CachedTags;
}
