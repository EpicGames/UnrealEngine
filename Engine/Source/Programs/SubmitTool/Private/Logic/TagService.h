// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Models/Tag.h"
#include "Parameters/SubmitToolParameters.h"
#include "Logic/Services/Interfaces/IChangelistService.h"
#include "Logic/Services/Interfaces/ITagService.h"

class FTagService : public ITagService
{
public:
	FTagService(const FSubmitToolParameters& InParameters, TSharedPtr<IChangelistService> clService);

	virtual void ParseCLDescription() override;
	virtual void ReApplyTags() override;

	virtual void ApplyTag(const FString& tagID) override;
	virtual void ApplyTag(FTag& tag) override;
	virtual void UpdateTagsInCL() override;
	virtual void RemoveTag(const FString& tagID) override;
	virtual void RemoveTag(FTag& tag) override;
	virtual void SetTagValues(const FString& tagID, const FString& values) override;
	virtual void SetTagValues(FTag& tag, const FString& values) override;
	virtual void SetTagValues(FTag& tag, const TArray<FString>& values) override;

	virtual FTag* GetTag(const FString& tagID) override
	{
		if(RegisteredTags.Contains(tagID))
		{
			return &RegisteredTags[tagID];
		}

		return nullptr;
	};

	virtual FTag* GetTagOfType(const FString& InType) override
	{
		for (const TPair<FString, FTag>& Tag : RegisteredTags)
		{
			if (Tag.Value.Definition.InputType.Equals(InType, ESearchCase::IgnoreCase))
			{
				return &RegisteredTags[Tag.Key];
			}
		}

		return nullptr;
	}

	virtual FTag* GetTagOfSubtype(const FString& InTagSubtype) override
	{
		for (const TPair<FString, FTag>& Tag : RegisteredTags)
		{
			if (Tag.Value.Definition.InputSubType.Equals(InTagSubtype, ESearchCase::IgnoreCase))
			{
				return &RegisteredTags[Tag.Key];
			}
		}

		return nullptr;
	}

	virtual const TArray<const FTag*>& GetTagsArray() const override;

private:
	FString& GetCLDescription() { return const_cast<FString&>(ChangelistService->GetCLDescription()); }

	mutable TArray<const FTag*> CachedTags;

	const FSubmitToolParameters& Parameters;
	TMap<FString, FTag> RegisteredTags;
	TSharedPtr<IChangelistService> ChangelistService;

	void RegisterTags();
	void UpdateTagsPositions(size_t changePos, int32 delta);
};

Expose_TNameOf(FTagService);