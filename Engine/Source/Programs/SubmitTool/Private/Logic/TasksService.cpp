// Copyright Epic Games, Inc. All Rights Reserved.

#include "TasksService.h"


#include "Logging/SubmitToolLog.h"
#include "Models/ModelInterface.h"
#include "Telemetry/TelemetryService.h"
#include "AnalyticsEventAttribute.h"
#include "Policies/CondensedJsonPrintPolicy.h"
#include "Serialization/JsonWriter.h"



FTasksService::FTasksService(const TMap<FString, FString>& InTasks, const FString& InTelemetryEventsId) :
	bLastTasksRunState(false),
	bLastRunningTasks(false),
	TelemetryBaseId(InTelemetryEventsId)
{
}

FTasksService::~FTasksService()
{
	FTSTicker::RemoveTicker(TickerHandle);
	OnTasksRunResultUpdated.Clear();
	OnSingleTaskFinished.Clear();
	OnTasksQueueFinished.Clear();

	StopTasks(); 
}

void FTasksService::InitializeTasks(const TArray<TSharedRef<FValidatorBase>>& InTasks)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FTasksService::InitializeTasks);
	for(const TSharedRef<FValidatorBase>& Task : InTasks)
	{
		if(!Task->Activate())
		{
			UE_LOGF(LogSubmitToolDebug, Error, "[%ls] has errors and is in an invalid state.", *Task->GetValidatorNameId().ToString());
		}
		else
		{
			UE_LOGF(LogSubmitToolDebug, Log, "Task '%ls' is active.", *Task->GetValidatorNameId().ToString());
		}

		// Create completed handle and register it with ValidationService
		Task->OnValidationFinished.Add(FOnValidatorFinished::FDelegate::CreateRaw(this, &FTasksService::OnTaskFinishedCallback));
		Tasks.Add(Task->GetValidatorNameId(), Task);
		CachedTasksArray.Emplace(Task);

		if(Task->Definition->ExecutionBlockGroups.Num() != 0)
		{
			CachedTasksWithGroups.Emplace(Task);
		}
		
		for(const FName& Id : Task->Definition->DependsOn)
		{
			const TSharedRef<FValidatorBase>* ParentTask = InTasks.FindByPredicate([&Id](const TSharedRef<FValidatorBase>& InOther){ return InOther->GetValidatorNameId() == Id;});

			if(ParentTask != nullptr)
			{
				(*ParentTask)->Dependants.Add(Task->GetValidatorNameId());
			}
		}
	}

	TickerHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateRaw(this, &FTasksService::Tick));		
}


bool FTasksService::QueueAll()
{
	bool bHasQueued = false;
	TSet<FName> TasksVisited;
	for(const TPair<FName, TSharedPtr<FValidatorBase>>& TaskPair : Tasks)
	{
		// Explictily bitwise because we want Queue for execution to be called everytime
		bHasQueued |= QueueForExecution(TaskPair.Value, false, TasksVisited);
	}

	return bHasQueued;
}

void FTasksService::QueueSingle(const FName& TaskId, bool bForceRun)
{
	if(Tasks.Contains(TaskId))
	{
		QueueForExecution(Tasks[TaskId], bForceRun);
	}
}

void FTasksService::QueueTypes(const FString& TaskType)
{
	TSet<FName> TasksVisited;

	for(const TPair<FName, TSharedPtr<FValidatorBase>>& TaskPair : Tasks)
	{
		if(TaskPair.Value->GetValidatorTypeName().Equals(TaskType))
		{
			QueueForExecution(TaskPair.Value, false, TasksVisited);
		}
	}
}

void FTasksService::QueueByArea(const ETaskArea& InArea)
{
	TSet<FName> TasksVisited;

	for(const TPair<FName, TSharedPtr<FValidatorBase>>& TaskPair : Tasks)
	{
		if((TaskPair.Value->Definition->TaskArea & InArea) != ETaskArea::None)
		{
			QueueForExecution(TaskPair.Value, false, TasksVisited);
		}
	}
}

bool FTasksService::QueueForExecution(const TSharedPtr<FValidatorBase>& InTask, bool InbForceRun, TSet<FName>& InOutVisitedTasks)
{
	InOutVisitedTasks.Add(InTask->GetValidatorNameId());
	if(InTask->GetIsQueued() || InTask->Definition->bIsDisabled)
	{
		// If already queued or disabled
		return false;
	}

	if (!InTask->IsRelevantToCL())
	{
		InTask->SetNotApplicable();
		return false;
	}

	if(!InTask->GetHasPassed() || InbForceRun)
	{
		if(InTask->Definition->DependsOn.Num() > 0)
		{
			bool bQueuedDependencies = false;
			for(const FName& DependencyId : InTask->Definition->DependsOn)
			{
				if(Tasks.Contains(DependencyId))
				{
					if(!InOutVisitedTasks.Contains(DependencyId))
					{
						bQueuedDependencies = true;

						// When we queue up dependencies, do not force queue them
						QueueForExecution(Tasks[DependencyId], false, InOutVisitedTasks);
					}
				}
				else
				{
					UE_LOGF(LogSubmitTool, Error, "Task %ls had a dependency on %ls which doesn't exist.", *InTask->GetValidatorNameId().ToString(), *DependencyId.ToString());
				}
			}
		}

		InTask->SetQueued(InbForceRun);
		return true;
	}
	else
	{
		UE_LOGF(LogValidatorsResult, Log, "[%ls] Already succeeded in a previous Task and is still valid", *InTask->GetValidatorName());
		return false;
	}
}

void FTasksService::OnTaskFinishedCallback(const FValidatorBase& InTask)
{
	if (InTask.GetFailureReason() != EFailureReason::Cancelled)
	{
		FTelemetryService::Get()->CustomEvent(TelemetryBaseId + TEXT(".Finished"), InTask.GetTelemetryAttributes());
	}
	else
	{
		FTelemetryService::Get()->CustomEvent(TelemetryBaseId + TEXT(".Cancelled"), InTask.GetTelemetryAttributes());
	}

	if(OnSingleTaskFinished.IsBound())
	{
		OnSingleTaskFinished.Broadcast(InTask);
	}
}

void FTasksService::PrintErrorSummary()
{
	bool bHeaderPrinted = false;
	for(const TPair<FName, TSharedPtr<FValidatorBase>>& TaskPair : Tasks)
	{
		if (TaskPair.Value->CanPrintErrors())
		{
			if (!bHeaderPrinted)
			{
				UE_LOGF(LogValidators, Error, "========================[Summary #%d]========================", Execution);
				UE_LOGF(LogValidatorsResult, Error, "========================[Summary #%d]========================", Execution);
				bHeaderPrinted = true;
			}
			TaskPair.Value->PrintErrorSummary();
		}
	}

	if (bHeaderPrinted)
	{
		UE_LOGF(LogValidators, Error, "================================================================");
		UE_LOGF(LogValidatorsResult, Error, "================================================================");
	}
}

void FTasksService::InvalidateForChanges(ETaskArea InChangeType)
{
	for(const TPair<FName, TSharedPtr<FValidatorBase>>& TaskPair : Tasks)
	{
		const TSharedPtr<FValidatorBase>& Task = TaskPair.Value;
		if((Task->Definition->TaskArea & InChangeType) != ETaskArea::None)
		{
			Task->Invalidate(true);
		}
	}
}

void FTasksService::InvalidateDependants(const TSharedPtr<FValidatorBase>& Task)
{
	for(const FName& Id : Task->Dependants)
	{
		Tasks[Id]->Invalidate();
		InvalidateDependants(Tasks[Id]);
	}
}

void FTasksService::CheckForLocalFileEdit()
{
	for(const TPair<FName, TSharedPtr<FValidatorBase>>& TaskPair : Tasks)
	{
		TaskPair.Value->InvalidateLocalFileModifications();
	}
}

void FTasksService::CheckForTagSkips()
{
	for(const TPair<FName, TSharedPtr<FValidatorBase>>& TaskPair : Tasks)
	{
		// If we have a tag in the CL that validates this validator, mark it as valid
		TaskPair.Value->EvaluateTagSkip();
	}
}

void FTasksService::ToggleEnabled(const FName& InTaskId)
{
	if (Tasks.Contains(InTaskId))
	{
		Tasks[InTaskId]->ToggleEnabled();
	}
}

bool FTasksService::GetIsAnyTaskRunning() const
{
	bool bAnyTaskRunningOrQueued = false;
	for(const TPair<FName, TSharedPtr<FValidatorBase>>& TaskPair : Tasks)
	{
		bAnyTaskRunningOrQueued |= TaskPair.Value->GetIsRunningOrQueued();
	}

	return bAnyTaskRunningOrQueued;
}

bool FTasksService::AreTasksPendingQueue() const
{
	bool bPendingQueue = false;
	for(const TPair<FName, TSharedPtr<FValidatorBase>>& TaskPair : Tasks)
	{
		bPendingQueue = bPendingQueue || TaskPair.Value->GetState() == EValidationStates::Not_Run;
	}

	return bPendingQueue;
}


bool FTasksService::GetIsRunSuccessful(bool bWaitForOptionalCompletes) const
{
	for(const TPair<FName, TSharedPtr<FValidatorBase>>& TaskPair : Tasks)
	{
		if(TaskPair.Value->Definition->IsRequired)
		{
			if(!TaskPair.Value->GetHasPassed())
			{
				return false;
			}
		}
		else
		{
			if (bWaitForOptionalCompletes && TaskPair.Value->Definition->bRequireCompleteWhenOptional && TaskPair.Value->GetIsRunningOrQueued() && TaskPair.Value->GetState() != EValidationStates::Not_Run)
			{
				return false;
			}
		}
	}

	return true;
}

const TArray<TWeakPtr<const FValidatorBase>>& FTasksService::GetTasks() const
{
	return CachedTasksArray;
}

const TArray<TWeakPtr<const FValidatorBase>> FTasksService::GetTasksOfType(const FString& TaskType) const
{
	TArray<TWeakPtr<const FValidatorBase>> ReturnedTasks;
	for(const TPair<FName, TSharedPtr<FValidatorBase>>& Pair : Tasks)
	{
		if(Pair.Value->GetValidatorTypeName() == TaskType)
		{
			ReturnedTasks.Add(Pair.Value);
		}
	}

	return ReturnedTasks;
}

bool FTasksService::Tick(float InDeltaTime)
{
	bool bCurrentCLValidState = true;
	bool bAnyTaskRunningOrQueued = false;
	bool bTasksStarted = false;
	TSet<FName> CurrentGroupsInExecution;
	
	for(const TWeakPtr<const FValidatorBase>& Task : CachedTasksWithGroups)
	{
		const TSharedPtr<const FValidatorBase> SharedPtrTask = Task.Pin();
		if(SharedPtrTask->GetIsRunning())
		{
			for(const FName& Group : SharedPtrTask->Definition->ExecutionBlockGroups)
			{
				CurrentGroupsInExecution.Add(Group);
			}
		}
	}


	for(const TPair<FName, TSharedPtr<FValidatorBase>>& TaskPair : Tasks)
	{
		const TSharedPtr<FValidatorBase>& Task = TaskPair.Value;

		if(Task->GetIsQueued())
		{
			bool bDependenciesSatisfied = true;
			for(const FName& DependencyId : Task->Definition->DependsOn)
			{
				if (!Tasks.Contains(DependencyId))
				{
					bDependenciesSatisfied = true;
					UE_LOGF(LogValidators, Warning, "%ls has a dependency on an invalid Task %ls, continuing execution.", *Task->GetValidatorName(), *DependencyId.ToString());
					break;
				}

				const TSharedPtr<FValidatorBase>& DependencyTask = Tasks[DependencyId];

				if(!DependencyTask->GetHasPassed())
				{
					if(!DependencyTask->GetIsRunningOrQueued())
					{
						UE_LOGF(LogValidatorsResult, Log, "%ls was waiting for dependency %ls but its state is %ls, %ls won't run", *Task->GetValidatorName(), *DependencyTask->GetValidatorName(), *DependencyTask->GetStatusText(), *Task->GetValidatorName());
						UE_LOGF(LogValidators, Log, "%ls was waiting for dependency %ls but its state is %ls, %ls won't run", *Task->GetValidatorName(), *DependencyTask->GetValidatorName(), *DependencyTask->GetStatusText(), *Task->GetValidatorName());
						Task->Invalidate(true);
					}					

					bDependenciesSatisfied = false;
				}
			}

			bool bBlockedByGroup = false;
			for(const FName& Group : CurrentGroupsInExecution)
			{
				if(Task->Definition->ExecutionBlockGroups.Contains(Group))
				{
					bBlockedByGroup = true;
				}
			}

			if(bDependenciesSatisfied && !bBlockedByGroup && Task->CanStartTask())
			{
				for(const FName& Group : Task->Definition->ExecutionBlockGroups)
				{
					CurrentGroupsInExecution.Add(Group);
				}

				UE_LOGF(LogValidatorsResult, Log, "[%ls] Running Task", *Task->GetValidatorName());
				Task->StartValidation();
				bTasksStarted = true;
			}
		}

		if(Task->GetIsRunning())
		{
			Task->Tick(InDeltaTime);
		}

		if (Task->Definition->IsRequired)
		{
			bCurrentCLValidState = bCurrentCLValidState && Task->GetHasPassed();
		}
		else if (Task->Definition->bRequireCompleteWhenOptional)
		{
			bCurrentCLValidState = bCurrentCLValidState && !Task->GetIsRunningOrQueued() && Task->GetState() != EValidationStates::Not_Run;
		}

		bAnyTaskRunningOrQueued = bAnyTaskRunningOrQueued || Task->GetIsRunningOrQueued();

		if (!Task->GetIsRunningOrQueued() && !Task->GetHasPassed())
		{
			InvalidateDependants(Task);
		}
	}

	if(bLastTasksRunState != bCurrentCLValidState)
	{
		bLastTasksRunState = bCurrentCLValidState;
		UE_LOGF(LogSubmitToolDebug, Verbose, "Tasks Run state updated: %ls", bLastTasksRunState ? TEXT("Valid") : TEXT("Invalid"));
		OnTasksRunResultUpdated.Broadcast(bLastTasksRunState);
	}

	if(bAnyTaskRunningOrQueued != bLastRunningTasks || bTasksStarted)
	{
		bLastRunningTasks = bAnyTaskRunningOrQueued;

		if(!bAnyTaskRunningOrQueued)
		{
			PrintErrorSummary();			

			TArray<FAnalyticsEventAttribute> ValidationRunResults = MakeAnalyticsEventAttributeArray(TEXT("Success"), bCurrentCLValidState);

			if(!bCurrentCLValidState)
			{
				FString FailedTasks;
				TSharedRef<TJsonWriter<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>> JsonWriter = TJsonWriterFactory<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>::Create(&FailedTasks, /*Indent=*/0);
				JsonWriter->WriteArrayStart();
				
				for(const TPair<FName, TSharedPtr<FValidatorBase>>& TaskPair : Tasks)
				{
					const TSharedPtr<FValidatorBase>& Task = TaskPair.Value;
					if(!TaskPair.Value->GetHasPassed() && TaskPair.Value->GetState() != EValidationStates::Not_Run)
					{
						JsonWriter->WriteObjectStart();
						JsonWriter->WriteValue(TEXT("TaskId"), Task->GetValidatorNameId().ToString());
						JsonWriter->WriteObjectEnd();
					}
				}
				JsonWriter->WriteArrayEnd();
				JsonWriter->Close();

				ValidationRunResults = AppendAnalyticsEventAttributeArray(ValidationRunResults, TEXT("FailedTasks"), FJsonFragment(MoveTemp(FailedTasks)));
			}

			++Execution;
			FTelemetryService::Get()->CustomEvent(TelemetryBaseId + TEXT(".FullRun"), ValidationRunResults);

			UE_LOGF(LogSubmitToolDebug, Verbose, "Task queue finished");
			OnTasksQueueFinished.Broadcast(bCurrentCLValidState);
		}
	}

	// return true to keep ticking
	return true;
}


void FTasksService::ResetStates()
{
	for(const TPair<FName, TSharedPtr<FValidatorBase>>& TaskPair : Tasks)
	{
		TaskPair.Value->Invalidate();
	}
}

const TArray<FString> FTasksService::GetAddendums() const
{
	TArray<FString> Addendums;
	for(const TPair<FName, TSharedPtr<FValidatorBase>>& TaskPair : Tasks)
	{
		if(!TaskPair.Value->Definition->ChangelistDescriptionAddendum.IsEmpty())
		{
			if (TaskPair.Value->GetState() == EValidationStates::Valid ||
				(TaskPair.Value->Definition->bAppendAddendumOnAnyResult && TaskPair.Value->GetState() == EValidationStates::Failed))
			{
				Addendums.Emplace(TaskPair.Value->Definition->ChangelistDescriptionAddendum);
			}
		}
	}

	return Addendums;
}


void FTasksService::StopTasks(const FName& InTaskId, bool InbAsFailed)
{
	for(const TPair<FName, TSharedPtr<FValidatorBase>>& TaskPair : Tasks)
	{
		if(InTaskId.IsNone() || InTaskId.IsEqual(TaskPair.Value->GetValidatorNameId()))
		{
			TaskPair.Value->CancelValidation(InbAsFailed);
		}
	}
}

void FTasksService::StopTasksByArea(const ETaskArea& InArea)
{
	for (const TPair<FName, TSharedPtr<FValidatorBase>>& TaskPair : Tasks)
	{
		if ((TaskPair.Value->Definition->TaskArea & InArea) != ETaskArea::None)
		{
			TaskPair.Value->CancelValidation();
		}
	}
}

const TWeakPtr<const FValidatorBase> FTasksService::GetTask(const FName& InId) const
{
	if (Tasks.Contains(InId))
	{
		return Tasks[InId];
	}
	else
	{
		return nullptr;
	}
}
