// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Logic/Validators/ValidatorBase.h"
#include "Parameters/SubmitToolParameters.h"
#include "Logic/Services/Interfaces/ISubmitToolService.h"

class FChangelistService;
class FTagService;
class FPreflightService;
class FSubmitToolPerforce;

DECLARE_MULTICAST_DELEGATE_OneParam(FOnTaskFinished, bool /*bSuccess*/)
DECLARE_MULTICAST_DELEGATE_OneParam(FOnSingleTaskFinished, const FValidatorBase& /*Validator*/)
DECLARE_MULTICAST_DELEGATE_OneParam(FOnTaskRunStateChanged, bool /*bValid*/)

class FTasksService : public ISubmitToolService
{
public:
	FTasksService(const TMap<FString, FString>& Tasks, const FString& InTelemetryCustomEvent);
	virtual ~FTasksService();

	virtual void InitializeTasks(const TArray<TSharedRef<FValidatorBase>>& InTasks);

	virtual bool QueueAll();
	virtual void QueueSingle(const FName& InId, bool InbForceRun);
	virtual void QueueTypes(const FString& TaskType);
	virtual void QueueByArea(const ETaskArea& InArea);
	virtual void InvalidateForChanges(ETaskArea InChangeType);
	virtual void CheckForLocalFileEdit();
	virtual void CheckForTagSkips();
	virtual void ToggleEnabled(const FName& InTaskId);

	virtual bool GetIsRunSuccessful(bool bWaitForOptionalCompletes = true) const;
	virtual bool GetIsAnyTaskRunning() const;
	virtual bool AreTasksPendingQueue() const;

	virtual const TWeakPtr<const FValidatorBase> GetTask(const FName& InId) const;
	virtual const TArray<TWeakPtr<const FValidatorBase>>& GetTasks() const;
	virtual const TArray<TWeakPtr<const FValidatorBase>> GetTasksOfType(const FString& TaskType) const;

	virtual bool Tick(float InDeltaTime);
	virtual void ResetStates();

	virtual const TArray<FString> GetAddendums() const;

	virtual void StopTasks(const FName& InValidatorId = FName(), bool InbAsFailed = false);
	virtual void StopTasksByArea(const ETaskArea& InArea);

	FOnTaskFinished OnTasksQueueFinished;
	FOnSingleTaskFinished OnSingleTaskFinished;
	FOnTaskRunStateChanged OnTasksRunResultUpdated;
protected:
	virtual bool QueueForExecution(const TSharedPtr<FValidatorBase>& InTask, bool InbForceRun) { TSet<FName> VisitedValidators; return QueueForExecution(InTask, InbForceRun, VisitedValidators); }
	virtual bool QueueForExecution(const TSharedPtr<FValidatorBase>& InTask, bool InbForceRun, TSet<FName>&);

	virtual void OnTaskFinishedCallback(const FValidatorBase& InValidator);
	virtual void PrintErrorSummary();

	virtual void InvalidateDependants(const TSharedPtr<FValidatorBase>& InTask);

	TMap<FName, TSharedPtr<FValidatorBase>> Tasks;
	TArray<TWeakPtr<const FValidatorBase>> CachedTasksArray;
	TArray<TWeakPtr<const FValidatorBase>> CachedTasksWithGroups;
	bool bLastTasksRunState;
	bool bLastRunningTasks;

	const FString TelemetryBaseId;
	uint64_t Execution = 1;

	FTSTicker::FDelegateHandle TickerHandle;
};

