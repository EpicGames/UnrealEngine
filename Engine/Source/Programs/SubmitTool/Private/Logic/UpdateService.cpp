// Copyright Epic Games, Inc. All Rights Reserved.

#include "UpdateService.h"

#include "HAL/FileManager.h"
#include "HttpModule.h"
#include "Interfaces/IHttpResponse.h"
#include "JsonObjectConverter.h"
#include "Misc/CommandLine.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"

#include "CommandLine/CmdLineParameters.h"
#include "Logging/SubmitToolLog.h"
#include "Logic/ProcessWrapper.h"
#include "Models/HordeDeploymentData.h"
#include "Framework/Application/SlateApplication.h"
#include "Logic/Services/SubmitToolServiceProvider.h"
#include "Logic/Services/Interfaces/IChangelistService.h"

FUpdateService::FUpdateService(const FHordeParameters& InHordeParameters, const FAutoUpdateParameters& InAutoUpdateParameters, TWeakPtr<FSubmitToolServiceProvider> InServiceProvider) :
	HordeParameters(InHordeParameters),
	AutoUpdateParameters(InAutoUpdateParameters),
	ServiceProvider(InServiceProvider),
	DownloadFile(nullptr),
	Downloaded(0)
{
}

FUpdateService::~FUpdateService()
{
}

bool FUpdateService::CheckForNewVersion()
{
	if (!FPaths::IsStaged())
	{
		return false;
	}

	if (!this->AutoUpdateParameters.bIsAutoUpdateOn)
	{
		return false;
	}

	GetLocalVersion();
	GetLatestVersion();

	UE_LOGF(LogSubmitToolDebug, Log, "Submit Tool Versions:\nLocal Version: %ls\nRemote Version: %ls", *LocalVersion, *RemoteVersion);

	if (RemoteVersion.IsEmpty())
	{
		return false;
	}

	return LocalVersion != RemoteVersion;
}

void FUpdateService::StartAutoUpdateScript()
{
	// security to avoid deleting the current folder while debugging
	// the update script assumes that the submit tool is staged and installed in isolation
	if (!FPaths::IsStaged())
	{
		return;
	}
	
	FString Name = TEXT("Submit Tool Auto Update");
	FString Cmd = AutoUpdateParameters.AutoUpdateCommand;
	
#if PLATFORM_MAC
	// install path is five levels above the engine path for Mac: ExtractTargetFolder/Mac/SubmitTool.app/Contents/UE/Engine
	const FString InstallFolder = FPaths::ConvertRelativePathToFull(FString::Printf(TEXT("%s../../../../.."), *FPaths::EngineDir()));
#else
	// install path is two levels above the engine path for Linux & Windows: ExtractTargetFolder/Windows/Engine
	const FString InstallFolder = FPaths::ConvertRelativePathToFull(FString::Printf(TEXT("%s../.."), *FPaths::EngineDir()));
#endif
	// copy the auto update script
	const FString& UpdateScriptDirectory = FPaths::GetPath(this->AutoUpdateParameters.LocalAutoUpdateScript);
	IFileManager::Get().MakeDirectory(*UpdateScriptDirectory, true);
	IFileManager::Get().Copy(*this->AutoUpdateParameters.LocalAutoUpdateScript, *this->AutoUpdateParameters.AutoUpdateScript, true, true);

	FString RootDir;
	FCmdLineParameters::Get().GetValue(FSubmitToolCmdLine::RootDir, RootDir);

	FString Args = AutoUpdateParameters.AutoUpdateArgs;
	Args = Args.Replace(TEXT("$(script)"), *FPaths::ConvertRelativePathToFull(AutoUpdateParameters.LocalAutoUpdateScript));
	Args = Args.Replace(TEXT("$(zip)"), *AutoUpdateParameters.LocalDownloadZip);
	Args = Args.Replace(TEXT("$(folder)"), *InstallFolder);
	Args = Args.Replace(TEXT("$(executablename)"), FPlatformProcess::ExecutableName(true));
	Args = Args.Replace(TEXT("$(version)"), *LatestVersionDownloaded);
	Args = Args.Replace(TEXT("$(versionfile)"), *AutoUpdateParameters.LocalVersionFile);
	Args = Args.Replace(TEXT("$(executablepath)"), FPlatformProcess::ExecutablePath());
	Args = Args.Replace(TEXT("$(rootdir)"), *RootDir.TrimStartAndEnd());
	Args = Args.Replace(TEXT("$(executableargs)"), *GetSubmitToolArgs());

	UE_LOGF(LogSubmitTool, Warning, "Starting Auto-Update Script %ls %ls", *Cmd, *Args);

	const bool bLaunchDetached = false;
	const bool bLaunchesHidden = false;
	const bool bLaunchesReallyHidden = false;

	FProcessWrapper ScriptProcess = 
		FProcessWrapper(TEXT("AutoUpdateScript"),
			Cmd,
			Args,
			nullptr,
			FOnOutputLine::CreateLambda([](FString InOutput, const EProcessOutputType& OutputType){ UE_LOGF(LogSubmitToolDebug, Log, "AutoUpdateOutput: %ls", *InOutput); }),
			FString(),
			bLaunchesHidden,
			bLaunchesReallyHidden,
			bLaunchDetached,
			false);

	ScriptProcess.Start();

	FSlateApplication::Get().CloseAllWindowsImmediately();
}

const FString& FUpdateService::GetDeployId()
{
	if(DeployId.IsEmpty())
	{
		UE_LOGF(LogSubmitToolDebug, Log, "Trying to load deploy id from file %ls", *AutoUpdateParameters.DeployIdFilePath);
		if(FPaths::FileExists(this->AutoUpdateParameters.DeployIdFilePath))
		{
			FFileHelper::LoadFileToString(DeployId, *this->AutoUpdateParameters.DeployIdFilePath);
			DeployId.TrimStartAndEndInline();
		}
	}

	return DeployId;
}

void FUpdateService::Cancel()
{
	if (this->DownloadRequest.IsValid() && DownloadRequest->GetStatus() == EHttpRequestStatus::Processing)
	{
		this->DownloadRequest->CancelRequest();
	}

	this->Downloaded = 0;
}


void FUpdateService::InstallLatestVersion()
{
	if(GetDeployId().IsEmpty())
	{
		return;		
	}

	TSharedPtr<IHttpRequest> Request = FHttpModule::Get().CreateRequest();

	const FString& Url = FString::Format(TEXT("{0}api/v1/tools/{1}"), { HordeParameters.HordeServerAddress,  DeployId });

	Request->SetURL(Url);
	Request->SetVerb(TEXT("GET"));

	Request->OnProcessRequestComplete().BindLambda([this](FHttpRequestPtr Request, FHttpResponsePtr HttpResponse, bool bConnectedSuccessfully)
		{
			if (!bConnectedSuccessfully)
			{
				if (HttpResponse.IsValid())
				{
					UE_LOGF(LogSubmitTool, Warning, "Unable to connect to horde. Connection error %d", HttpResponse->GetResponseCode());
					UE_LOGF(LogSubmitToolDebug, Warning, "Unable to connect to horde. Connection error\nResponse: %ls", *HttpResponse->GetContentAsString());
				}
				else
				{
					UE_LOGF(LogSubmitTool, Warning, "Unable to connect to horde. Connection error, no response.");
				}
				return;
			}

			if (HttpResponse.IsValid())
			{
				if (EHttpResponseCodes::IsOk(HttpResponse->GetResponseCode()))
				{
					FDeploymentList DeploymentList;

					FString Content = HttpResponse->GetContentAsString();
					FJsonObjectConverter::JsonObjectStringToUStruct<FDeploymentList>(*Content, &DeploymentList);

					const int32 Num = DeploymentList.Deployments.Num();
					if (Num == 0)
					{
						UE_LOGF(LogSubmitTool, Warning, "Unable to retrieve latest deployment from Horde.");
						return;
					}

					int32 Latest = 0;
					for (int Idx = 1; Idx < Num; Idx++)
					{
						FDateTime LatestTime;
						FDateTime::ParseIso8601(*DeploymentList.Deployments[Latest].StartedAt, LatestTime);

						FDateTime CurrentTime;
						FDateTime::ParseIso8601(*DeploymentList.Deployments[Idx].StartedAt, CurrentTime);

						if (CurrentTime > LatestTime)
						{
							Latest = Idx;
						}
					}

					const FString& LatestVersion = DeploymentList.Deployments[Latest].Id;

					UE_LOGF(LogSubmitTool, Display, "Local SubmitTool version: %ls.", *GetLocalVersion());
					UE_LOGF(LogSubmitTool, Display, "Latest SubmitTool version available on Horde: %ls.", *LatestVersion);

					if (LatestVersion != GetLocalVersion())
					{
						UE_LOGF(LogSubmitTool, Display, "Submit Tool needs to be updated.");
						this->DownloadLatestVersion(DeployId, LatestVersion);
					}
				}
				else
				{
					UE_LOGF(LogSubmitTool, Warning, "Unable to fetch latest Submmit Tool deployment. Failed with code %d", HttpResponse->GetResponseCode());
					UE_LOGF(LogSubmitToolDebug, Warning, "Unable to fetch latest Submmit Tool deployment. Failed with code %d\nResponse: %ls", HttpResponse->GetResponseCode(), *HttpResponse->GetContentAsString());
				}
			}
		});

	Request->ProcessRequest();
}

const FString& FUpdateService::GetLatestVersion(bool bForce)
{
	if(GetDeployId().IsEmpty())
	{
		return RemoteVersion;
	}

	if(!RemoteVersion.IsEmpty() && !bForce)
	{
		return RemoteVersion;
	}

	TSharedPtr<IHttpRequest> Request = FHttpModule::Get().CreateRequest();

	const FString& Url = FString::Format(TEXT("{0}api/v1/tools/{1}"), { HordeParameters.HordeServerAddress,  DeployId });

	UE_LOGF(LogSubmitToolDebug, Log, "Fetching last version from horde using URL: %ls", *Url);

	Request->SetURL(Url);
	Request->SetVerb(TEXT("GET"));
	Request->ProcessRequestUntilComplete();

	FHttpResponsePtr HttpResponse = Request->GetResponse();

	if (Request->GetStatus() == EHttpRequestStatus::Succeeded)
	{
		FDeploymentList DeploymentList;

		FString Content = HttpResponse->GetContentAsString();
		FJsonObjectConverter::JsonObjectStringToUStruct<FDeploymentList>(*Content, &DeploymentList);

		const int32 Num = DeploymentList.Deployments.Num();
		if (Num == 0)
		{
			UE_LOGF(LogSubmitTool, Warning, "Unable to retrieve latest deployment from Horde.");
			return RemoteVersion;
		}

		int32 Latest = 0;
		for (int Idx = 1; Idx < Num; Idx++)
		{
			FDateTime LatestTime;
			FDateTime::ParseIso8601(*DeploymentList.Deployments[Latest].StartedAt, LatestTime);

			FDateTime CurrentTime;
			FDateTime::ParseIso8601(*DeploymentList.Deployments[Idx].StartedAt, CurrentTime);

			if (CurrentTime > LatestTime)
			{
				Latest = Idx;
			}
		}

		RemoteVersion = DeploymentList.Deployments[Latest].Id;
	}
	else if (Request->GetStatus() == EHttpRequestStatus::Failed)
	{
		if (HttpResponse.IsValid())
		{
			UE_LOGF(LogSubmitTool, Warning, "Unable to fetch latest Submmit Tool deployment. Failed with code %d", HttpResponse->GetResponseCode());
			UE_LOGF(LogSubmitToolDebug, Warning, "Unable to fetch latest Submmit Tool deployment. Failed with code %d\nResponse: %ls", HttpResponse->GetResponseCode(), *HttpResponse->GetContentAsString());
		}
		else
		{
			UE_LOGF(LogSubmitTool, Warning, "Unable to fetch latest Submmit Tool deployment.");
		}
	}

	return RemoteVersion;
}

const FString& FUpdateService::GetLocalVersion()
{
	if (LocalVersion.IsEmpty() && FPaths::FileExists(this->AutoUpdateParameters.LocalVersionFile))
	{
		FFileHelper::LoadFileToString(LocalVersion, *this->AutoUpdateParameters.LocalVersionFile);
			
		LocalVersion = LocalVersion
			.TrimStartAndEnd()
			.Replace(*FString("\r"), *FString(""))
			.Replace(*FString("\n"), *FString(""))
			;
	}

	return LocalVersion;
}


const FString FUpdateService::GetDownloadMessage()
{
	if(Downloaded != 0 && DownloadErrorMessage.IsEmpty())
	{
		return FString::Printf(TEXT("Downloading: %s"), *GetReadableDownloadSize());
	}
	else
	{
		return DownloadErrorMessage;
	}
}

void FUpdateService::SaveLocalVersionToFile() const
{
	if(!LocalVersion.IsEmpty())
	{
		UE_LOGF(LogSubmitToolDebug, Log, "Saving current version to file %ls", *AutoUpdateParameters.LocalVersionFile);
		FFileHelper::SaveStringToFile(LocalVersion, *this->AutoUpdateParameters.LocalVersionFile);
	}
}


bool FUpdateService::DownloadLatestVersion(const FString& InDeployId, const FString& InLatestVersion)
{
	UE_LOGF(LogSubmitTool, Display, "Downloading latest Submit Tool version: %ls.", *InLatestVersion);

	FString Url = FString::Format(TEXT("{0}api/v1/tools/{1}?action=download"), { HordeParameters.HordeServerAddress,  InDeployId });

	this->DownloadRequest = FHttpModule::Get().CreateRequest();
	this->DownloadRequest->SetURL(Url);
	this->DownloadRequest->SetVerb(TEXT("GET"));

	IFileManager::Get().Delete(*this->AutoUpdateParameters.LocalDownloadZip, false, true, true);

	this->DownloadFile = IFileManager::Get().CreateFileWriter(*this->AutoUpdateParameters.LocalDownloadZip, EFileWrite::FILEWRITE_AllowRead);

	FHttpRequestStreamDelegateV2 StreamDelegate;
	StreamDelegate.BindLambda([this](void* InDataPtr, int64& InOutLength) {this->OnProcessDownloadRequestStream(InDataPtr, InOutLength); });
	this->DownloadRequest->SetResponseBodyReceiveStreamDelegateV2(StreamDelegate);

	this->DownloadRequest->OnProcessRequestComplete().BindLambda([this, InLatestVersion](FHttpRequestPtr Request, FHttpResponsePtr HttpResponse, bool bConnectedSuccessfully)
		{
			if (!bConnectedSuccessfully)
			{
				if (HttpResponse.IsValid())
				{
					DownloadErrorMessage = FString::Printf(TEXT("Unable to download latest Submit Tool deployment. Failed with code %d, See Logs for more info."), HttpResponse->GetResponseCode());
					UE_LOGF(LogSubmitTool, Warning, "Unable to connect to horde. Connection error %d\nResponse: %ls", HttpResponse->GetResponseCode(), *HttpResponse->GetContentAsString());
				}
				else
				{
					DownloadErrorMessage = FString::Printf(TEXT("Unable to download latest Submit Tool deployment. Unknown Error."));
					UE_LOGF(LogSubmitTool, Warning, "Unable to connect to horde. Connection error, no response.");
				}
				return;
			}

			if (HttpResponse.IsValid())
			{
				if(DownloadFile)
				{
					this->DownloadFile->Flush();
					this->DownloadFile->Close();
					delete DownloadFile; DownloadFile = nullptr;
				}

				if (EHttpResponseCodes::IsOk(HttpResponse->GetResponseCode()))
				{
					LatestVersionDownloaded = InLatestVersion;
					this->StartAutoUpdateScript();
				}
				else
				{
					DownloadErrorMessage = FString::Printf(TEXT("Unable to download latest Submit Tool deployment. Failed with code %d, See Logs for more info."), HttpResponse->GetResponseCode());
					UE_LOGF(LogSubmitTool, Warning, "Unable to download latest Submit Tool deployment. Failed with code %d", HttpResponse->GetResponseCode());
					UE_LOGF(LogSubmitToolDebug, Warning, "Unable to download latest Submit Tool deployment. Failed with code %d\nResponse: %ls", HttpResponse->GetResponseCode(), *HttpResponse->GetContentAsString());
				}
			}
			else
			{
				DownloadErrorMessage = FString::Printf(TEXT("Unable to download latest Submit Tool deployment. Unknown Error."));
				UE_LOGF(LogSubmitTool, Warning, "Unable to connect to horde. Connection error, no response.");
			}
		});

	DownloadErrorMessage = FString();
	DownloadRequest->ProcessRequest();
	return true;
}

void FUpdateService::OnProcessDownloadRequestStream(void* InDataPtr, int64& InOutLength)
{
	if (DownloadFile == nullptr || !DownloadRequest.IsValid())
	{
		return;
	}

	if (DownloadRequest->GetStatus() == EHttpRequestStatus::Failed)
	{
		return;
	}
	this->DownloadFile->Serialize(InDataPtr, InOutLength);
	Downloaded += InOutLength;
}

FString FUpdateService::GetSubmitToolArgs() const
{
	FString Commandline = FCommandLine::Get();

	if (Commandline.Contains(TEXT("-cl default")))
	{
		Commandline.ReplaceInline(TEXT("-cl default"), *FString::Printf(TEXT("-cl %s"), *ServiceProvider.Pin()->GetService<IChangelistService>()->GetCLID()));
	}

	return Commandline;
}

const FString FUpdateService::GetReadableDownloadSize()
{
	static const TCHAR* Units[] = { TEXT("B"), TEXT("KB"), TEXT("MB"), TEXT("GB"), TEXT("TB"), TEXT("PB"), TEXT("EB") };

	uint64 UnitIndex = 0;
	long bytes = Downloaded;
	while((bytes > 1024) && (UnitIndex++ < (UE_ARRAY_COUNT(Units) - 1)))
	{
		bytes >>= 10llu;
	}

	return FString::Printf(TEXT("%llu%s"), bytes, Units[UnitIndex]);
}