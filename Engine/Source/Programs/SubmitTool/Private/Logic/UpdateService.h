// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Parameters/SubmitToolParameters.h"
#include "Logic/Services/Interfaces/ISubmitToolService.h"

class IHttpRequest;
class FSubmitToolServiceProvider;

class FUpdateService final: public ISubmitToolService
{
public:
	FUpdateService() = delete;
	FUpdateService(const FHordeParameters& InHordeParameters, const FAutoUpdateParameters& InAutoUpdateParameters, TWeakPtr<FSubmitToolServiceProvider> InServiceProvider);
	~FUpdateService();

	bool CheckForNewVersion();

	const FString& GetDeployId();
	const FString& GetLocalVersion();
	const FString& GetLatestVersion(bool bForce = false);

	void InstallLatestVersion();
	void Cancel();

	const FString GetDownloadMessage();

private:
	bool DownloadLatestVersion(const FString& DeployId, const FString& LatestVersion);
	FString GetSubmitToolArgs() const;
	void StartAutoUpdateScript();
	void SaveLocalVersionToFile() const;

	void OnProcessDownloadRequestStream(void* InDataPtr, int64& InOutLength);
	const FString GetReadableDownloadSize();

private:
	const FHordeParameters& HordeParameters;
	const FAutoUpdateParameters& AutoUpdateParameters;
	TWeakPtr<FSubmitToolServiceProvider> ServiceProvider;

	FArchive* DownloadFile;
	TSharedPtr<IHttpRequest> DownloadRequest;
	long Downloaded = 0;

	FString LatestVersionDownloaded;

	FString DeployId;
	FString LocalVersion;
	FString RemoteVersion;

	FString DownloadErrorMessage;
};

Expose_TNameOf(FUpdateService);