// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreRedirectsValidator.h"

#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Configuration/Configuration.h"
#include "Logging/SubmitToolLog.h"
#include "Logic/Services/Interfaces/ISTSourceControlService.h"
#include "Logic/Validators/ValidatorFactory.h"

REGISTER_VALIDATOR_TYPE(SubmitToolParseConstants::CoreRedirectsValidator, FCoreRedirectsValidator)

void FCoreRedirectsValidator::StartAsyncWork(const FString& CLDescription, const TArray<FSCFileRef>& FilteredFilesInCL, const TArray<const FTag*>& Tags) 
{
	FilesToValidate.Reset();
	for (const FSCFileRef& FileInCl : FilteredFilesInCL)
	{
		if (FileInCl->IsDeleted())
		{
			continue;
		}

		if (FPaths::GetCleanFilename(FileInCl->GetFilename()).EndsWith(TEXT("Engine.ini"), ESearchCase::IgnoreCase))
		{
			FilesToValidate.Add(FileInCl);
		}
	}

	TagStatus = Tags;

	TSharedPtr<ISTSourceControlService> SourceControlService = ServiceProvider.Pin()->GetService<ISTSourceControlService>();

	StartAsyncTask([this](const UE::Tasks::FCancellationToken& InCancellationToken)->bool
		{
			if (FilesToValidate.IsEmpty())
			{
				return true;
			}

			return DoWork(InCancellationToken);
		});
}

bool FCoreRedirectsValidator::DoWork(const UE::Tasks::FCancellationToken& InCancellationToken)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FCoreRedirectsValidator::DoWork);

	// First check the tags to avoid doing unnecessary work if the user has already promised it's safe
	FString DocumentationURL;
	for (const FTag* Tag : TagStatus)
	{
		if (Tag)
		{
			if (Tag->Definition.TagId.Compare(TEXT("#redirectsaresafe"), ESearchCase::IgnoreCase) == 0)
			{
				if (Tag->IsEnabled())
				{
					return true;
				}
				else
				{
					DocumentationURL = Tag->Definition.DocumentationUrl;
				}
			}
		}
	}

	TArray<FString> FilePaths;

	// validate added files
	bool bAnyFilesContainRelevantModifications = false;
	for (const FSCFileRef& File : FilesToValidate)
	{
		if (InCancellationToken.IsCanceled())
		{
			return false;
		}

		if (File->IsAdded())
		{
			// This file is new (or potentially moved/renamed). We'll just look to see if it contains redirects.
			TArray<FString> WorkspaceContents;
			if (!FFileHelper::LoadFileToStringArray(WorkspaceContents, *File->GetFilename()))
			{
				LogFailure(FString::Printf(TEXT("[%s]: Unable to perform validation because we failed to load '%s'."), *GetValidatorName(), *File->GetFilename()));
				return false;
			}
			else
			{
				if (ContainsAnyRedirects(WorkspaceContents))
				{
					LogFailure(FString::Printf(TEXT("[%s]: Your changelist includes changes to PackageRedirectors via ini file. Please read the linked documentation"), *GetValidatorName()));
					LogFailure(FString::Printf(TEXT("[%s]: %s"), *GetValidatorName(), *DocumentationURL));
					LogFailure(FString::Printf(TEXT("[%s]: If your changes are safe, add #redirectsaresafe"), *GetValidatorName()));
					return false;
				}
			}
		}
		else
		{
			FilePaths.Add(File->GetFilename());
		}
	}

	if (FilePaths.IsEmpty())
	{
		return true;
	}

	// download remaining files
	TMap<FString, FSharedBuffer> FileBuffers;
	if (!ServiceProvider.Pin()->GetService<ISTSourceControlService>()->DownloadFiles(CopyTemp(FilePaths), FileBuffers).GetResult() || FileBuffers.Num() != FilePaths.Num())
	{
		LogFailure(FString::Printf(TEXT("[%s]: Unable to perform validation because we could not download %d unedited file(s) from version control."), *GetValidatorName(), FilePaths.Num()));
		return false;
	}

	// validate downloaded files
	TRACE_CPUPROFILER_EVENT_SCOPE(ValidateDownloadedFiles);
	for (const TPair<FString, FSharedBuffer>& DownloadedFile : FileBuffers)
	{
		if (InCancellationToken.IsCanceled())
		{
			return false;
		}

		FString Filename;
		for (int32 i = 0; i < FilePaths.Num(); ++i)
		{
			if (FPaths::GetCleanFilename(FilePaths[i]).Equals(FPaths::GetCleanFilename(DownloadedFile.Key), ESearchCase::IgnoreCase))
			{
				Filename = FilePaths[i];
				break;
			}
		}

		if (Filename.IsEmpty())
		{
			continue;
		}

		const FSharedBuffer& FileBuffer = DownloadedFile.Value;

		if (FileBuffer.GetSize() == 0)
		{
			LogFailure(FString::Printf(TEXT("[%s]: Unable to perform validation because we received bad data for file ('%s') from version control."), *GetValidatorName(), *Filename));
			return false;
		}

		const FAnsiStringView DataView(reinterpret_cast<const char*>(FileBuffer.GetData()), FileBuffer.GetSize());
		TSet<FAnsiStringView> DepotContents;
		PopulateSetFromStringViewOfFile(DataView, DepotContents);

		TArray<FString> WorkspaceContents;
		if (!FFileHelper::LoadFileToStringArray(WorkspaceContents, *Filename))
		{
			LogFailure(FString::Printf(TEXT("[%s]: Unable to perform validation because we failed to load '%s'."), *GetValidatorName(), *Filename));
			return false;
		}

		if (ContainsModifiedRedirects(DepotContents, WorkspaceContents))
		{
			LogFailure(FString::Printf(TEXT("[%s]: Your changelist includes changes to PackageRedirectors via ini file. Please read the linked documentation"), *GetValidatorName()));
			LogFailure(FString::Printf(TEXT("[%s]: %s"), *GetValidatorName(), *DocumentationURL));
			LogFailure(FString::Printf(TEXT("[%s]: If your changes are safe, add #redirectsaresafe"), *GetValidatorName()));
			return false;
		}
	}

	return true;

}

void FCoreRedirectsValidator::PopulateSetFromStringViewOfFile(FAnsiStringView InFile, TSet<FAnsiStringView>& OutSet)
{
	int32 LineTerminator = 0;
	// Iterate over the total string and repeatedly split it on line endings into a new line for OutSet and a remainder
	while (InFile.FindChar('\n', LineTerminator))
	{
		OutSet.Add(InFile.Left(LineTerminator));
		InFile = InFile.RightChop(LineTerminator + 1);
	}
	
	if (InFile.Len())
	{
		// Add the last line
		OutSet.Add(InFile);
	}
}

bool FCoreRedirectsValidator::ContainsModifiedRedirects(const TSet<FAnsiStringView>& DepotContents, const TArray<FString>& WorkspaceContents)
{
	for (const FString& Line : WorkspaceContents)
	{
		if (!DepotContents.Contains(FAnsiStringView(TCHAR_TO_ANSI(*Line), Line.Len())))
		{
			FStringView LineView(Line);
			LineView = LineView.TrimStart();
			if (LineView.StartsWith(TEXT("+PackageRedirects")))
			{
				return true;
			}
		}
	}
	return false;
}

bool FCoreRedirectsValidator::ContainsAnyRedirects(const TArray<FString>& FileContents)
{
	for (const FString& Line : FileContents)
	{
		FStringView LineView(Line);
		LineView = LineView.TrimStart();
		if (LineView.StartsWith(TEXT("+PackageRedirects")))
		{
			return true;
		}
	}
	return false;
}
