// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ValidatorBaseAsync.h"
#include "Tasks/Task.h"

namespace SubmitToolParseConstants
{
	const FString CoreRedirectsValidator = TEXT("CoreRedirectsValidator");
}

class FCoreRedirectsValidator : public FValidatorBaseAsync
{
public:
	using FValidatorBaseAsync::FValidatorBaseAsync;
	virtual void StartAsyncWork(const FString& CLDescription, const TArray<FSCFileRef>& FilteredFilesInCL, const TArray<const FTag*>& Tags) override;

public:

	virtual const FString& GetValidatorTypeName() const override { return SubmitToolParseConstants::CoreRedirectsValidator; }

private:

	TArray<const FTag*> TagStatus;
	TArray<FSCFileRef> FilesToValidate;

	bool DoWork(const UE::Tasks::FCancellationToken& InCancellationToken);

	void PopulateSetFromStringViewOfFile(FAnsiStringView InFile, TSet<FAnsiStringView>& OutSet);

	virtual bool ContainsModifiedRedirects(const TSet<FAnsiStringView>& DepotContents, const TArray<FString>& WorkspaceContents);

	bool ContainsAnyRedirects(const TArray<FString>& FileContents);
};
