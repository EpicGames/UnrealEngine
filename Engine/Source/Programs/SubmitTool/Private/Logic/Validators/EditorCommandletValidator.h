// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ValidatorRunExecutable.h"

struct FEditorParameters
{
	FString EditorExePath;
	FString EditorArguments;
};

namespace SubmitToolParseConstants
{
	const FString EditorValidator = TEXT("EditorValidator");
}

class FEditorCommandletValidator : public FValidatorRunExecutable
{

public:		
	FEditorCommandletValidator(const FName& InNameId, const FSubmitToolParameters& InParameters, TSharedRef<FSubmitToolServiceProvider> InServiceProvider, const FString& InDefinition);
	virtual void ParseDefinition(const FString& InDefinition) override;
	virtual bool Validate(const FString& InCLDescription, const TArray<FSCFileRef>& InFilteredFilesInCL, const TArray<const FTag*>& InTags) override;
	virtual const FString& GetValidatorTypeName() const override { return SubmitToolParseConstants::EditorValidator; }
	virtual bool Activate() override;

protected:
	virtual void ProcessSingleMessageForTelemetry(FString& InOutMessage) const override;
	void GetEditorsForPaths(const TArray<FSCFileRef>& InFilteredFilesInCL, TMap<FString, FEditorParameters>& OutProjectEditorParameters) const;
	void SortFilesByProjects(const TArray<FSCFileRef>& InFiles, TMap<FString, TArray<FSCFileRef>>& OutProjects, TMap<FString, TArray<FSCFileRef>>& OutSubProjects) const;
	TMap<FString, TSet<FString>> GetEnableDisablePluginArguments(const TArray<FSCFileRef>& InFilesInCL) const;

	void ExtractArgumentValues(FString& InArguments, const FString& InKey, TSet<FString>& InOutValues, const TCHAR InDelimiter = TCHAR(',')) const;

	TMap<FString, TUniquePtr<FProcessWrapper>> Processes;

private:
	FString MakeTemporaryUserDirectory() const;

	// Regex used for clearing the editor log timestamp, looks for "]Log******:" and removes everything before it
	FString RegexMessageProcessing = TEXT(".*](?=Log\\w+:)");
	TSharedPtr<FRegexPattern> BuiltRegexProcessing;
};
