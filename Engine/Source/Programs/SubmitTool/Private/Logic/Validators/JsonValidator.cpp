// Copyright Epic Games, Inc. All Rights Reserved.

#include "JsonValidator.h"

#include "Internationalization/Regex.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "Logic/Validators/ValidatorFactory.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Misc/StringOutputDevice.h"

#include "Models/ModelInterface.h"

REGISTER_VALIDATOR_TYPE(SubmitToolParseConstants::JsonValidator, FJsonValidator)

FJsonValidator::FJsonValidator(const FName& InNameId, const FSubmitToolParameters& InParameters, TSharedRef<FSubmitToolServiceProvider> InServiceProvider, const FString& InDefinition) :
	FValidatorBaseAsync(InNameId, InParameters, InServiceProvider, InDefinition)
{
	ParseDefinition(InDefinition);
}

void FJsonValidator::StartAsyncWork(const FString& CLDescription, const TArray<FSCFileRef>& FilteredFilesInCL, const TArray<const FTag*>& Tags)
{
	TArray<TArray<FSCFileRef>> Batches;
	Batches.Add(TArray<FSCFileRef>());

	const int32 MaxTasks = FMath::Max(1, FilteredFilesInCL.Num() / 4);
	for (const FSCFileRef& File : FilteredFilesInCL)
	{
		if (Batches.Last().Num() >= MaxTasks)
		{
			Batches.Add(TArray<FSCFileRef>());
		}

		Batches.Last().Add(File);
	}

	for (const TArray<FSCFileRef>& Batch : Batches)
	{
		this->StartAsyncTask([this, Batch](const UE::Tasks::FCancellationToken& InCancellationToken) -> bool
			{
				return ValidateJson(Batch, InCancellationToken);
			});
	}
}

bool FJsonValidator::ValidateJson(const TArray<FSCFileRef>& InFilteredFilesInCL, const UE::Tasks::FCancellationToken& InCancellationToken) const
{
	bool bValid = true;

	for (const FSCFileRef& File : InFilteredFilesInCL)
	{
		if (InCancellationToken.IsCanceled())
		{
			break;
		}

		const FString& Filename = File->GetFilename();

		FString JSON;
		const FString& RegexFiltering = GetTypedDefinition<FJsonValidatorDefinition>()->RegexLineExclusion;
		if (!RegexFiltering.IsEmpty())
		{
			FRegexPattern Pattern = FRegexPattern(RegexFiltering, ERegexPatternFlags::CaseInsensitive);

			TArray<FString> Lines;
			if (!FFileHelper::LoadFileToStringArrayWithPredicate(Lines, *Filename, [&Pattern](const FString& Line) { return !FRegexMatcher(Pattern, Line).FindNext(); }))
			{
				LogFailure(FString::Printf(TEXT("[%s] %s could not be loaded"), *GetValidatorName(), *Filename));
				bValid = false;
				continue;
			}

			JSON = FString::Join(Lines, TEXT("\n"));
		}
		else
		{
			if (!FFileHelper::LoadFileToString(JSON, *Filename))
			{
				LogFailure(FString::Printf(TEXT("[%s] %s could not be loaded"), *GetValidatorName(), *Filename));
				bValid = false;
				continue;
			}
		}

		TSharedRef<TJsonReader<>> JsonReader = TJsonReaderFactory<>::Create(MoveTemp(JSON));
		TSharedPtr<FJsonValue> JsonValue;
		if (!FJsonSerializer::Deserialize(JsonReader, JsonValue) || !JsonValue.IsValid())
		{
			const FString& Error = JsonReader->GetErrorMessage();
			LogFailure(FString::Printf(TEXT("[%s] %s is an invalid JSON file: %s"), *GetValidatorName(), *Filename, *Error));
			bValid = false;
		}
	}

	return bValid;
}

void FJsonValidator::ParseDefinition(const FString& InDefinition)
{
	FStringOutputDevice Errors;

	FJsonValidatorDefinition* DefinitionToLoad = new FJsonValidatorDefinition;
	FJsonValidatorDefinition::StaticStruct()->ImportText(*InDefinition, DefinitionToLoad, nullptr, 0, &Errors, FJsonValidatorDefinition::StaticStruct()->GetName());

	Definition.Reset(DefinitionToLoad);

	if (!Errors.IsEmpty())
	{
		UE_LOGF(LogSubmitTool, Error, "[%ls] Error loading parameter file %ls", *GetValidatorName(), *Errors);
		FModelInterface::SetErrorState();
	}
}
