// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ValidatorBaseAsync.h"
#include "Tasks/Task.h"

namespace SubmitToolParseConstants
{
	const FString JsonValidator = TEXT("JsonValidator");
}

class FJsonValidator final : public FValidatorBaseAsync
{
public:
	FJsonValidator(const FName& InNameId, const FSubmitToolParameters& InParameters, TSharedRef<FSubmitToolServiceProvider> InServiceProvider, const FString& InDefinition);

	virtual void StartAsyncWork(const FString& CLDescription, const TArray<FSCFileRef>& FilteredFilesInCL, const TArray<const FTag*>& Tags) override;

protected:
	bool ValidateJson(const TArray<FSCFileRef>& InFilteredFilesInCL, const UE::Tasks::FCancellationToken& InCancellationToken) const;
	virtual void ParseDefinition(const FString& InDefinition) override;

	virtual const FString& GetValidatorTypeName() const override
	{
		return SubmitToolParseConstants::JsonValidator;
	}
};
