// Copyright Epic Games, Inc. All Rights Reserved.

#include "PackageDataValidator.h"

#include "Logic/Validators/ValidatorFactory.h"
#include "Misc/StringOutputDevice.h"
#include "Models/ModelInterface.h"
#include "UObject/PackageTrailer.h"
#include "VirtualizationUtilities.h"

REGISTER_VALIDATOR_TYPE(SubmitToolParseConstants::PackageDataValidator, FPackageDataValidator)

FPackageDataValidator::FPackageDataValidator(const FName& InNameId, const FSubmitToolParameters& InParameters, TSharedRef<FSubmitToolServiceProvider> InServiceProvider, const FString& InDefinition)
	: FValidatorBaseAsync(InNameId, InParameters, InServiceProvider, InDefinition)
{
	ParseDefinition(InDefinition);
}

void FPackageDataValidator::ParseDefinition(const FString& InDefinition)
{
	FStringOutputDevice Errors;
	
	FPackageDataValidatorDefinition* DefinitionToLoad = new FPackageDataValidatorDefinition;
	FPackageDataValidatorDefinition::StaticStruct()->ImportText(*InDefinition, DefinitionToLoad, nullptr, 0, &Errors, FPackageDataValidatorDefinition::StaticStruct()->GetName());
	
	Definition.Reset(DefinitionToLoad);

	if (!Errors.IsEmpty())
	{
		UE_LOGF(LogSubmitTool, Error, "[%ls] Error loading parameter file %ls", *GetValidatorName(), *Errors);
		FModelInterface::SetErrorState();
	}
}

void FPackageDataValidator::StartAsyncWork(const FString& InCLDescription, const TArray<FSCFileRef>& InFilteredFilesInCL, const TArray<const FTag*>& InTags)
{
	this->StartAsyncTask([this, InFilteredFilesInCL](const UE::Tasks::FCancellationToken& InCancellationToken) -> bool
	{
		const FPackageDataValidatorDefinition* TypedDefinition = GetTypedDefinition<FPackageDataValidatorDefinition>();
		check(TypedDefinition != nullptr);

		bool bValid = true;

		for (const FSCFileRef& FileState : InFilteredFilesInCL)
		{
			for (const FString& Extension : TypedDefinition->ExcludedExtensions)
			{
				if (InCancellationToken.IsCanceled())
				{
					break;
				}

				FString ProjectFilePath;
				FString PluginFilePath;

				if (UE::Virtualization::Utils::TryFindProject(FileState->GetFilename(), Extension, ProjectFilePath, PluginFilePath))
				{
					// Note we cannot use the asset registry in the SubmitTool as without the proper mount points we can
					// only work with absolute file paths so we need to load the package trailer to tell if the package
					// has virtualized data or not.
					UE::FPackageTrailer Trailer;
					if (UE::FPackageTrailer::TryLoadFromFile(FileState->GetFilename(), Trailer))
					{
						if (Trailer.GetNumPayloads(UE::EPayloadStorageType::Virtualized) > 0)
						{
							LogFailure(FString::Printf(TEXT("[%s] %s has virtualized content and will not work for a '%s' project, please rehydrate!"), *GetValidatorName(), *FileState->GetFilename(), *Extension));
							bValid = false;
						}
					}
				}
			}
		}

		return bValid;
	});
}

