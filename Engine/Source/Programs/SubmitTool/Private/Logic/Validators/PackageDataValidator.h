// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ValidatorBaseAsync.h"

namespace SubmitToolParseConstants
{
	const FString PackageDataValidator = TEXT("PackageDataValidator");
}

class FPackageDataValidator final : public FValidatorBaseAsync
{
public:
	using FValidatorBaseAsync::FValidatorBaseAsync;

	FPackageDataValidator(const FName& InNameId, const FSubmitToolParameters& InParameters, TSharedRef<FSubmitToolServiceProvider> InServiceProvider, const FString& InDefinition);
	virtual ~FPackageDataValidator() = default;

	virtual void StartAsyncWork(const FString& InCLDescription, const TArray<FSCFileRef>& InFilteredFilesInCL, const TArray<const FTag*>& InTags) override;
private:

	virtual void ParseDefinition(const FString& InDefinition) override;

	

	virtual const FString& GetValidatorTypeName() const override
	{
		return SubmitToolParseConstants::PackageDataValidator;
	}
};
