// Copyright Epic Games, Inc. All Rights Reserved.

#include "PreflightValidator.h"

#include "Logic/PreflightService.h"
#include "Logic/Services/Interfaces/ITagService.h"
#include "Logic/Validators/ValidatorFactory.h"
#include "Logging/SubmitToolLog.h"
#include "Internationalization/Regex.h"
#include "Models/ModelInterface.h"
#include "Misc/StringOutputDevice.h"
#include "HAL/PlatformFileManager.h"
#include "GenericPlatform/GenericPlatformFile.h"

REGISTER_VALIDATOR_TYPE(SubmitToolParseConstants::PreflightValidator, FPreflightValidator)

FPreflightValidator::FPreflightValidator(const FName& InNameId, const FSubmitToolParameters& InParameters, TSharedRef<FSubmitToolServiceProvider> InServiceProvider, const FString& InDefinition)
: FValidatorBase(InNameId, InParameters, InServiceProvider, InDefinition)
{
	ParseDefinition(InDefinition);
}

bool FPreflightValidator::Activate()
{
	bIsValidSetup = FValidatorBase::Activate();
	PreflightTag = ServiceProvider.Pin()->GetService<ITagService>()->GetTagOfSubtype(TEXT("preflight"));

	bIsValidSetup = bIsValidSetup && PreflightTag != nullptr;

	return bIsValidSetup;
}


void FPreflightValidator::ParseDefinition(const FString& InDefinition)
{
	FStringOutputDevice Errors;
	Definition = MakeUnique<FPreflightValidatorDefinition>();
	FPreflightValidatorDefinition* ModifiableDefinition = const_cast<FPreflightValidatorDefinition*>(GetTypedDefinition<FPreflightValidatorDefinition>());
	FPreflightValidatorDefinition::StaticStruct()->ImportText(*InDefinition, ModifiableDefinition, nullptr, 0, &Errors, FPreflightValidatorDefinition::StaticStruct()->GetName());

	if (!Errors.IsEmpty())
	{
		UE_LOGF(LogSubmitTool, Error, "[%ls] Error loading parameter file %ls", *GetValidatorName(), *Errors);
		FModelInterface::SetErrorState();
	}
}

bool FPreflightValidator::Validate(const FString& InCLDescription, const TArray<FSCFileRef>& InFilteredFilesInCL, const TArray<const FTag*>& InTags)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPreflightValidator::Validate);
	if(!PreflightTag->IsEnabled() || PreflightTag->GetValues().Num() == 0)
	{
		Skip();
		return true;
	}
	else
	{
		for(const FString& PreflightId : PreflightTag->GetValues())
		{
			if(PreflightId.Equals(TEXT("skip"), ESearchCase::IgnoreCase) || PreflightId.Equals(TEXT("none"), ESearchCase::IgnoreCase))
			{
				// if we have at least one skip or none value, skip the validation altogether
				Skip();
				return true;
			}
		}
	}
	TSharedPtr<FPreflightService> PreflightService = ServiceProvider.Pin()->GetService<FPreflightService>();
	if(!PreflightUpdateHandler.IsValid())
	{
		PreflightUpdateHandler = PreflightService->OnPreflightDataUpdated.AddRaw(this, &FPreflightValidator::ValidatePreflights);
	}

	// If horde connections fails in a way that cannot be recovered, fail the validator
	if (!HordeConnectionFailedHandler.IsValid())
	{
		HordeConnectionFailedHandler = PreflightService->OnHordeConnectionFailed.AddLambda([this](){ ValidationFinished(false); });
	}

	if(!TagUpdateHandler.IsValid())
	{
		TagUpdateHandler = PreflightTag->OnTagUpdated.AddLambda([this, PreflightService](const FTag&){ ValidatePreflights(PreflightService->GetPreflightData(), PreflightService->GetUnlinkedPreflights()); });
	}

	if(PreflightService->GetPreflightData().IsValid())
	{
		if (!PreflightService->SelectPreflightTemplate(SuggestedTemplate))
		{
			SuggestedTemplate.Template = ServiceProvider.Pin()->GetService<FPreflightService>()->GetDefaultPreflightTemplate();
		}
		ValidatePreflights(PreflightService->GetPreflightData(), PreflightService->GetUnlinkedPreflights());
	}

	return true;
}

const FString FPreflightValidator::GetPopupMessageWhenFailedText() const
{
	const FString FailedPreflightProp = TEXT("$(FailedPreflightUrls)");

	if(Definition->PopupMessageWhenFailed.Contains(FailedPreflightProp))
	{
		TSharedPtr<FPreflightService> PreflightService = ServiceProvider.Pin()->GetService<FPreflightService>();
		const TArray<FString>& PreflightValues = PreflightTag->GetValues();

		TStringBuilder<1024> PreflightURLs;

		for (const FPreflightData* PreflightData : PreflightService->GetTaggedPreflights())
		{
			if (PreflightData != nullptr && (PreflightData->CachedResults.State != EPreflightState::Completed || PreflightData->CachedResults.Outcome != EPreflightOutcome::Success))
			{
				if (PreflightURLs.Len() != 0)
				{
					PreflightURLs << TEXT(" - ");
				}
				PreflightURLs << GetURLTextblock(PreflightData->ID);
			}
		}

		return Definition->PopupMessageWhenFailed.Replace(*FailedPreflightProp, PreflightURLs.ToString());
	}

	return Definition->PopupMessageWhenFailed;
}

	const FString FPreflightValidator::GetURLTextblock(const FString& InPFId) const
	{
		return FString::Printf(TEXT("<a id=\"browser\" style=\"Hyperlink\" href=\"%sjob/%s\">%s</>"), *SubmitToolParameters.HordeParameters.HordeServerAddress, *InPFId, *InPFId);
	}

	void FPreflightValidator::ValidatePreflights(const TUniquePtr<FPreflightList>& InPreflightListPtr, const TMap<FString, FPreflightData>& InUnlinkedPreflights)
{
	if (!InPreflightListPtr.IsValid())
	{
		return;
	}

	size_t IgnoredPreflights = 0;
	size_t RunningPreflights = 0;
	TArray<const FPreflightData*, TInlineAllocator<8>> ValidPreflights;
	TArray<const FPreflightData*, TInlineAllocator<8>> FailedPreflights;
	TArray<const FPreflightData*, TInlineAllocator<8>> WarningPreflights;
	TArray<FString, TInlineAllocator<8>> UnverifiedPreflights;

	const TArray<FString>& PreflightValues = PreflightTag->GetValues();
	if(PreflightValues.Num() != 0)
	{
		for(FString PreflightId : PreflightValues)
		{
			if(PreflightId.Equals(TEXT("skip")) || PreflightId.Equals(TEXT("none")))
			{
				// if we have at least one skip or none value, skip the validation altogether
				Skip();
				return;
			}

			if(PreflightId.Contains(TEXT("/")))
			{
				int32 SlashIdx;
				PreflightId.FindLastChar(TCHAR('/'), SlashIdx);
				PreflightId = PreflightId.RightChop(SlashIdx + 1);
			}

			PreflightId.TrimStartAndEndInline();

			FRegexPattern Pattern = FRegexPattern(TEXT("(?:[0-9]|[a-f]){24}"), ERegexPatternFlags::CaseInsensitive);
			FRegexMatcher regex = FRegexMatcher(Pattern, PreflightId);
			bool match = regex.FindNext();
			if(match)
			{
				// Look into the linked preflights and if we don't find it check the unlinked preflights
				const FPreflightData* FoundData = InPreflightListPtr->PreflightList.FindByPredicate([&PreflightId](const FPreflightData& InData) { return InData.ID == PreflightId; }); 
				if(FoundData == nullptr)
				{
					FoundData = InUnlinkedPreflights.Find(PreflightId);
				}

				if(FoundData != nullptr && Definition != nullptr)
				{
					if(Definition->bInvalidatesWhenOutOfDate)
					{
						IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
						for(const FSCFileRef& file : ServiceProvider.Pin()->GetService<IChangelistService>()->GetFilesInCL())
						{
							const FDateTime FileTimestampUTC = PlatformFile.GetTimeStamp(*file->GetFilename());
							if(FileTimestampUTC > FoundData->CreateTime)
							{
								LogFailure(FString::Printf(TEXT("[%s] %s is out of date! Please run a new preflight with the newest set of files."), *GetValidatorName(), *PreflightId));
								FailedPreflights.Add(FoundData);
								continue;
							}
						}
					}

					FTimespan PreflightAge = FDateTime::UtcNow() - FoundData->UpdateTime;
					switch(FoundData->CachedResults.State)
					{
						case EPreflightState::Running:
						case EPreflightState::Ready:
							++RunningPreflights;
							break;

						case EPreflightState::Completed:
							if (PreflightAge.GetTotalHours() > GetTypedDefinition<FPreflightValidatorDefinition>()->MaxPreflightAgeInHours)
							{
								LogFailure(FString::Printf(TEXT("[%s] Preflight %s is %d hours old, submitting changes with preflights older than %d hours is not recommended."), *GetValidatorName(), *PreflightId, FMath::FloorToInt(PreflightAge.GetTotalHours()), GetTypedDefinition<FPreflightValidatorDefinition>()->MaxPreflightAgeInHours));
								WarningPreflights.Add(FoundData);
							}

							switch(FoundData->CachedResults.Outcome)
							{
								case EPreflightOutcome::Success:
									FoundData->CachedResults.WasSuccessful() ? ValidPreflights.Add(FoundData) : FailedPreflights.Add(FoundData);
									break;
								case EPreflightOutcome::Unspecified:
								case EPreflightOutcome::Failure:
									FailedPreflights.Add(FoundData);
									break;
								case EPreflightOutcome::Warnings:
									if(Definition->bTreatWarningsAsErrors)
									{
										LogFailure(FString::Printf(TEXT("[%s] %s preflight has completed with warnings and they are treated as errors."), *GetValidatorName(), *PreflightId));
										FailedPreflights.Add(FoundData);
									}
									else
									{
										if (!WarningPreflights.Contains(FoundData))
										{
											WarningPreflights.Add(FoundData);
										}
										ValidPreflights.Add(FoundData);
									}
									break;
							}
							break;

						case EPreflightState::Skipped:
						case EPreflightState::Unspecified:
							FailedPreflights.Add(FoundData);
							break;

					}
				}
				else
				{
					UnverifiedPreflights.Add(PreflightId);
				}
			}
			else
			{
				UE_LOGF(LogValidators, Log, "[%ls] Tag value '%ls' is not a valid preflight id or the preflight list is empty.", *GetValidatorName(), *PreflightId);
				++IgnoredPreflights;
			}
		}
	}
	else
	{
		Skip();
		return;
	}

	if (RunningPreflights > 0)
	{
		UE_LOGF(LogValidators, Log, "[%ls] Periodically checking updated horde state... If you see your PF has finished in horde you can force a refresh in the preflight tag refresh button", *GetValidatorName());
		UE_LOGF(LogValidatorsResult, Log, "[%ls] Periodically checking updated horde state... If you see your PF has finished in horde you can force a refresh in the preflight tag refresh button", *GetValidatorName());
	}

	// Fail early even if other PF are running
	if(!FailedPreflights.IsEmpty())
	{
		for(const FPreflightData* PreflightPtr : FailedPreflights)
		{
			LogFailure(FString::Printf(TEXT("[%s] %s preflight has failed with errors."), *GetValidatorName(), *PreflightPtr->ID));

			for(const FString& ErrorString : PreflightPtr->CachedResults.Errors)
			{
				LogFailure(FString::Printf(TEXT("[%s] Reported error: %s"), *GetValidatorName(), *ErrorString));
			}
		}

		ValidationFinished(false);
	}
	else if(RunningPreflights == 0)
	{
		for (const FPreflightData* PreflightPtr : WarningPreflights)
		{
			UE_LOGF(LogValidators, Warning, "[%ls] %ls preflight has warnings.", *GetValidatorName(), *PreflightPtr->ID);
			UE_LOGF(LogValidatorsResult, Warning, "[%ls] %ls preflight has warnings.", *GetValidatorName(), *PreflightPtr->ID);
		}

		for(const FString& PreflightId : UnverifiedPreflights)
		{
			UE_LOGF(LogValidators, Error, "[%ls] %ls preflight can't be verified with Horde, check that the id is correct and there are no connection errors.", *GetValidatorName(), *PreflightId);
			UE_LOGF(LogValidatorsResult, Error, "[%ls] %ls preflight can't be verified with Horde, check that the id is correct and there are no connection errors.", *GetValidatorName(), *PreflightId);
		}		

		for(const FPreflightData* PreflightPtr : ValidPreflights)
		{
			if (!SuggestedTemplate.Template.IsEmpty() && SuggestedTemplate.Template != ServiceProvider.Pin()->GetService<FPreflightService>()->GetDefaultPreflightTemplate() && PreflightPtr->TemplateId != SuggestedTemplate.Template)
			{
				UE_LOGF(LogValidators, Warning, "[%ls] %ls preflight used template %ls, submit tool recommended preflight for your CL was %ls, make sure your changes are covered by the %ls preflight", *GetValidatorName(), *PreflightPtr->ID, *PreflightPtr->TemplateId, *SuggestedTemplate.Template, *PreflightPtr->TemplateId);
				UE_LOGF(LogValidatorsResult, Warning, "[%ls] %ls preflight used template %ls, submit tool recommended preflight for your CL was %ls, make sure your changes are covered by the %ls preflight", *GetValidatorName(), *PreflightPtr->ID, *PreflightPtr->TemplateId, *SuggestedTemplate.Template, *PreflightPtr->TemplateId);
			}

			UE_LOGF(LogValidators, Log, "[%ls] %ls preflight is valid and has succeeded", *GetValidatorName(), *PreflightPtr->ID);
			UE_LOGF(LogValidatorsResult, Log, "[%ls] %ls preflight is valid and has succeeded", *GetValidatorName(), *PreflightPtr->ID);
		}

		ValidationFinished(ValidPreflights.Num() == (PreflightValues.Num() - IgnoredPreflights));
	}
}

void FPreflightValidator::Skip()
{
	RemoveCallbacks();
	FValidatorBase::Skip();
}

void FPreflightValidator::RemoveCallbacks()
{
	if (PreflightUpdateHandler.IsValid())
	{
		ServiceProvider.Pin()->GetService<FPreflightService>()->OnPreflightDataUpdated.Remove(PreflightUpdateHandler);
		PreflightUpdateHandler.Reset();
	}

	if (HordeConnectionFailedHandler.IsValid())
	{
		ServiceProvider.Pin()->GetService<FPreflightService>()->OnHordeConnectionFailed.Remove(HordeConnectionFailedHandler);
		HordeConnectionFailedHandler.Reset();
	}

	if(TagUpdateHandler.IsValid())
	{
		PreflightTag->OnTagUpdated.Remove(TagUpdateHandler);
		TagUpdateHandler.Reset();
	}
}

void FPreflightValidator::ValidationFinished(bool bSuccess)
{
	RemoveCallbacks();
	FValidatorBase::ValidationFinished(bSuccess);
}
