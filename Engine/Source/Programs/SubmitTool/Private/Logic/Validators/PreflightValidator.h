// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ValidatorBase.h"
#include "Models/PreflightData.h"

class FPreflightService;

namespace SubmitToolParseConstants
{
	const FString PreflightValidator = TEXT("PreflightValidator");
}

class FPreflightValidator : public FValidatorBase
{
public:
	FPreflightValidator(const FName& InNameId, const FSubmitToolParameters& InParameters, TSharedRef<FSubmitToolServiceProvider> InServiceProvider, const FString& InDefinition);

	virtual void ParseDefinition(const FString& InDefinition) override;
	virtual bool Activate() override;

	virtual bool Validate(const FString& InCLDescription, const TArray<FSCFileRef>& InFilteredFilesInCL, const TArray<const FTag*>& InTags) override;
	virtual const FString& GetValidatorTypeName() const override { return SubmitToolParseConstants::PreflightValidator; }

	virtual const FString GetPopupMessageWhenFailedText() const override;
protected:
	const FString GetURLTextblock(const FString& InPFId) const;

	void ValidatePreflights(const TUniquePtr<FPreflightList>& InPreflightList, const TMap<FString, FPreflightData>& InUnlinkedPreflights);
	virtual void Skip() override;

	void RemoveCallbacks();

	FTag* PreflightTag; 

	FPreflightTemplateDefinition SuggestedTemplate;

	virtual void ValidationFinished(bool bSuccess) override;
	FDelegateHandle PreflightUpdateHandler;
	FDelegateHandle HordeConnectionFailedHandler;
	FDelegateHandle TagUpdateHandler;
};
