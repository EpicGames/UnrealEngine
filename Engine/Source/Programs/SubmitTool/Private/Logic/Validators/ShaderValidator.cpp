// Copyright Epic Games, Inc. All Rights Reserved.

#include "ShaderValidator.h"

// This is commented out for the moment due to this validator not being in used and having a 20MB cost in the package size due to requiring the RenderCore module as a dependency.
// it should eventually be split off as a plugin


//#include "Misc/FileHelper.h"
//#include "Misc/Paths.h"
//#include "ShaderCompilerCommon.h"
//
//#include "CommandLine/CmdLineParameters.h"
//
//#include "Configuration/Configuration.h"
//#include "Logging/SubmitToolLog.h"
//#include "Models/ModelInterface.h"
//#include "Models/SubmitToolUserPrefs.h"


FShaderValidator::FShaderValidator(const FName& InNameId, const FSubmitToolParameters& InParameters, TSharedRef<FSubmitToolServiceProvider> InServiceProvider, const FString& InDefinition) :
	FValidatorBaseAsync(InNameId, InParameters, InServiceProvider, InDefinition)
{
}

void FShaderValidator::StartAsyncWork(const FString& InCLDescription, const TArray<FSCFileRef>& InFilteredFilesInCL, const TArray<const FTag*>& InTags)
{
	/*this->StartAsyncTask([this, InFilteredFilesInCL](const UE::Tasks::FCancellationToken& InCancellationToken) -> bool
	{
		bool bValid = true;

		for (const FSCFileRef& File : InFilteredFilesInCL)
		{
			if (InCancellationToken.IsCanceled())
			{
				break;
			}

			const FString& Filename = File->GetFilename();

			FString ShaderSourceCode;
			if (!FFileHelper::LoadFileToString(ShaderSourceCode, *Filename))
			{
				LogFailure(FString::Printf(TEXT("[%s] %s could not be loaded"), *GetValidatorName(), *Filename));
				bValid = false;
				continue;
			}

			TArray<FString> Errors;
			if (!UE::ShaderCompilerCommon::ValidateShaderAgainstKnownIssues(ShaderSourceCode, Errors, *Filename))
			{
				TStringBuilder<4096> ErrorStringBuilder;
				for (const FString& Error : Errors)
				{
					ErrorStringBuilder.Appendf(TEXT("\n%s"), *Error);
				}
				LogFailure(FString::Printf(TEXT("[%s] %s has validation errors:%s"), *GetValidatorName(), *Filename, ErrorStringBuilder.ToString()));
				bValid = false;
			}
		}

		return bValid;
	});*/
}
