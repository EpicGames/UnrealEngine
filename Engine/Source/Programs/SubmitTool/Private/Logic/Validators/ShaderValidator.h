// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ValidatorBaseAsync.h"

namespace SubmitToolParseConstants
{
	const FString ShaderValidator = TEXT("ShaderValidator");
}

class FShaderValidator final : public FValidatorBaseAsync
{
public:		
	FShaderValidator(const FName& InNameId, const FSubmitToolParameters& InParameters, TSharedRef<FSubmitToolServiceProvider> InServiceProvider, const FString& InDefinition);
	virtual ~FShaderValidator() = default;

	virtual void StartAsyncWork(const FString& InCLDescription, const TArray<FSCFileRef>& InFilteredFilesInCL, const TArray<const FTag*>& InTags) override;

	virtual const FString& GetValidatorTypeName() const override
	{
		return SubmitToolParseConstants::ShaderValidator;
	}
};
