// Copyright Epic Games, Inc. All Rights Reserved.

#include "TagValidator.h"
#include "Models/Tag.h"
#include "Logging/SubmitToolLog.h"
#include "Logic/Services/Interfaces/ISTSourceControlService.h"
#include "Logic/Services/Interfaces/IChangelistService.h"
#include "Logic/Validators/ValidatorFactory.h"
#include "Internationalization/Regex.h"

REGISTER_VALIDATOR_TYPE(SubmitToolParseConstants::TagValidator, FTagValidator)

bool FTagValidator::Validate(const FString& InCLDescription, const TArray<FSCFileRef>& InFilteredFilesInCL, const TArray<const FTag*>& InTags)
{
	bResult = true;
	bHasFinished = false;

	for (const FString& StringToFind : SubmitToolParameters.GeneralParameters.ForbiddenDescriptions)
	{
		if(InCLDescription.TrimStartAndEnd().StartsWith(StringToFind, ESearchCase::IgnoreCase))
		{
			LogFailure(FString::Printf(TEXT("[%s] Please replace the generated description text \"%s\" providing a useful description..."), *GetValidatorName(), *StringToFind));
			bResult = false;
		}
	}

	TArray<FString> DescriptionLines;
	InCLDescription.ParseIntoArrayLines(DescriptionLines, true);
	bool bHasDescription = false;
	for (const FString& Line : DescriptionLines)
	{
		if (!Line.TrimStart().StartsWith(TEXT("#")))
		{
			bHasDescription = true;
			break;
		}
	}

	if (!bHasDescription)
	{
		LogFailure(FString::Printf(TEXT("[%s] Please introduce at least one description line that doesn't start with a tag"), *GetValidatorName()));
		bResult = false;
	}

	// Wait for the GetUsersAndGroups asynchronously to finish validation
	ServiceProvider.Pin()->GetService<ISTSourceControlService>()->GetUsersAndGroups(FOnUsersAndGroupsGet::FDelegate::CreateLambda([this, InTags](TArray<TSharedPtr<FUserData>>& InP4Users, TArray<TSharedPtr<FString>>& InP4Groups)
		{
			for (const FTag* tag : InTags)
			{
				if (tag->IsEnabled() && tag->Definition.InputType.Equals("PerforceUser"))
				{
					P4UserTags.Emplace(tag);
				}

				if (ValidateTag(tag, InP4Users, InP4Groups))
				{
					tag->SetTagState(ETagState::Success);
				}
				else
				{
					tag->SetTagState(ETagState::Failed);
					bResult = false;
				}
			}

			bHasFinished = true;
		}));

	return true;
}

bool FTagValidator::ValidateTag(const FTag* InTag, const TArray<TSharedPtr<FUserData>>& InP4Users, const TArray<TSharedPtr<FString>>& InP4Groups) const
{
	bool bIsTagValid = true;

	FTagValidationConfig ValidationConfig = InTag->GetCurrentValidationConfig(ServiceProvider.Pin()->GetService<IChangelistService>()->GetFilesInCL());

	if(ValidationConfig.bIsMandatory && !InTag->IsEnabled())
	{
		if (ValidationConfig.RegexErrorMessage.IsEmpty())
		{
			LogFailure(FString::Printf(TEXT("[%s] Tag %s is mandatory"), *GetValidatorName(), *InTag->Definition.GetTagId()));
		}
		else
		{
			LogFailure(FString::Printf(TEXT("[%s] Tag %s is mandatory: %s"), *GetValidatorName(), *InTag->Definition.GetTagId(), *ValidationConfig.RegexErrorMessage));
		}

		bIsTagValid = false;
	}

	if(InTag->IsEnabled())
	{
		if(!(InTag->GetValues().Num() >= InTag->Definition.MinValues && InTag->GetValues().Num() <= InTag->Definition.MaxValues))
		{
			LogFailure(FString::Printf(TEXT("[%s] Tag %s needs to have between %d and %d values"), *GetValidatorName(), *InTag->Definition.GetTagId(), InTag->Definition.MinValues, InTag->Definition.MaxValues));
			bIsTagValid = false;
		}

		if (InTag->Definition.InputType.Equals("PerforceUser"))
		{
			TUniquePtr<FRegexPattern> Pattern = nullptr;

			if (!ValidationConfig.RegexValidation.IsEmpty())
			{
				Pattern = MakeUnique<FRegexPattern>(ValidationConfig.RegexValidation, ERegexPatternFlags::CaseInsensitive);
			}

			bool bValidUsers = true;
			for (const FString& Value : InTag->GetValues())
			{
				if (Pattern.IsValid())
				{
					FRegexMatcher regex = FRegexMatcher(*Pattern, Value);
					bool match = regex.FindNext();
					if (match)
					{
						continue;
					}
				}

				if (InP4Users.IsEmpty() && InP4Groups.IsEmpty())
				{
					LogFailure(FString::Printf(TEXT("[%s] P4 User list is empty and couldn't be used to validate Tag %s"), *GetValidatorName(), *InTag->Definition.GetTagId()));
					bIsTagValid = false;
				}
				else
				{
					const FString TrimmedValue = Value.TrimChar(TCHAR('@'));
					if (!InP4Users.ContainsByPredicate([&TrimmedValue](const TSharedPtr<FUserData>& UserData) { return UserData->Username.Equals(TrimmedValue, ESearchCase::IgnoreCase); })
						&& !InP4Groups.ContainsByPredicate([&TrimmedValue](const TSharedPtr<FString>& Group) { return (*Group).Equals(TrimmedValue, ESearchCase::IgnoreCase); }))
					{
						LogFailure(FString::Printf(TEXT("[%s] Value '%s' of Tag %s is not a valid perforce username or group"), *GetValidatorName(), *Value, *InTag->Definition.GetTagId()));
						bIsTagValid = false;
					}
				}
			}

			if (!bIsTagValid && !ValidationConfig.RegexErrorMessage.IsEmpty())
			{
				LogFailure(FString::Printf(TEXT("[%s] Tag %s doesn't match the regex validation: %s"), *GetValidatorName(), *InTag->Definition.GetTagId(), *ValidationConfig.RegexErrorMessage));
			}
		}
		else
		{
			if (!ValidationConfig.RegexValidation.IsEmpty())
			{
				FRegexPattern Pattern = FRegexPattern(ValidationConfig.RegexValidation, ERegexPatternFlags::CaseInsensitive);
				for (const FString& Value : InTag->GetValues())
				{
					FRegexMatcher regex = FRegexMatcher(Pattern, Value);
					bool match = regex.FindNext();
					if (!match)
					{
						LogFailure(FString::Printf(TEXT("[%s] Value '%s' of Tag %s doesn't match the regex validation: %s"), *GetValidatorName(), *Value, *InTag->Definition.GetTagId(), *ValidationConfig.RegexErrorMessage));
						bIsTagValid = false;
					}
				}
			}
		}

		if(bIsTagValid)
		{
			UE_LOGF(LogValidators, Log, "[%ls] Tag %ls is valid", *GetValidatorName(), *InTag->Definition.GetTagId());
		}
	}

	return bIsTagValid;
}

void FTagValidator::Tick(float InDeltaTime)
{
	FValidatorBase::Tick(InDeltaTime);

	if (bHasFinished)
	{
		ValidationFinished(bResult);
	}
}
