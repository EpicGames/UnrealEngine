// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ValidatorBase.h"
struct FUserData;
class FTag;

namespace SubmitToolParseConstants
{
	const FString TagValidator = TEXT("TagValidator");
}

class FTagValidator : public FValidatorBase
{
public:
	using FValidatorBase::FValidatorBase;

	virtual bool Validate(const FString& InCLDescription, const TArray<FSCFileRef>& InFilteredFilesInCL, const TArray<const FTag*>& InTags) override;
	virtual const FString& GetValidatorTypeName() const override { return SubmitToolParseConstants::TagValidator; }

protected:
	bool ValidateTag(const FTag* InTag, const TArray<TSharedPtr<FUserData>>& InP4Users, const TArray<TSharedPtr<FString>>& InP4Groups) const;

	virtual void Tick(float InDeltaTime) override;

private:
	TArray<const FTag*> P4UserTags;
	bool bHasFinished = false;
	bool bResult = false;
};
