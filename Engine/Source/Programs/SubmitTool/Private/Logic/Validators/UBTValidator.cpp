// Copyright Epic Games, Inc. All Rights Reserved.

#include "UBTValidator.h"

#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Misc/StringOutputDevice.h"

#include "CommandLine/CmdLineParameters.h"

#include "Configuration/Configuration.h"
#include "Logic/Validators/ValidatorFactory.h"
#include "Logging/SubmitToolLog.h"
#include "Models/ModelInterface.h"
#include "Models/SubmitToolUserPrefs.h"

REGISTER_VALIDATOR_TYPE(SubmitToolParseConstants::UBTValidator, FUBTValidator)

FUBTValidator::FUBTValidator(const FName& InNameId, const FSubmitToolParameters& InParameters, TSharedRef<FSubmitToolServiceProvider> InServiceProvider, const FString& InDefinition) :
	FValidatorRunExecutable(InNameId, InParameters, InServiceProvider, InDefinition)
{
	ParseDefinition(InDefinition);
}
void FUBTValidator::ParseDefinition(const FString& InDefinition)
{
	FStringOutputDevice Errors;
	Definition = MakeUnique<FUBTValidatorDefinition>();
	FUBTValidatorDefinition* ModifyableDefinition = const_cast<FUBTValidatorDefinition*>(GetTypedDefinition<FUBTValidatorDefinition>());
	FUBTValidatorDefinition::StaticStruct()->ImportText(*InDefinition, ModifyableDefinition, nullptr, 0, &Errors, FUBTValidatorDefinition::StaticStruct()->GetName());

	if(!Errors.IsEmpty())
	{
		UE_LOGF(LogSubmitTool, Error, "[%ls] Error loading parameter file %ls", *GetValidatorName(), *Errors);
		FModelInterface::SetErrorState();
	}
}

bool FUBTValidator::Validate(const FString& InCLDescription, const TArray<FSCFileRef>& InFilteredFilesInCL, const TArray<const FTag*>& InTags)
{
	const FUBTValidatorDefinition* TypedDefinition = GetTypedDefinition<FUBTValidatorDefinition>();

	// get the list of files to validate
	TArray<FString> Files;

	for(const FSCFileRef& File : InFilteredFilesInCL)
	{
		Files.Add(File->GetFilename());
	}

	TArray<FString> NotProgramFiles;
	TMap<FString, TArray<FString>> ProgramFiles;
	FilterProgramFiles(Files, NotProgramFiles, ProgramFiles);

	// list of final targets arguments to pass to UBT
	TArray<FString> Targets;
	const FString TargetEngineDir = FPaths::Combine(FConfiguration::Substitute(TEXT("$(root)")), TEXT("Engine"));

	const FString SubmitToolIntermediateDir = FPaths::EngineIntermediateDir();

	FString Platform = TypedDefinition->Platform;
	if(TypedDefinition->Platforms.Num() != 0)
	{
		FString SelectedOption = OptionsProvider.GetSelectedOptionKey(PlatformOptions);
		if(!TypedDefinition->Platforms.Contains(SelectedOption))
		{
			UE_LOGF(LogValidators, Error, "[%ls] Selected option %ls is not contained in the Platforms:\n%ls", *GetValidatorName(), *SelectedOption, *FString::Join(TypedDefinition->Platforms, TEXT("\n")));
			return false;
		}

		Platform = SelectedOption;
	}

	FString Configuration = TypedDefinition->Configuration;
	if(TypedDefinition->Configurations.Num() != 0)
	{
		FString SelectedOption = OptionsProvider.GetSelectedOptionKey(ConfigurationOptions);
		if(!TypedDefinition->Configurations.Contains(SelectedOption))
		{
			UE_LOGF(LogValidators, Error, "[%ls] Selected option %ls is not contained in the Configurations:\n%ls", *GetValidatorName(), *SelectedOption, *FString::Join(TypedDefinition->Configurations, TEXT("\n")));
			return false;
		}

		Configuration = SelectedOption;
	}

	FString Target = TypedDefinition->Target;
	if(TypedDefinition->Targets.Num() != 0)
	{
		FString SelectedOption = OptionsProvider.GetSelectedOptionKey(TargetOptions);
		if(!TypedDefinition->Targets.Contains(SelectedOption))
		{
			UE_LOGF(LogValidators, Error, "[%ls] Selected option %ls is not contained in the Targets:\n%ls", *GetValidatorName(), *SelectedOption, *FString::Join(TypedDefinition->Targets, TEXT("\n")));
			return false;
		}

		Target = SelectedOption;
	}

	FString Arguments = TypedDefinition->ExecutableArguments;
	if(TypedDefinition->bUseStaticAnalyser)
	{
		FString StaticAnalyzser = TypedDefinition->StaticAnalyser;
		if(TypedDefinition->StaticAnalysers.Num() > 0)
		{
			FString SelectedOption = OptionsProvider.GetSelectedOptionKey(StaticAnalyserOptions);
			if(!TypedDefinition->StaticAnalysers.Contains(SelectedOption))
			{
				UE_LOGF(LogValidators, Error, "[%ls] Selected option %ls is not contained in the StaticAnalysers:\n%ls", *GetValidatorName(), *SelectedOption, *FString::Join(TypedDefinition->StaticAnalysers, TEXT("\n")));
				return false;
			}

			StaticAnalyzser = SelectedOption;
		}

		Arguments = FString::Printf(TEXT("%s %s%s"), *TypedDefinition->ExecutableArguments, *TypedDefinition->StaticAnalyserArg, *StaticAnalyzser);
	}
	

	// add the file lists for the programs
	if(!ProgramFiles.IsEmpty())
	{
		for(const TPair<FString, TArray<FString>>& ProgramFileList : ProgramFiles)
		{
			FString ProgramFileListPath = CreateFileList(ProgramFileList.Value, SubmitToolIntermediateDir + TEXT("SubmitTool/FileLists/"));

			FString TargetStr = FString::Printf(TEXT("%s %s %s %s"),
				*ProgramFileList.Key,
				*Platform,
				*Configuration,
				*Arguments);

			TSet<FString> ProgramProjectFiles = ExtractUprojectFiles(ProgramFileList.Value);
			if (ProgramProjectFiles.Num() > 0)
			{
				if (ProgramProjectFiles.Num() > 1)
				{
					UE_LOGF(LogSubmitTool, Warning, "More than one uprojects found for Program %ls, using the first one of:\n%ls", *ProgramFileList.Key, *FString::Join(ProgramProjectFiles, TEXT("\n")));
				}

				for (const FString& Project : ProgramProjectFiles)
				{
					TargetStr = FString::Printf(TEXT("%s %s\"%s\""),
						*TargetStr,
						*TypedDefinition->ProjectArgument,
						*Project);

					break;
				}
			}

			TargetStr = FString::Printf(TEXT("%s %s\"%s\""),
				*TargetStr,
				*TypedDefinition->FileListArgument,
				*ProgramFileListPath);

			UE_LOGF(LogValidators, Log, "[%ls] Using Target: %ls", *GetValidatorName(), *TargetStr);
			Targets.Add(TargetStr);
		}
	}

	// extract the uprojects files
	TSet<FString> ProjectFiles = ExtractUprojectFiles(NotProgramFiles);

	// add the file list for each collected uproject
	TArray<FString> ProjectDirs;
	for (const FString& ProjectFile : ProjectFiles)
	{
		FString ProjectDir = FPaths::GetPath(ProjectFile);
		ProjectDirs.Add(ProjectDir);
		TArray<FString> ProjectFilteredFiles = FilterFiles(NotProgramFiles, ProjectDir, TArray<FString>());
		if (!ProjectFilteredFiles.IsEmpty())
		{

			FString ProjectFileListPath = CreateFileList(ProjectFilteredFiles, SubmitToolIntermediateDir + TEXT("SubmitTool/FileLists/"));
			FString TargetStr = FString::Printf(TEXT("%s %s -TargetType=%s %s %s\"%s\" %s\"%s\""),
				*Platform,
				*Configuration,
				*Target,
				*Arguments,
				*TypedDefinition->ProjectArgument,
				*ProjectFile,
				*TypedDefinition->FileListArgument,
				*ProjectFileListPath);
			UE_LOGF(LogValidators, Log, "[%ls] Using Target: %ls", *GetValidatorName(), *TargetStr);
			Targets.Add(TargetStr);
		}
	}

	// add the file list for the engine folder
	TArray<FString> EngineFilteredFiles = FilterFiles(NotProgramFiles, TargetEngineDir, ProjectDirs);
	if(!EngineFilteredFiles.IsEmpty())
	{
		FString EngineFileListPath = CreateFileList(EngineFilteredFiles, SubmitToolIntermediateDir + TEXT("SubmitTool/FileLists/"));
		FString TargetStr = FString::Printf(TEXT("%s %s -TargetType=%s %s %s\"%s\""),
			*Platform,
			*Configuration,
			*Target,
			*Arguments,
			*TypedDefinition->FileListArgument,
			*EngineFileListPath);
		UE_LOGF(LogValidators, Log, "[%ls] Using Target: %ls", *GetValidatorName(), *TargetStr);
		Targets.Add(TargetStr);
	}


	// create a TargetFile with all the file lists
	const FString TargetListPath = CreateFileList(Targets, SubmitToolIntermediateDir + TEXT("SubmitTool/TargetLists/"));
	const FString FinalArgs = FString::Printf(TEXT("%s\"%s\""), *TypedDefinition->TargetListArgument, *TargetListPath);

	FString ExecutablePath = TypedDefinition->ExecutablePath;
	if(TypedDefinition->ExecutableCandidates.Num() != 0)
	{
		ExecutablePath = OptionsProvider.GetSelectedOptionValue(ExecutableOptions);
	}

	const FString AbsoluteExecutablePath = FConfiguration::SubstituteAndNormalizeFilename(ExecutablePath);

	return QueueProcess(GetProcessName(AbsoluteExecutablePath), AbsoluteExecutablePath, FinalArgs);
}

bool FUBTValidator::Activate()
{
	bIsValidSetup = FValidatorRunExecutable::Activate();

	const FUBTValidatorDefinition* TypedDefinition = GetTypedDefinition<FUBTValidatorDefinition>();

	PrepareUBTOptions();
	PrepareExecutableOptions();

	bIsValidSetup = bIsValidSetup && !TypedDefinition->ProjectArgument.IsEmpty();

	bIsValidSetup = bIsValidSetup && !TypedDefinition->Configuration.IsEmpty();
	bIsValidSetup = bIsValidSetup && !TypedDefinition->Platform.IsEmpty();

	if(TypedDefinition->bUseStaticAnalyser)
	{
		bIsValidSetup = bIsValidSetup && !TypedDefinition->StaticAnalyserArg.IsEmpty();
		bIsValidSetup = bIsValidSetup && (!TypedDefinition->StaticAnalyser.IsEmpty() || TypedDefinition->StaticAnalysers.Num() > 0);
	}

	return bIsValidSetup;
}

TSet<FString> FUBTValidator::ExtractUprojectFiles(const TArray<FString>& InFiles)
{
	TSet<FString> CheckedDirectories;
	TSet<FString> ProjectFiles;

	for(const FString& File : InFiles)
	{
		FString CurrentDir = FPaths::GetPath(File);

		while(!CurrentDir.IsEmpty())
		{
			bool bIsAlreadyInSet = false;
			CheckedDirectories.Add(CurrentDir, &bIsAlreadyInSet);
			if(bIsAlreadyInSet)
			{
				break;
			}

			TArray<FString> Projects;
			IFileManager::Get().FindFiles(Projects, *(CurrentDir / TEXT("*.uproject")), true, false);

			for(const FString& Project : Projects)
			{
				ProjectFiles.Add(CurrentDir + "/" + Project);
			}

			CurrentDir = FPaths::GetPath(CurrentDir);
		}
	}

	return ProjectFiles;
}

FString FUBTValidator::CreateFileList(const TArray<FString>& InFiles, const FString& InDirectory)
{
	FGuid guid = FGuid::NewGuid();
	FString FileListPath = FPaths::ConvertRelativePathToFull(InDirectory + guid.ToString(EGuidFormats::DigitsWithHyphens) + TEXT(".txt"));

	FFileHelper::SaveStringArrayToFile(InFiles, *FileListPath, FFileHelper::EEncodingOptions::AutoDetect, &IFileManager::Get(), EFileWrite::FILEWRITE_None);

	return FileListPath;
}

TArray<FString> FUBTValidator::FilterFiles(const TArray<FString>& InFiles, const FString& InDirectory, const TArray<FString>& InExcludedDirectories)
{
	TArray<FString> Output;

	FString Directory{ InDirectory };
	FPaths::NormalizeDirectoryName(Directory);

	for(const FString& InFile : InFiles)
	{
		FString File{ InFile };
		FPaths::NormalizeFilename(File);

		bool bSkip = false;
		for(const FString& ExDir : InExcludedDirectories)
		{			
			if(File.StartsWith(*ExDir, ESearchCase::IgnoreCase))
			{
				bSkip = true;
				break;
			}
		}

		if(!bSkip && File.StartsWith(*Directory, ESearchCase::IgnoreCase))
		{
			Output.Add(File);
		}
	}

	return Output;
}

void FUBTValidator::FilterProgramFiles(const TArray<FString>& InFiles, TArray<FString>& OutNotProgramFiles, TMap<FString, TArray<FString>>& OutProgramFiles)
{
	FString SourceDirectory = TEXT("Source/Programs");
	FPaths::NormalizeDirectoryName(SourceDirectory);

	for(const FString& File : InFiles)
	{
		// if the file is not under the source directory
		if(!File.Contains(SourceDirectory))
		{
			OutNotProgramFiles.Add(File);
			continue;
		}
		else
		{
			FString CurrentDir = FPaths::GetPath(File);
			FPaths::NormalizeDirectoryName(CurrentDir);

			bool bBuildfound = false;
			while(!CurrentDir.IsEmpty())
			{
				TArray<FString> BuildFiles;
				IFileManager::Get().FindFiles(BuildFiles, *(CurrentDir / TEXT("*.Build.cs")), true, false);

				TArray<FString> TargetFiles;
				IFileManager::Get().FindFiles(TargetFiles, *(CurrentDir / TEXT("*.Target.cs")), true, false);

				if(BuildFiles.Num() > 0)
				{
					bBuildfound = true;
				}

				if(bBuildfound && TargetFiles.Num() > 0)
				{
					for(const FString& Target : TargetFiles)
					{
						OutProgramFiles.FindOrAdd(Target.Replace(TEXT(".Target.cs"), TEXT(""))).Add(File);
					}
					break;
				}

				CurrentDir = FPaths::GetPath(CurrentDir);
			}
		}
	}
}

void FUBTValidator::PrepareUBTOptions()
{
	const FUBTValidatorDefinition* TypedDefinition = GetTypedDefinition<FUBTValidatorDefinition>();

	if(TypedDefinition->Platforms.Num() > 0)
	{
		TMap<FString, FString> Options;
		Options.Reserve(TypedDefinition->Platforms.Num());

		FString* UserSelectedOption = FSubmitToolUserPrefs::Get()->ValidatorOptions.Find(OptionsProvider.GetUserPrefsKey(PlatformOptions));
		FString SelectedOption = UserSelectedOption != nullptr ? *UserSelectedOption : TypedDefinition->Platform;
		for(const FString& Platform : TypedDefinition->Platforms)
		{
			Options.Add(Platform, Platform);

			if(SelectedOption.IsEmpty())
			{
				SelectedOption = Platform;
			}
		}
		OptionsProvider.InitializeValidatorOptions(PlatformOptions, Options, SelectedOption, EValidatorOptionType::Standard);
	}

	if(TypedDefinition->Configurations.Num() > 0)
	{
		TMap<FString, FString> Options;
		Options.Reserve(TypedDefinition->Configurations.Num());

		FString* UserSelectedOption = FSubmitToolUserPrefs::Get()->ValidatorOptions.Find(OptionsProvider.GetUserPrefsKey(ConfigurationOptions));
		FString SelectedOption = UserSelectedOption != nullptr ? *UserSelectedOption : TypedDefinition->Configuration;
		for(const FString& Configuration : TypedDefinition->Configurations)
		{
			Options.Add(Configuration, Configuration);

			if(SelectedOption.IsEmpty())
			{
				SelectedOption = Configuration;
			}
		}
		OptionsProvider.InitializeValidatorOptions(ConfigurationOptions, Options, SelectedOption, EValidatorOptionType::Standard);
	}

	if(TypedDefinition->Targets.Num() > 0)
	{
		TMap<FString, FString> Options;
		Options.Reserve(TypedDefinition->Targets.Num());

		FString* UserSelectedOption = FSubmitToolUserPrefs::Get()->ValidatorOptions.Find(OptionsProvider.GetUserPrefsKey(TargetOptions));
		FString SelectedOption = UserSelectedOption != nullptr ? *UserSelectedOption : TypedDefinition->Target;
		for(const FString& Target : TypedDefinition->Targets)
		{
			Options.Add(Target, Target);

			if(SelectedOption.IsEmpty())
			{
				SelectedOption = Target;
			}
		}
		OptionsProvider.InitializeValidatorOptions(TargetOptions, Options, SelectedOption, EValidatorOptionType::Standard);
	}

	if(TypedDefinition->bUseStaticAnalyser)
	{
		TMap<FString, FString> Options;
		Options.Reserve(TypedDefinition->StaticAnalysers.Num());

		FString* UserSelectedOption = FSubmitToolUserPrefs::Get()->ValidatorOptions.Find(OptionsProvider.GetUserPrefsKey(StaticAnalyserOptions));
		FString SelectedOption = UserSelectedOption != nullptr ? *UserSelectedOption : TypedDefinition->StaticAnalyser;
		for(const FString& StaticAnalyser : TypedDefinition->StaticAnalysers)
		{
			Options.Add(StaticAnalyser, StaticAnalyser);

			if(SelectedOption.IsEmpty())
			{
				SelectedOption = StaticAnalyser;
			}
		}
		OptionsProvider.InitializeValidatorOptions(StaticAnalyserOptions, Options, SelectedOption, EValidatorOptionType::Standard);
	}
}
