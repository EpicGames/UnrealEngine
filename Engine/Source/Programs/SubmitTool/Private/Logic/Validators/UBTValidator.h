// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ValidatorRunExecutable.h"

namespace SubmitToolParseConstants
{
	const FString UBTValidator = TEXT("UBTValidator");
}

class FUBTValidator : public FValidatorRunExecutable
{

public:		
	FUBTValidator(const FName& InNameId, const FSubmitToolParameters& InParameters, TSharedRef<FSubmitToolServiceProvider> InServiceProvider, const FString& InDefinition);
	virtual void ParseDefinition(const FString& InDefinition) override;
	virtual bool Validate(const FString& InCLDescription, const TArray<FSCFileRef>& InFilteredFilesInCL, const TArray<const FTag*>& InTags) override;
	virtual const FString& GetValidatorTypeName() const override { return SubmitToolParseConstants::UBTValidator; }
	virtual bool Activate() override;
	
protected:
	TSet<FString> ExtractUprojectFiles(const TArray<FString>& InFiles);
	FString CreateFileList(const TArray<FString>& InFiles, const FString& InDirectory);
	TArray<FString> FilterFiles(const TArray<FString>& InFiles, const FString& InDirectory, const TArray<FString>& InExcludeDirectories);
	void FilterProgramFiles(const TArray<FString>& InFiles, TArray<FString>& OutNotProgramFiles, TMap<FString, TArray<FString>>& OutProgramFiles);


	void PrepareUBTOptions();

protected:
	const FString ConfigurationOptions = TEXT("ConfigurationOptions");
	const FString PlatformOptions = TEXT("PlatformOptions");
	const FString StaticAnalyserOptions = TEXT("StaticAnalyserOptions");
	const FString TargetOptions = TEXT("TargetOptions");

};
