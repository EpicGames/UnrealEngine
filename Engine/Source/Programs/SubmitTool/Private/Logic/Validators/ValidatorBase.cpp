// Copyright Epic Games, Inc. All Rights Reserved.

#include "ValidatorBase.h"
#include "HAL/FileManagerGeneric.h"
#include "Internationalization/Regex.h"
#include "Misc/Paths.h"
#include "Misc/Timespan.h"
#include "Misc/StringOutputDevice.h"
#include "AnalyticsEventAttribute.h"
#include "CommandLine/CmdLineParameters.h"
#include "Configuration/Configuration.h"
#include "Logic/DialogFactory.h"
#include "Logic/Services/Interfaces/ITagService.h"
#include "Logic/PreflightService.h"
#include "Logic/Services/Interfaces/ICacheDataService.h"
#include "Logic/Services/Interfaces/ISTSourceControlService.h"
#include "SubmitToolCoreUtils.h"
#include "Telemetry/TelemetryService.h"

FValidatorBase::FValidatorBase(const FName& InNameId, const FSubmitToolParameters& InParameters, TWeakPtr<FSubmitToolServiceProvider> InServiceProvider, const FString& InDefinition) :
	ValidatorNameID(InNameId),
	OptionsProvider(InNameId),
	ServiceProvider(InServiceProvider),
	SubmitToolParameters(InParameters),
	Start(FDateTime::MinValue())
{
	ParseDefinition(InDefinition);
	ValidatorName = Definition->CustomName.IsEmpty() ? ValidatorNameID.ToString() : Definition->CustomName;
	CorrelationId = FGuid::NewGuid();
}

void FValidatorBase::ParseDefinition(const FString& InDefinition)
{
	Definition = MakeUnique<FValidatorDefinition>();
	FStringOutputDevice Errors;
	FValidatorDefinition::StaticStruct()->ImportText(*InDefinition, const_cast<FValidatorDefinition*>(Definition.Get()), nullptr, 0, &Errors, FValidatorDefinition::StaticStruct()->GetName());

	if (!Errors.IsEmpty())
	{
		UE_LOGF(LogSubmitTool, Error, "[%ls] Error loading parameter file %ls", *GetValidatorNameId().ToString(), *Errors);
	}

	// Convert the "flat" definition entries into an extension -> (path1, path2) map
	for (const FPathPerExtension& PPE : Definition->IncludeFilesInDirectoryPerExtension)
	{
		FString Extension = PPE.Extension.ToLower();
		FString Path = FConfiguration::Substitute(PPE.Path);
		PathsPerExtension.FindOrAdd(MoveTemp(Extension)).Add(MoveTemp(Path));
	}
}

FValidatorBase::~FValidatorBase()
{
	OnValidationFinished.Clear();
}

void FValidatorBase::StartValidation()
{
	if (Definition->bIsDisabled)
	{
		State = EValidationStates::Disabled;
		return;
	}

	TRACE_BEGIN_REGION(*ValidatorName, TEXT("Validators"));
	TRACE_CPUPROFILER_EVENT_SCOPE(StartValidation);

	RunTime = 0;
	Start = FDateTime::UtcNow();
	State = EValidationStates::Running;

	if (LockdownIds.Num() != 0)
	{
		LockdownIds.Reset();
		OnLockdownIdsChanged.Broadcast(*this);
	}

	FailureReason = EFailureReason::None;
	ErrorListCache.Reset();
	WarningListCache.Reset();
	CorrelationId = FGuid::NewGuid();

	const TSharedPtr<IChangelistService>& ChangelistService = ServiceProvider.Pin()->GetService<IChangelistService>();
	const TSharedPtr<ITagService>& TagService = ServiceProvider.Pin()->GetService<ITagService>();

	FilteredFiles.RemoveAt(0, FilteredFiles.Num(), EAllowShrinking::No);

	bool bIncrementalValidation = Definition->bUsesIncrementalCache && !bForceRun;
	bForceRun = false;
	TArray<FSCFileRef> IncrementallySkippedFiles;
	bool bAppliesToCL = AppliesToCL(ChangelistService->GetCLDescription(), ChangelistService->GetFilesInCL(), TagService->GetTagsArray(), FilteredFiles, IncrementallySkippedFiles, bIncrementalValidation);

	if (bAppliesToCL)
	{
		if (FilteredFiles.Num() > 0)
		{
			UE_LOGF(LogSubmitToolDebug, Log, "[%ls] Validating files:\n - %ls", *ValidatorName, *FString::JoinBy(FilteredFiles, TEXT("\n - "), [](const FSCFileRef& FileRef) { return FileRef->GetFilename(); }));
		}

		if (!bIsValidSetup)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Activate);
			// Try to recover if things have changed since startup
			ActivationErrors.Empty();
			if (!Activate())
			{
				LogFailure(FString::Printf(TEXT("[%s] Task is not correctly setup and should run in this CL"), *ValidatorName));
				for (const FString& ActivationError : ActivationErrors)
				{
					LogFailure(ActivationError);
				}

				FailureReason = EFailureReason::InvalidSetup;
				ValidationFinished(false);
			}
		}

		if (bIsValidSetup)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Validate);
			const TSharedPtr<FPreflightService>& PreflightService = ServiceProvider.Pin()->GetService<FPreflightService>();
			if (!Definition->PreflightTemplate.IsEmpty())
			{
				for (const FPreflightData* PreflightData : PreflightService->GetTaggedPreflights())
				{
					if (PreflightData != nullptr &&
						PreflightData->TemplateId.Equals(Definition->PreflightTemplate) && 
						PreflightData->CachedResults.State == EPreflightState::Completed && 
						(PreflightData->CachedResults.Outcome == EPreflightOutcome::Success || PreflightData->CachedResults.Outcome == EPreflightOutcome::Warnings))
					{
						UE_LOGF(LogSubmitTool, Log, "[%ls] Preflight %ls with id %ls has succeeded and covers this validation.", *GetValidatorName(), *PreflightData->Name, *PreflightData->ID);
						Skip();
						return;
					}
				}
			}

			if (Definition->MinMemoryGBRequired > 0)
			{
				FPlatformMemoryStats MemoryStats = FPlatformMemory::GetStats();

				// Only consider Physical memory to be conservative 
				UE_LOGF(LogSubmitToolDebug, Log, "Available memory %llu", (MemoryStats.AvailablePhysical / 1024 / 1024 / 1024));

				if ((static_cast<uint64>(Definition->MinMemoryGBRequired) * 1024 * 1024 * 1024) > MemoryStats.AvailablePhysical)
				{
					TStringBuilder<256> Message;
					Message.Append(Definition->NotEnoughMemoryMessage);
					Message.Appendf(TEXT("\nAvailable memory: %llu GB\nRequired memory: %d GB"), MemoryStats.AvailablePhysical / 1024 / 1024 / 1024, Definition->MinMemoryGBRequired);
					const FString FinalMessage = Definition->PreflightTemplate.IsEmpty() ? Message.ToString() : FString::Format(Message.ToString(), PreflightService->GetFormatParameters(Definition->PreflightTemplate));

					TArray<FString> Buttons{ TEXT("Retry"), Definition->bAllowSkippingOnMemoryFail ? TEXT("Skip Validation") : TEXT("Close")};

					if(!Definition->PreflightTemplate.IsEmpty())
					{
						Buttons.Add(TEXT("Request Preflight"));
					}

					EDialogFactoryResult Result = FDialogFactory::ShowDialog(FText::FromString(GetValidatorName() + TEXT(" - Not Enough Memory")), FText::FromString(FinalMessage), Buttons, nullptr);
					
					if (Result != EDialogFactoryResult::FirstButton)
					{						
						if(Definition->bAllowSkippingOnMemoryFail && Result == EDialogFactoryResult::SecondButton)
						{
							FailureReason = EFailureReason::SkippedOOM;
							UE_LOGF(LogSubmitTool, Log, "[%ls] Skipped due to lack of memory.\n%ls", *GetValidatorName(), *FinalMessage);
							Skip();
							return;
						}

						LogFailure(FString::Printf(TEXT("[%s] %s"), *GetValidatorName(), *FinalMessage));
						if (Result == EDialogFactoryResult::ThirdButton)
						{
							PreflightService->RequestPreflight(Definition->PreflightTemplate);
						}

						FailureReason = EFailureReason::OutOfMemory;
						ValidationFinished(false);
						return;
					}
				}
			}

			if (IncrementallySkippedFiles.Num() != 0)
			{
				const FString FileList = FString::JoinBy(IncrementallySkippedFiles, TEXT("\n"), [](const FSCFileRef& InFile) { return InFile->GetFilename(); });
				UE_LOGF(LogValidators, Log, "[%ls] Skipping Files because they were already validated in a previous execution:\n%ls", *GetValidatorName(), *FileList);
			}

			UE_LOGF(LogSubmitToolDebug, Log, "[%ls] CorrelationId: %ls", *GetValidatorName(), *CorrelationId.ToString());
			if (!Validate(ChangelistService->GetCLDescription(), FilteredFiles, TagService->GetTagsArray()))
			{
				if(FailureReason == EFailureReason::None)
				{
					FailureReason = EFailureReason::UnspecifiedFailure;
				}

				ValidationFinished(false);
			}
		}
	}
	else if (!bAppliesToCL)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(Skip);
		if (IncrementallySkippedFiles.Num() != 0)
		{
			UE_LOGF(LogValidators, Log, "[%ls] All files were validated in a previous validation and are still valid. To force a validation click 'Run' in the validator list", *ValidatorName);
			UE_LOGF(LogValidatorsResult, Log, "[%ls] All files were validated in a previous validation and are still valid. To force a validation click 'Run' in the validator list", *ValidatorName);
		}
		else
		{
			if (!Definition->AppliesToCLRegex.IsEmpty())
			{
				UE_LOGF(LogValidators, Log, "[%ls] No files match the regex %ls. %ls", *ValidatorName, *FConfiguration::Substitute(Definition->AppliesToCLRegex), *Definition->NotApplicableToCLMessage);
				UE_LOGF(LogValidatorsResult, Log, "[%ls] No files match the regex %ls. %ls", *ValidatorName, *FConfiguration::Substitute(Definition->AppliesToCLRegex), *Definition->NotApplicableToCLMessage);
			}
			else
			{
				const FString Extensions = Definition->IncludeFilesWithExtension.IsEmpty() ? TEXT(".*") : *FString::Join(Definition->IncludeFilesWithExtension, TEXT("|"));
				UE_LOGF(LogValidators, Log, "[%ls] No files match the filter %ls{%ls} %ls doesn't need to run", *ValidatorName, *Definition->IncludeFilesInDirectory, *Extensions, *ValidatorName);
				UE_LOGF(LogValidatorsResult, Log, "[%ls] No files match the filter %ls{%ls} %ls doesn't need to run", *ValidatorName, *Definition->IncludeFilesInDirectory, *Extensions, *ValidatorName);
			}
		}
		Skip();
	}
}

void FValidatorBase::ToggleEnabled()
{
	FValidatorDefinition* ModifiableDefinition = const_cast<FValidatorDefinition*>(Definition.Get());
	ModifiableDefinition->bIsDisabled = !ModifiableDefinition->bIsDisabled;

	CancelValidation();

	if (ModifiableDefinition->bIsDisabled)
	{
		State = EValidationStates::Disabled;
	}
	else
	{
		State = EValidationStates::Not_Run;
	}
}

void FValidatorBase::Tick(float InDeltaTime)
{
	RunTime += InDeltaTime;

	if (Definition->TimeoutLimit > 0 && RunTime >= Definition->TimeoutLimit)
	{
		LogFailure(FString::Printf(TEXT("[%s] Timeout limit has been reached (%.0f seconds), cancelling task."), *GetValidatorName(), Definition->TimeoutLimit));

		StopInternalValidations();
		SetValidationFinished(EValidationStates::Timeout);
	}
}

void FValidatorBase::CancelValidation(const bool InbAsFailed)
{
	if (State == EValidationStates::Disabled)
	{
		return;
	}

	if (State == EValidationStates::Running)
	{
		StopInternalValidations();
	}

	FailureReason = EFailureReason::Cancelled;
	State = InbAsFailed ? EValidationStates::Failed : EValidationStates::Not_Run;
	SetValidationFinished(State);
}

bool FValidatorBase::CanStartTask() const
{
	bool bCanRun = true;
	if (Definition->bRequireShelveOperation)
	{		
		bCanRun &= ServiceProvider.Pin()->GetService<IChangelistService>()->IsShelfReady();
	}

	if (!Definition->PreflightTemplate.IsEmpty())
	{
		bCanRun &= ServiceProvider.Pin()->GetService<FPreflightService>()->GetIsHordeInformationReady();
	}

	return bCanRun;
}

bool FValidatorBase::Activate()
{
	bIsValidSetup = true;

	if (Definition != nullptr)
	{
		if (!Definition->IncludeFilesInDirectory.IsEmpty())
		{
			FValidatorDefinition* ModifiableDefinition = const_cast<FValidatorDefinition*>(Definition.Get());
			ModifiableDefinition->IncludeFilesInDirectory = FConfiguration::SubstituteAndNormalizeDirectory(ModifiableDefinition->IncludeFilesInDirectory);
		}
	}
	else
	{
		bIsValidSetup = false;
	}

	return bIsValidSetup;
}

void FValidatorBase::InvalidateLocalFileModifications()
{
	if ((Definition->TaskArea & ETaskArea::LocalFiles) == ETaskArea::LocalFiles && (State == EValidationStates::Valid || State == EValidationStates::Running || State == EValidationStates::Skipped || State == EValidationStates::Not_Applicable))
	{
		FFileManagerGeneric FileManager;
		for (const FSCFileRef& File : ServiceProvider.Pin()->GetService<IChangelistService>()->GetFilesInCL())
		{
			bool bIncrementallySkipped;

			if (AppliesToFile(File, false, bIncrementallySkipped))
			{
				FString Filename = File->GetFilename();
				FFileStatData FileModifiedDate = FileManager.GetStatData(*Filename);
				if (FileModifiedDate.ModificationTime > Start)
				{
					if (GetIsRunning())
					{
						UE_LOGF(LogValidators, Warning, "[%ls] File %ls was modified during the run, this task needs to be run again.", *GetValidatorName(), *Filename);
						UE_LOGF(LogValidatorsResult, Warning, "[%ls] File %ls was modified during the run, this task needs to be run again.", *GetValidatorName(), *Filename);
					}
					else
					{
						UE_LOGF(LogValidators, Warning, "[%ls] File %ls has been modified after the last run, this task needs to be run again.", *GetValidatorName(), *Filename);
						UE_LOGF(LogValidatorsResult, Warning, "[%ls] File %ls has been modified after the last run, this task needs to be run again.", *GetValidatorName(), *Filename);
					}

					Invalidate();
					break;
				}
			}
		}
	}
}

const FString FValidatorBase::GetStatusText() const
{
	const FString StateStr = StaticEnum<EValidationStates>()->GetNameStringByValue(static_cast<int64>(State))
		.Replace(TEXT("_"), TEXT(" "));

	FString AdditionalInformation;

	if (State == EValidationStates::Queued)
	{
		if (Definition->bRequireShelveOperation && !ServiceProvider.Pin()->GetService<IChangelistService>()->IsShelfReady())
		{
			AdditionalInformation = TEXT("Waiting Shelf");
		}

		if (!Definition->PreflightTemplate.IsEmpty() && !ServiceProvider.Pin()->GetService<FPreflightService>()->GetIsHordeInformationReady())
		{
			AdditionalInformation = TEXT("Waiting Horde");
		}
	}

	// do not clutter the UI with uninteresting information
	if (State == EValidationStates::Failed || State == EValidationStates::Valid || State == EValidationStates::Running || State == EValidationStates::Timeout)
	{
		if (RunTime >= 0.5f)
		{
			AdditionalInformation = FGenericPlatformTime::PrettyTime(RunTime);
		}		
	}

	if (AdditionalInformation.IsEmpty())
	{
		return StateStr;
	}
	else
	{
		return FString::Printf(TEXT("%s (%s)"), *StateStr,	*AdditionalInformation);
	}

}

const TArray<FAnalyticsEventAttribute> FValidatorBase::GetTelemetryAttributes() const
{
	const TSharedPtr<IChangelistService> ChangelistService = ServiceProvider.Pin()->GetService<IChangelistService>();

	FString StealthStateStr = TEXT("No Stealth Mode");

	if (Definition->bIsStealthMode)
	{
		StealthStateStr = StaticEnum<EValidationStates>()->GetNameStringByValue(static_cast<int64>(StealthState)).Replace(TEXT("_"), TEXT(" "));
	}

	FString CLString = ChangelistService->GetCLID();
	int64 CLNumber;
	if (!LexTryParseString(CLNumber, *CLString))
	{
		CLNumber = 0;
		CLString = TEXT("0");
	}

	return MakeAnalyticsEventAttributeArray(
		TEXT("ValidatorID"), *GetValidatorNameId().ToString(),
		TEXT("ValidatorName"), *GetValidatorName(),
		TEXT("Status"), Definition->bIsStealthMode ? !bStealthFailure : GetHasPassed(),
		TEXT("Success"), Definition->bIsStealthMode ? !bStealthFailure : GetHasPassed(),
		TEXT("ValidatorState"), StaticEnum<EValidationStates>()->GetNameStringByValue(static_cast<int64>(State)).Replace(TEXT("_"), TEXT(" ")),
		TEXT("IsStealthValidator"), Definition->bIsStealthMode,
		TEXT("StealthActualState"), MoveTemp(StealthStateStr),
		TEXT("Runtime"), RunTime,
		TEXT("Stream"), ServiceProvider.Pin()->GetService<ISTSourceControlService>()->GetCurrentStreamName(),
		TEXT("PendingChangelist"), MoveTemp(CLString),
		TEXT("PendingChangeNumber"), CLNumber,
		TEXT("FailureReason"), StaticEnum<EFailureReason>()->GetNameStringByValue(static_cast<int64>(FailureReason)),
		TEXT("FromEditor"), FCmdLineParameters::Get().Contains(FSubmitToolCmdLine::EditorFlag),
		TEXT("Errors"), ProcessMessagesForTelemetry(ErrorListCache),
		TEXT("Warnings"), ProcessMessagesForTelemetry(WarningListCache),
		TEXT("NumErrors"), ErrorListCache.Num(),
		TEXT("NumWarnings"), WarningListCache.Num(),
		TEXT("CorrelationId"), CorrelationId.ToString()
	);
}

TArray<FString> FValidatorBase::ProcessMessagesForTelemetry(const TArray<FString>& InMessages) const
{
	TArray<FString> ProcessedMessages = InMessages;

	for (FString& Message : ProcessedMessages)
	{
		ProcessSingleMessageForTelemetry(Message);
	}

	return ProcessedMessages;
}

void FValidatorBase::ProcessSingleMessageForTelemetry(FString& InOutMessage) const
{
	// Remove commas to not mess up with telemetry
	InOutMessage.ReplaceInline(TEXT(","), TEXT("[comma]"));
}

TArray<FAnalyticsEventAttribute> FValidatorBase::GetSingleMessageTelemetryAttributes(const FString& InMessage) const
{
	return MakeAnalyticsEventAttributeArray(
		TEXT("ValidatorID"), *GetValidatorNameId().ToString(),
		TEXT("ValidatorName"), *GetValidatorName(),
		TEXT("IsStealthValidator"), Definition->bIsStealthMode,
		TEXT("Runtime"), RunTime,
		TEXT("Stream"), ServiceProvider.Pin()->GetService<ISTSourceControlService>()->GetCurrentStreamName(),
		TEXT("PendingChangelist"), ServiceProvider.Pin()->GetService<IChangelistService>()->GetCLID(),
		TEXT("FromEditor"), FCmdLineParameters::Get().Contains(FSubmitToolCmdLine::EditorFlag),
		TEXT("Message"), InMessage
	);
}

void FValidatorBase::ValidationFinished(const bool bPassed)
{
	if (!Definition->bIsStealthMode) // Only log results when not in stealth mode
	{
		if (bPassed)
		{
			UE_LOGF(LogValidatorsResult, Log, "[%ls] Task Succeeded! (%ls)", *GetValidatorName(), *FGenericPlatformTime::PrettyTime(RunTime));
		}
		else if (Definition->IsRequired)
		{
			UE_LOGF(LogValidatorsResult, Error, "[%ls] Failed on Required Task!", *GetValidatorName());
		}
		else
		{
			UE_LOGF(LogValidatorsResult, Warning, "[%ls] Failed on Optional Task!", *GetValidatorName());
		}


		if (!bPassed)
		{
			for (const FString& ErrorMsg : Definition->AdditionalValidationErrorMessages)
			{
				LogFailure(FString::Printf(TEXT("[%s] %s"), *GetValidatorName(), *ErrorMsg));
			}
		}
	}

	if (bPassed && Definition->bUsesIncrementalCache)
	{
		ServiceProvider.Pin()->GetService<ICacheDataService>()->UpdateLastValidationForFiles(ServiceProvider.Pin()->GetService<IChangelistService>()->GetCLID(), GetValidatorNameId(), GetValidationConfigId(), FilteredFiles, FDateTime::UtcNow());
	}

	EValidationStates NewState = bPassed ? EValidationStates::Valid : EValidationStates::Failed;
	SetValidationFinished(NewState);
}

void FValidatorBase::LogFailure(const FString& FormattedMsg) const
{
	if (Definition->IsRequired)
	{
		if (Definition->bIsStealthMode)
		{
			UE_LOGF(LogValidators, Log, "%ls", *FormattedMsg);
		}
		else
		{
			UE_LOGF(LogValidators, Error, "%ls", *FormattedMsg);
			UE_LOGF(LogValidatorsResult, Error, "%ls", *FormattedMsg);
		}

		FScopeLock Lock(&ErrorLogsMutex);
		ErrorListCache.Add(FormattedMsg);
	}
	else
	{
		if (Definition->bIsStealthMode)
		{
			UE_LOGF(LogValidators, Log, "%ls", *FormattedMsg);
		}
		else
		{
			UE_LOGF(LogValidators, Warning, "%ls", *FormattedMsg);
			UE_LOGF(LogValidatorsResult, Warning, "%ls", *FormattedMsg);
		}

		FScopeLock Lock(&ErrorLogsMutex);
		WarningListCache.Add(FormattedMsg);
	}

	// Launch the telemetry on the game thread
	UE::Tasks::Launch(UE_SOURCE_LOCATION, [this, FormattedMsg]() mutable
	{
		FString TelemetryMsg = FormattedMsg;
		ProcessSingleMessageForTelemetry(TelemetryMsg);
		FString EventName = Definition->IsRequired ? TEXT("SubmitTool.Task.Error") : TEXT("SubmitTool.Task.Warning");

		FTelemetryService::Get()->CustomEvent(MoveTemp(EventName), GetSingleMessageTelemetryAttributes(TelemetryMsg));
	}, LowLevelTasks::ETaskPriority::Normal, UE::Tasks::EExtendedTaskPriority::GameThreadNormalPri);
}

void FValidatorBase::SetValidationFinished(EValidationStates NewState)
{
	TRACE_BOOKMARK(TEXT("ValidationFinished - %s"), *ValidatorName);

	check(NewState != EValidationStates::Running);
	check(NewState != EValidationStates::Queued);

	if (Definition->bIsStealthMode)
	{
		StealthState = NewState;
		bStealthFailure = NewState == EValidationStates::Timeout || NewState == EValidationStates::Failed;
		if (bStealthFailure)
		{
			NewState = EValidationStates::Valid;
		}
	}

	State = NewState;

	if (OnValidationFinished.IsBound())
	{
		OnValidationFinished.Broadcast(*this);
	}
	TRACE_END_REGION(*ValidatorName);
}

bool FValidatorBase::EvaluateTagSkip()
{
	TSharedPtr<IChangelistService> ChangelistService = ServiceProvider.Pin()->GetService<IChangelistService>();
	if (Definition->SkipForbiddenFiles.Num() > 0)
	{
		for (const FString& File : Definition->SkipForbiddenFiles)
		{
			for (const FSCFileRef& SCCFile : ChangelistService->GetFilesInCL())
			{
				if (FPaths::GetCleanFilename(SCCFile->GetFilename()).Contains(File))
				{
					return false;
				}
			}
		}
	}

	if (Definition->SkipForbiddenTags.Num() > 0)
	{
		for (const FString& Tag : Definition->SkipForbiddenTags)
		{
			if (ChangelistService->GetCLDescription().Find(Tag, ESearchCase::IgnoreCase) != INDEX_NONE)
			{
				UE_LOGF(LogValidators, Log, "[%ls] The Description contains '%ls'. %ls is not allowed to be skipped", *ValidatorName, *Tag, *ValidatorName);
				UE_LOGF(LogValidatorsResult, Log, "[%ls] The Description contains '%ls'. %ls is not allowed to be skipped", *ValidatorName, *Tag, *ValidatorName);
				return false;
			}
		}
	}

	if (Definition->SkipForbiddenStreams.Num() > 0)
	{
		for (const FSCFileRef& File : ChangelistService->GetFilesInCL())
		{
			for (const FString& Stream : Definition->SkipForbiddenStreams)
			{
				if (File->GetDepotPath().StartsWith(Stream, ESearchCase::IgnoreCase))
				{
					UE_LOGF(LogValidators, Log, "[%ls] Changes in %ls require the validation to be run from submit tool. %ls won't be skipped", *ValidatorName, *Stream, *ValidatorName);
					UE_LOGF(LogValidatorsResult, Log, "[%ls] Changes in %ls require the validation to be run from submit tool. %ls won't be skipped", *ValidatorName, *Stream, *ValidatorName);
					return false;
				}
			}
		}
	}


	if (Definition->bSkipWhenCalledFromEditor && FCmdLineParameters::Get().Contains(FSubmitToolCmdLine::EditorFlag))
	{
		UE_LOGF(LogValidators, Log, "[%ls] Submit tool has been called from the editor %ls doesn't need to run.", *ValidatorName, *ValidatorName);
		Start = FDateTime::UtcNow();
		State = EValidationStates::Skipped;
		return true;
	}

	if (Definition->bSkipWhenAddendumInDescription && !Definition->ChangelistDescriptionAddendum.IsEmpty())
	{
		if (ChangelistService->GetCLDescription().Find(Definition->ChangelistDescriptionAddendum, ESearchCase::IgnoreCase) != INDEX_NONE)
		{
			UE_LOGF(LogValidators, Log, "[%ls] The Description Addendum '%ls' is already present in the CL. %ls doesn't need to run", *ValidatorName, *Definition->ChangelistDescriptionAddendum, *ValidatorName);
			UE_LOGF(LogValidatorsResult, Log, "[%ls] The Description Addendum '%ls' is already present in the CL. %ls doesn't need to run", *ValidatorName, *Definition->ChangelistDescriptionAddendum, *ValidatorName);
			Start = FDateTime::UtcNow();
			State = EValidationStates::Skipped;
			return true;
		}
	}

	return false;
}

void FValidatorBase::Skip()
{
	SetValidationFinished(EValidationStates::Skipped);
}

bool FValidatorBase::IsRelevantToCL() const
{
	TSharedPtr<IChangelistService> ChangelistService = ServiceProvider.Pin()->GetService<IChangelistService>();

	TArray<FSCFileRef> IncrementallySkippedFiles;
	TArray<FSCFileRef> OutFiles;
	return AppliesToCL(ChangelistService->GetCLDescription(), ChangelistService->GetFilesInCL(), ServiceProvider.Pin()->GetService<ITagService>()->GetTagsArray(), OutFiles, IncrementallySkippedFiles, false);
}

void FValidatorBase::SetSelectedOption(const FString& InOptionName, const FString& InOptionValue)
{
	UE_LOGF(LogValidators, Log, "[%ls] Task stopped due to a change in options, %ls = %ls", *GetValidatorName(), *InOptionName, *InOptionValue);
	CancelValidation();
	OptionsProvider.SetSelectedOption(InOptionName, InOptionValue);
}

bool FValidatorBase::CanPrintErrors() const
{
	if (!Definition->bIsStealthMode && (State == EValidationStates::Failed || State == EValidationStates::Timeout || State == EValidationStates::Valid))
	{
		return !ErrorListCache.IsEmpty() || !WarningListCache.IsEmpty();
	}
	else
	{
		return false;
	}
}

void FValidatorBase::PrintErrorSummary() const
{
	FScopeLock Lock(&ErrorLogsMutex);

	if (CanPrintErrors())
	{
		if (Definition->IsRequired)
		{
			for (const FString& ErrorStr : ErrorListCache)
			{
				UE_LOGF(LogValidators, Error, "%ls", *ErrorStr);
				UE_LOGF(LogValidatorsResult, Error, "%ls", *ErrorStr);
			}
		}
		else
		{
			for (const FString& ErrorStr : ErrorListCache)
			{
				UE_LOGF(LogValidators, Warning, "%ls", *ErrorStr);
				UE_LOGF(LogValidatorsResult, Warning, "%ls", *ErrorStr);
			}
		}

		if (Definition->bIncludeWarningsInErrorSummary)
		{
			for (const FString& WarningStr : WarningListCache)
			{
				UE_LOGF(LogValidators, Warning, "%ls", *WarningStr);
				UE_LOGF(LogValidatorsResult, Warning, "%ls", *WarningStr);
			}
		}
	}
}

const FString FValidatorBase::GetValidationConfigId() const
{
	TStringBuilder<512> StringBuilder;
	for (const TPair<FString, FString>& Pair : OptionsProvider.GetSelectedOptions())
	{
		StringBuilder.Append(Pair.Key);
		StringBuilder.Append(TEXT("_"));
		StringBuilder.Append(Pair.Value);
		StringBuilder.Append(TEXT("-"));
	}

	return StringBuilder.ToString();
}

bool FValidatorBase::AppliesToFile(const FSCFileRef InFile, bool InbAllowIncremental, bool& OutbIsIncrementalSkip) const
{
	bool bIncluded = false;
	OutbIsIncrementalSkip = false;

	if ((Definition->TaskArea & ETaskArea::LocalFiles) == ETaskArea::None)
	{
		// For validators that do not work on local files, we always apply
		return true;
	}


	// Check "-from-editor" cmdline arg
	// Don't apply if the validator requires being launched from editor and the arg is not present
	if ((Definition->bRequiresSTLaunchedFromEditor && !FCmdLineParameters::Get().Contains(FSubmitToolCmdLine::EditorFlag))
		// Don't apply if the validator requires being launched from standalone and the arg is present
		|| (Definition->bRequiresSTLaunchedStandalone && FCmdLineParameters::Get().Contains(FSubmitToolCmdLine::EditorFlag)))
	{
		return false;
	}

	FString Filename = InFile->GetFilename();
	FPaths::NormalizeFilename(Filename);
	uint32 FileHash = GetTypeHash(InFile->GetFilename());

	if (FileHashes.Contains(FileHash))
	{
		bIncluded = FileHashes[FileHash];
	}
	else
	{
		if (!InFile->IsDeleted()
			|| (InFile->IsDeleted() && Definition->bAcceptDeletedFiles))
		{
			if (!Definition->AppliesToCLRegex.IsEmpty())
			{
				const FString RegexPat = FConfiguration::Substitute(Definition->AppliesToCLRegex);
				FRegexPattern Pattern = FRegexPattern(RegexPat, ERegexPatternFlags::CaseInsensitive);
				FRegexMatcher regex = FRegexMatcher(Pattern, Filename);

				bIncluded = regex.FindNext();
			}
			else
			{
				const bool bIncludeDot = true;
				FString Extension = FPaths::GetExtension(Filename, bIncludeDot).ToLower();

				// 1. (If given) the per-extension setting takes over the IncludeFilesInDirectory
				const TArray<FString>* PathPrefixes = PathsPerExtension.Find(Extension);
				if (PathPrefixes)
				{
					bool bIncludedInPaths = false;
					for (const FString& PathPrefix : *PathPrefixes)
					{
						if (Filename.StartsWith(PathPrefix, ESearchCase::IgnoreCase))
						{
							bIncludedInPaths = true;
							break;
						}
					}
					if (!bIncludedInPaths)
					{
						return false;
					}
				}
				// 2. No per-extension setting, use the common setting
				else if (!Definition->IncludeFilesInDirectory.IsEmpty())
				{
					if (!Filename.StartsWith(Definition->IncludeFilesInDirectory, ESearchCase::IgnoreCase))
					{
						return false;
					}
				}
				// 3. No directory filter given, carry on

				if (Definition->IncludeFilesWithExtension.IsEmpty())
				{
					bIncluded = true;
				}

				for (int Idx = 0; Idx < Definition->IncludeFilesWithExtension.Num(); Idx++)
				{
					if (Filename.EndsWith(Definition->IncludeFilesWithExtension[Idx], ESearchCase::IgnoreCase))
					{
						bIncluded = true;
						break;
					}
				}
			}

			if (bIncluded)
			{
				if (!Definition->RequireFileInHierarchy.IsEmpty())
				{
					if (!FSubmitToolCoreUtils::IsFileInHierarchy(Definition->RequireFileInHierarchy, Filename))
					{
						bIncluded = false;
					}
				}

				if (!Definition->ExcludeWhenFileInHierarchy.IsEmpty())
				{
					if (FSubmitToolCoreUtils::IsFileInHierarchy(Definition->ExcludeWhenFileInHierarchy, Filename))
					{
						bIncluded = false;
					}
				}
			}
		}

		FileHashes.FindOrAdd(FileHash, bIncluded);
	}

	if (InbAllowIncremental && bIncluded)
	{
		FFileManagerGeneric FileManager;
		FDateTime LastValidation = ServiceProvider.Pin()->GetService<ICacheDataService>()->GetLastValidationDate(ServiceProvider.Pin()->GetService<IChangelistService>()->GetCLID(), GetValidatorNameId(), GetValidationConfigId(), Filename);
		FFileStatData FileModifiedDate = FileManager.GetStatData(*Filename);
		if (LastValidation != FDateTime::MinValue() && FileModifiedDate.ModificationTime < LastValidation)
		{
			OutbIsIncrementalSkip = true;
			bIncluded = false;
		}
	}

	return bIncluded;
}

bool FValidatorBase::AppliesToCL(const FString& InCLDescription, const TArray<FSCFileRef>& FilesInCL, const TArray<const FTag*>& Tags, TArray<FSCFileRef>& OutFilteredFiles, TArray<FSCFileRef>& OutIncrementalSkips, bool InbAllowIncremental) const
{
	if ((Definition->TaskArea & ETaskArea::LocalFiles) == ETaskArea::None)
	{
		// For validators that do not work on files, we always apply
		return true;
	}

	// Check "-from-editor" cmdline arg
	// Don't apply if the validator requires being launched from editor and the arg is not present
	if ((Definition->bRequiresSTLaunchedFromEditor && !FCmdLineParameters::Get().Contains(FSubmitToolCmdLine::EditorFlag))
	// Don't apply if the validator requires being launched from standalone and the arg is present
		|| (Definition->bRequiresSTLaunchedStandalone && FCmdLineParameters::Get().Contains(FSubmitToolCmdLine::EditorFlag)))
	{
		return false;
	}

	for (const FString& Filepath : Definition->CheckFileExists)
	{
		if (!FPaths::FileExists(FConfiguration::Substitute(Filepath)))
		{
			return false;
		}
	}

	for (const FSCFileRef& File : FilesInCL)
	{
		bool bIsIncrementalSkip;
		if (AppliesToFile(File, InbAllowIncremental, bIsIncrementalSkip))
		{
			OutFilteredFiles.Add(File);
		}
		else if (bIsIncrementalSkip)
		{
			OutIncrementalSkips.Add(File);
		}
	}

	return OutFilteredFiles.Num() > 0;
}
