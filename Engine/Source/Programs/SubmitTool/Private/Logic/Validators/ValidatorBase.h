// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "CoreMinimal.h"

#include "Logging/SubmitToolLog.h"
#include "Logic/ProcessWrapper.h"
#include "Parameters/SubmitToolParameters.h" 

#include "Models/SCFile.h"
#include "Logic/Services/SubmitToolServiceProvider.h"
#include "Logic/Validators/ValidatorDefinition.h"
#include "ValidatorOptionsProvider.h"

#include "ValidatorBase.generated.h"

class FTag;
struct FAnalyticsEventAttribute;

UENUM()
enum class EValidationStates : uint8
{
	Not_Run,
	Skipped,
	Not_Applicable,
	Running,
	Valid,
	Failed,
	Timeout,
	Queued,
	Disabled
};


UENUM()
enum class EFailureReason : uint8
{
	None,
	InvalidSetup,
	OutOfMemory,
	SkippedOOM,
	Cancelled,
	InvalidExitCode,
	Errors,
	UnspecifiedFailure
};


template<class T>
concept DerivedFromDefinition = std::is_base_of<FValidatorDefinition, T>::value;

class FValidatorBase;
DECLARE_MULTICAST_DELEGATE_OneParam(FOnValidatorFinished, const FValidatorBase& /*Validator*/)
DECLARE_MULTICAST_DELEGATE_OneParam(FOnLockdownIdsChanged, const FValidatorBase& /*Validator*/)

/**
 * @brief Base class for validator.
 */
class FValidatorBase
{
public:
	FValidatorBase() = delete;
	// if signature changes, need to update FValidatorFactory
	FValidatorBase(const FName& InNameId, const FSubmitToolParameters& InParameters, const TWeakPtr<FSubmitToolServiceProvider> InServiceProvider, const FString& InDefinition);
	virtual ~FValidatorBase();

	virtual void ParseDefinition(const FString& InDefinition);

	virtual const FString& GetValidatorTypeName() const = 0;
	const FName& GetValidatorNameId() const
	{
		return ValidatorNameID;
	}
	const FString& GetValidatorName() const
	{
		return ValidatorName;
	}

	virtual void StartValidation();

	virtual void ToggleEnabled();

	void Invalidate(bool bForce = false)
	{
		if (State == EValidationStates::Disabled)
		{
			return;
		}

		if(State == EValidationStates::Running)
		{
			CancelValidation();
		}
		else if(State != EValidationStates::Queued || bForce)
		{
			State = EValidationStates::Not_Run;
		}
	}

	bool GetHasPassed() const
	{
		return State == EValidationStates::Valid || State == EValidationStates::Skipped || State == EValidationStates::Not_Applicable || State == EValidationStates::Disabled;
	}

	EValidationStates GetStealthState() const
	{
		return StealthState;
	}

	bool GetIsRunningOrQueued() const
	{
		return GetIsRunning() || GetIsQueued();
	}

	bool GetIsRunning() const
	{
		return State == EValidationStates::Running;
	}

	bool GetIsQueued() const
	{
		return State == EValidationStates::Queued;
	}

	float GetRunTime() const
	{
		return RunTime;
	}

	EValidationStates GetState() const
	{
		return State;
	}

	EFailureReason GetFailureReason() const
	{
		return FailureReason;
	}

	virtual void Tick(float InDeltaTime);

	virtual void CancelValidation(const bool bInAsFailed = false);

	virtual void SetQueued(bool InbForceRun = false)
	{
		bForceRun = InbForceRun;
		if(State == EValidationStates::Running)
		{
			StopInternalValidations();
		}

		State = EValidationStates::Queued;
	}

	void SetNotApplicable()
	{
		State = EValidationStates::Not_Applicable;
	}

	virtual bool CanStartTask() const;

	virtual bool Activate();

	virtual void InvalidateLocalFileModifications();
	virtual bool EvaluateTagSkip();
	virtual bool IsRelevantToCL() const;
	virtual const FString GetPopupMessageWhenFailedText() const
	{
		return Definition->PopupMessageWhenFailed;
	}

	virtual bool IsEnabled() const
	{
		return !Definition->bIsDisabled;
	}

	virtual const TSet<FString>& GetLockdownIds() const
	{
		return LockdownIds;
	}

	const FString GetStatusText() const;
	const EValidationStates GetValidatorState() const
	{
		return State;
	}

	const TMap<FString, TMap<FString, FString>>& GetValidatorOptions() const
	{
		return OptionsProvider.GetValidatorOptions();
	}

	const FString GetSelectedOptionValue(const FString& InOptionName) const
	{
		return OptionsProvider.GetSelectedOptionValue(InOptionName);
	}
	const FString GetSelectedOptionKey(const FString& InOptionName) const
	{
		return OptionsProvider.GetSelectedOptionKey(InOptionName);
	}
	EValidatorOptionType GetOptionType(const FString& InOptionName) const
	{
		return OptionsProvider.GetOptionType(InOptionName);
	}

	void SetSelectedOption(const FString& InOptionName, const FString& InOptionValue);

	virtual const TArray<FAnalyticsEventAttribute> GetTelemetryAttributes() const;
	
	bool CanPrintErrors() const;
	void PrintErrorSummary() const;

	const FString GetValidationConfigId() const;

	template<DerivedFromDefinition T>
	const T* GetTypedDefinition() const;

	TArray<FName> Dependants;
	TUniquePtr<const FValidatorDefinition> Definition;
	TMap<FString, TArray<FString>> PathsPerExtension; // Digested from Definition->IncludeFilesInDirectoryPerExtension
	FOnValidatorFinished OnValidationFinished;
	FOnLockdownIdsChanged OnLockdownIdsChanged;

	FGuid CorrelationId; // Automatically set to a new GUID each time before Validate is called

protected:
	FName ValidatorNameID;
	FString ValidatorName;
	EFailureReason FailureReason;
	FValidatorOptionsProvider OptionsProvider;
	TWeakPtr<FSubmitToolServiceProvider> ServiceProvider;
	TArray<FSCFileRef> FilteredFiles;
	TArray<FString> ActivationErrors;
	mutable TArray<FString> ErrorListCache;
	mutable TArray<FString> WarningListCache;
	bool bIsValidSetup = false;
	TSet<FString> LockdownIds;
	bool bForceRun = false;
	const FSubmitToolParameters& SubmitToolParameters;
	bool bStealthFailure = false;
	EValidationStates StealthState = EValidationStates::Not_Run;
	FDateTime Start;
	float RunTime = 0;

	/**
	 * @brief Starts the validator.
	 * @param SubmitSettings The submit settings to use for this validation cycle.
	 * @return Returns true if validation has started successfully, false if validation failed.
	 */
	virtual bool Validate(const FString& CLDescription, const TArray<FSCFileRef>& FilteredFilesInCL, const TArray<const FTag*>& Tags) = 0;

	virtual bool AppliesToFile(const FSCFileRef InFile, bool InbAllowIncremental, bool& OutbIsIncrementalSkip) const;
	virtual bool AppliesToCL(const FString& CLDescription, const TArray<FSCFileRef>& FilesInCL, const TArray<const FTag*>& Tags, TArray<FSCFileRef>& OutFilteredFiles, TArray<FSCFileRef>& OutIncrementalSkips, bool InbAllowIncremental) const;
	virtual void StopInternalValidations()
	{};
	virtual void ValidationFinished(const bool bHasPassed);

	virtual void LogFailure(const FString& FormattedMsg) const;

	virtual void Skip();
	virtual TArray<FString> ProcessMessagesForTelemetry(const TArray<FString>& InMessages) const;
	virtual void ProcessSingleMessageForTelemetry(FString& InOutMessage) const;
	virtual TArray<FAnalyticsEventAttribute> GetSingleMessageTelemetryAttributes(const FString& InMessage) const;


	EValidationStates State = EValidationStates::Not_Run;


	mutable TMap<uint32, bool> FileHashes;
	mutable FCriticalSection ErrorLogsMutex;

private:
	void SetValidationFinished(EValidationStates NewState);
};

template<DerivedFromDefinition T>
inline const T* FValidatorBase::GetTypedDefinition() const
{
	return  static_cast<const T*>(Definition.Get());
}
