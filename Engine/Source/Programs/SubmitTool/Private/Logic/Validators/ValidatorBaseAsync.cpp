// Copyright Epic Games, Inc. All Rights Reserved.

#include "ValidatorBaseAsync.h"

FValidatorBaseAsync::FValidatorBaseAsync(const FName& InNameId, const FSubmitToolParameters& InParameters, TSharedRef<FSubmitToolServiceProvider> InServiceProvider, const FString& InDefinition):
	FValidatorBase(InNameId, InParameters, InServiceProvider, InDefinition)
{
}

bool FValidatorBaseAsync::Validate(const FString& CLDescription, const TArray<FSCFileRef>& FilteredFilesInCL, const TArray<const FTag*>& Tags)
{
	if(!CurrentTasks.IsEmpty())
	{
		LogFailure(FString::Printf(TEXT("[%s] attempting to run a new validation while a previous validation pass is still running!"), *GetValidatorName()));
		return false;
	}

	CancellationToken = MakeUnique<UE::Tasks::FCancellationToken>();

	// Create a new token for each validation
	this->StartAsyncWork(CLDescription, FilteredFilesInCL, Tags);

	// add validation to check if some work has been done
	if(CurrentTasks.Num() == 0)
	{
		return false;
	}

	return true;
}

void FValidatorBaseAsync::StartAsyncTask(TFunction<bool(const UE::Tasks::FCancellationToken&)>&& InTask)
{
	UE::Tasks::TTask<bool> Task = UE::Tasks::Launch(UE_SOURCE_LOCATION, [this, InTask]() -> bool
		{
			return InTask(*CancellationToken);
		});

	CurrentTasks.Add(Task);
}

void FValidatorBaseAsync::Tick(float InDeltaTime)
{
	FValidatorBase::Tick(InDeltaTime);

	if (HasWorkPending())
	{
		return;
	}

	bool bAllValidationsSuccessful = true;

	for (UE::Tasks::TTask<bool>& Task : CurrentTasks)
	{
		bAllValidationsSuccessful &= Task.GetResult();
	}

	ValidationFinished(bAllValidationsSuccessful);

	CurrentTasks.Empty();
}

void FValidatorBaseAsync::StopInternalValidations()
{
	// First, signal that the validation has been canceled to all running jobs.
	if (CancellationToken.IsValid())
	{
		CancellationToken->Cancel();
	}
	
	// Secondly, wait on any job that is still running until their cancellation is complete.
	for (UE::Tasks::TTask<bool>& Task : CurrentTasks)
	{
		// that's just for you Juan
		Task.Wait();
	}

	// Lastly we can clean up the jobs.
	CurrentTasks.Reset();
}

bool FValidatorBaseAsync::HasWorkPending() const
{
	for (const UE::Tasks::TTask<bool>& Task : CurrentTasks)
	{
		if (!Task.IsCompleted())
		{
			return true;
		}
	}

	return false;
}
