// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ValidatorBase.h"
#include "Tasks/Task.h"

class FValidatorBaseAsync : public FValidatorBase
{
public:
	FValidatorBaseAsync(const FName& InNameId, const FSubmitToolParameters& InParameters, TSharedRef<FSubmitToolServiceProvider> InServiceProvider, const FString& InDefinition);

	virtual bool Validate(const FString& CLDescription, const TArray<FSCFileRef>& FilteredFilesInCL, const TArray<const FTag*>& Tags) override final;

	//
	// Override this method to start the async work
	// Please make sure to check that the CancellationToken has not been cancelled and stop processing in that case
	//
	virtual void StartAsyncWork(const FString& CLDescription, const TArray<FSCFileRef>& FilteredFilesInCL, const TArray<const FTag*>& Tags) = 0;

	virtual void Tick(float InDeltaTime) override  final;
	virtual void StopInternalValidations() override final;

protected:
	void StartAsyncTask(TFunction<bool(const UE::Tasks::FCancellationToken&)>&& InTask);

private:

	bool HasWorkPending() const;

	TUniquePtr<UE::Tasks::FCancellationToken> CancellationToken;
	TArray<UE::Tasks::TTask<bool>> CurrentTasks;
};
