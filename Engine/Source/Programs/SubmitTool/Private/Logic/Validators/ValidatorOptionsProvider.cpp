// Copyright Epic Games, Inc. All Rights Reserved.

#include "ValidatorOptionsProvider.h"

#include "Models/SubmitToolUserPrefs.h"


FValidatorOptionsProvider::FValidatorOptionsProvider(const FName& InValidatorId) :
	ValidatorId(InValidatorId)
{}

void FValidatorOptionsProvider::InitializeValidatorOptions(const FString& InOptionsNameKey, const TMap<FString, FString>& InOptions, const FString& InSelectedOption, const EValidatorOptionType InOptionType)
{
	Options.FindOrAdd(InOptionsNameKey) = InOptions;

	SelectedOptions.FindOrAdd(InOptionsNameKey) = InSelectedOption;
	OptionTypes.FindOrAdd(InOptionsNameKey) = InOptionType;
}

const TMap<FString, TMap<FString, FString>>& FValidatorOptionsProvider::GetValidatorOptions() const
{
	return Options;
}

const TMap<FString, FString>& FValidatorOptionsProvider::GetSelectedOptions() const
{
	return SelectedOptions;
}

const FString FValidatorOptionsProvider::GetSelectedOptionKey(const FString& InOptionName) const
{
	if(SelectedOptions.Num() > 0)
	{
		if(SelectedOptions.Contains(InOptionName))
		{
			return SelectedOptions[InOptionName];
		}

		UE_LOGF(LogSubmitTool, Warning, "Option %ls is not part of the selected option list", *InOptionName);
	}

	return FString{};
}

const FString FValidatorOptionsProvider::GetSelectedOptionValue(const FString& InOptionName) const
{
	if(!Options.Contains(InOptionName))
	{
		return FString();
	}

	if(SelectedOptions.Num() > 0)
	{
		if(SelectedOptions.Contains(InOptionName) && Options[InOptionName].Contains(SelectedOptions[InOptionName]))
		{
			return Options[InOptionName][SelectedOptions[InOptionName]];
		}
	}

	return FString{};
}

EValidatorOptionType FValidatorOptionsProvider::GetOptionType(const FString& InOptionName) const
{
	if(OptionTypes.Contains(InOptionName))
	{
		return OptionTypes[InOptionName];
	}

	return EValidatorOptionType::Invalid;
}

void FValidatorOptionsProvider::SetSelectedOption(const FString& InOptionName, const FString& InOptionValue)
{
	if(!Options.Contains(InOptionName))
	{
		return;
	}

	if(!Options[InOptionName].Contains(InOptionValue))
	{
		return;
	}

	SelectedOptions.FindOrAdd(InOptionName) = InOptionValue;
	FSubmitToolUserPrefs::Get()->ValidatorOptions.FindOrAdd(GetUserPrefsKey(InOptionName)) = InOptionValue;
}


FString FValidatorOptionsProvider::GetUserPrefsKey(const FString& InOptionName)
{
	return FString::Printf(TEXT("Validator_%s_%s"), *ValidatorId.ToString(), *InOptionName);
}