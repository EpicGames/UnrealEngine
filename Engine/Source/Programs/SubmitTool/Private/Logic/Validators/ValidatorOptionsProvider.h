// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/Map.h"

enum class EValidatorOptionType
{
	Standard,
	FilePath,
	Invalid
};

class FValidatorOptionsProvider
{
public:
	FValidatorOptionsProvider(const FName& InValidatorId);

	void InitializeValidatorOptions(
		const FString& InOptionName,
		const TMap<FString, FString>& InOptions,
		const FString& InSelectedOption,
		const EValidatorOptionType InOptionType);

	const TMap<FString, TMap<FString, FString>>& GetValidatorOptions() const;
	const TMap<FString, FString>& GetSelectedOptions() const;

	const FString GetSelectedOptionKey(const FString& InOptionName) const;
	const FString GetSelectedOptionValue(const FString& InOptionName) const;
	EValidatorOptionType GetOptionType(const FString& InOptionName) const;

	void SetSelectedOption(const FString& InOptionName, const FString& InOptionValue);
	FString GetUserPrefsKey(const FString& InOptionName);

private:
	const FName ValidatorId;
	TMap<FString, TMap<FString, FString>> Options;
	TMap<FString, FString> SelectedOptions;
	TMap<FString, EValidatorOptionType> OptionTypes;
};