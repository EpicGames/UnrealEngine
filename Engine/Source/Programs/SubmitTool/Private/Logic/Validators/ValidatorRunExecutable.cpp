// Copyright Epic Games, Inc. All Rights Reserved.


#include "ValidatorRunExecutable.h"

#include "AnalyticsEventAttribute.h"
#include "CommandLine/CmdLineParameters.h"
#include "Configuration/Configuration.h"
#include "HAL/FileManagerGeneric.h"
#include "Internationalization/Regex.h"
#include "Logic/Validators/ValidatorFactory.h"
#include "Misc/FileHelper.h"
#include "Misc/PathViews.h"
#include "Misc/Paths.h"
#include "Misc/StringOutputDevice.h"
#include "Models/ModelInterface.h"
#include "Models/SubmitToolUserPrefs.h"
#include "Modules/BuildVersion.h"
#include "Policies/CondensedJsonPrintPolicy.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "Serialization/JsonWriter.h"
#include "Telemetry/TelemetryService.h"

// Both "CustomValidator" and "Executable" will resolve to this same type, CustomValidator will eventually be deprecated
REGISTER_VALIDATOR_TYPE(SubmitToolParseConstants::CustomValidator, FValidatorRunExecutable)
REGISTER_VALIDATOR_TYPE_ALIAS(SubmitToolParseConstants::ExecutableValidator, FValidatorRunExecutable, ExecutableAlias) 


int32 FValidatorRunExecutable::ProcessCounter = 0;
FString FValidatorRunExecutable::GetProcessName(FStringView ExecutablePath)
{
	return FString::Format(TEXT("#{0}|{1}"), { ++ProcessCounter, FPathViews::GetCleanFilename(ExecutablePath) });
}

FValidatorRunExecutable::FValidatorRunExecutable(const FName& InNameId, const FSubmitToolParameters& InParameters, TSharedRef<FSubmitToolServiceProvider> InServiceProvider, const FString& InDefinition) :
	FValidatorBase(InNameId, InParameters, InServiceProvider, InDefinition)
{
	ParseDefinition(InDefinition);
}

void FValidatorRunExecutable::ParseDefinition(const FString& InDefinition)
{
	FStringOutputDevice Errors;
	Definition = MakeUnique<FValidatorRunExecutableDefinition>();
	FValidatorRunExecutableDefinition* ModifyableDefinition = const_cast<FValidatorRunExecutableDefinition*>(GetTypedDefinition<FValidatorRunExecutableDefinition>());
	FValidatorRunExecutableDefinition::StaticStruct()->ImportText(*InDefinition, ModifyableDefinition, nullptr, 0, &Errors, FValidatorRunExecutableDefinition::StaticStruct()->GetName());

	if(!Errors.IsEmpty())
	{
		UE_LOGF(LogSubmitTool, Error, "Error loading parameter file %ls", *Errors);
		FModelInterface::SetErrorState();
	}
}

void FValidatorRunExecutable::StartValidation()
{
	Processes.Reset();
	FValidatorBase::StartValidation();
}

bool FValidatorRunExecutable::Validate(const FString& InCLDescription, const TArray<FSCFileRef>& InFilteredFilesInCL, const TArray<const FTag*>& InTags)
{
	const FValidatorRunExecutableDefinition* TypedDefinition = GetTypedDefinition<FValidatorRunExecutableDefinition>();

	FString FinalArgs{ FConfiguration::Substitute(TypedDefinition->ExecutableArguments) };
	FinalArgs.ReplaceInline(TEXT("$(TaskName)"), *GetValidatorName());

	TArray<FString> Files;

	// aggregate all the valid files
	for(const FSCFileRef& File : InFilteredFilesInCL)
	{
		Files.Add(File.Get().GetFilename());
	}

	if(!bIsValidSetup)
	{
		LogFailure(FString::Printf(TEXT("[%s] This task is not correctly setup and it's required for this change"), *ValidatorName));
		return false;
	}

	// create a file with the files and pass it as an argument of the validator (better to avoid breaking command line length limits)
	if(!TypedDefinition->FileListArgument.IsEmpty())
	{
		FString ValidatorDirectory = FPaths::EngineDir() + TEXT("Intermediate/SubmitTool/FileLists/");

		FGuid guid = FGuid::NewGuid();
		FString FileListPath = FPaths::ConvertRelativePathToFull(ValidatorDirectory + guid.ToString(EGuidFormats::DigitsWithHyphens) + TEXT(".txt"));

		FFileHelper::SaveStringArrayToFile(Files, *FileListPath, FFileHelper::EEncodingOptions::ForceAnsi, &IFileManager::Get(), EFileWrite::FILEWRITE_None);

		FinalArgs += FString::Printf(TEXT(" %s\"%s\""), *TypedDefinition->FileListArgument, *FileListPath);
	}
	// Legacy, pass down each file to the validator in the command line
	else if(!TypedDefinition->FileInCLArgument.IsEmpty())
	{
		for(const FString& File : Files)
		{
			FinalArgs += " " + TypedDefinition->FileInCLArgument + File;
		}
	}

	if (!TypedDefinition->FromEditorOnlyArguments.IsEmpty() && FCmdLineParameters::Get().Contains(FSubmitToolCmdLine::EditorFlag))
	{
		FinalArgs.Append(TEXT(" "));
		FinalArgs.Append(TypedDefinition->FromEditorOnlyArguments);
	}

	FString ExecutablePath;
	if(TypedDefinition->ExecutableCandidates.Num() != 0)
	{
		ExecutablePath = OptionsProvider.GetSelectedOptionValue(ExecutableOptions);
	}
	else
	{
		ExecutablePath = TypedDefinition->ExecutablePath;
	}

	return QueueProcess(GetProcessName(ExecutablePath), ExecutablePath, FinalArgs);
}

bool FValidatorRunExecutable::Activate()
{
	bIsValidSetup = FValidatorBase::Activate();

	PrepareExecutableOptions();
	const FValidatorRunExecutableDefinition* TypedDefinition = GetTypedDefinition<FValidatorRunExecutableDefinition>();

	FString SelectedOption = OptionsProvider.GetSelectedOptionKey(ExecutableOptions);
	if(TypedDefinition->bValidateExecutableExists && TypedDefinition->ExecutableCandidates.Num() > 0 && SelectedOption.IsEmpty())
	{
		TArray<FString> ExecutablePaths;
		TypedDefinition->ExecutableCandidates.GenerateValueArray(ExecutablePaths);
		ActivationErrors.Add(FString::Printf(TEXT("[%s] None of the executable candidates exists locally:\n%s"), *ValidatorName, *FString::Join(ExecutablePaths, TEXT("\n"))));
		bIsValidSetup = false;
	}

	FValidatorRunExecutableDefinition* ModifiableDefinition = const_cast<FValidatorRunExecutableDefinition*>(TypedDefinition);

	ModifiableDefinition->ExecutablePath = FConfiguration::SubstituteAndNormalizeFilename(TypedDefinition->ExecutablePath);
	ModifiableDefinition->BuiltRegexError = MakeShared<FRegexPattern>(ModifiableDefinition->RegexErrorParsing, ERegexPatternFlags::CaseInsensitive);
	ModifiableDefinition->BuiltRegexWarning = MakeShared<FRegexPattern>(ModifiableDefinition->RegexWarningParsing, ERegexPatternFlags::CaseInsensitive);

	for (FRegexLockdownTrigger& LockdownTrigger : ModifiableDefinition->LogLockdownTriggers)
	{
		LockdownTrigger.BuiltRegex = MakeShared<FRegexPattern>(LockdownTrigger.RegexPath, ERegexPatternFlags::CaseInsensitive);
	}

	static TArray<FString> ValidExtensions =
	{
#if PLATFORM_WINDOWS
		TEXT(".exe"),
		TEXT(".bat"),
#elif PLATFORM_MAC
		TEXT(".app"),
		TEXT(".sh"),
		TEXT(".command"),
		TEXT(""),
#elif PLATFORM_LINUX
		TEXT(".sh"),
		TEXT(""),
#endif
	};

	if(TypedDefinition->ExecutablePath.IsEmpty())
	{
		for(const TTuple<FString,FString>& Paths : TypedDefinition->ExecutableCandidates)
		{
			if(!ValidExtensions.Contains(FPaths::GetExtension(Paths.Value, true)))
			{
				ActivationErrors.Add(FString::Printf(TEXT("Task '%s' executable has an invalid extension for this platform: %s"), *ValidatorName, *Paths.Value));
				bIsValidSetup = false;
			}
		}

		if(TypedDefinition->ExecutableCandidates.Num() == 0)
		{
			ActivationErrors.Add(FString::Printf(TEXT("Task '%s' does not have a value for 'ExecutablePath' or 'ExecutableCandidates'."), *ValidatorName));
			bIsValidSetup = false;
		}
	}
	else
	{
		if(!ValidExtensions.Contains(FPaths::GetExtension(TypedDefinition->ExecutablePath, true)))
		{
			ActivationErrors.Add(FString::Printf(TEXT("Task '%s' executable has an invalid extension for this platform: %s"), *ValidatorName, *TypedDefinition->ExecutablePath));
			bIsValidSetup = false;
		}

		if(TypedDefinition->bValidateExecutableExists && !FPaths::FileExists(TypedDefinition->ExecutablePath))
		{
			ActivationErrors.Add(FString::Printf(TEXT("Task '%s' executable is not found on disk: %s."), *ValidatorName, *TypedDefinition->ExecutablePath));
			bIsValidSetup = false;
		}
	}

	if(!TypedDefinition->ExecutablePath.IsEmpty() && TypedDefinition->ExecutableCandidates.Num() > 0)
	{
		ActivationErrors.Add(FString::Printf(TEXT("Specifying ExecutablePath and ExecutableCandidates for task %s is not supported, please check your config."), *GetValidatorName()));
	}


	return bIsValidSetup;
}

void FValidatorRunExecutable::StopInternalValidations()
{
	if(GetValidatorState() == EValidationStates::Running)
	{
		for (FProcessWrapper& ProcessWrapper : Processes)
		{
			ProcessWrapper.Stop();
		}
	}
}

bool FValidatorRunExecutable::QueueProcess(const FString& InId, const FString& LocalPath, const FString& Args)
{
	if (!IFileManager::Get().FileExists(*LocalPath))
	{
		LogFailure(FString::Printf(TEXT("[%s (%s)] Process executable doesn't exist: %s"), *GetValidatorName(), *InId, *LocalPath));
		return false;
	}
	
	FProcessTrackingData ProcessData;
	ProcessData.bIgnoringOutputError = !GetTypedDefinition<FValidatorRunExecutableDefinition>()->EnableOutputErrorsAnchor.IsEmpty();
	ProcessData.bIgnoringWarningsAsErrors = !GetTypedDefinition<FValidatorRunExecutableDefinition>()->EnableWarningsAsErrorsAnchor.IsEmpty();
	ProcessesData.FindOrAdd(InId) = ProcessData;

	FOnOutputLine LineHandler = FOnOutputLine::CreateLambda([this, InId](FString InOutput, const EProcessOutputType& InType)
	{
		OnProcessOutputLine(InId, MoveTemp(InOutput), InType);
	});

	FOnCompleted CompleteHandler = FOnCompleted::CreateLambda([this, InId](const int& InResult)
	{
		OnProcessComplete(InId, InResult);
	});
	Processes.Emplace(InId, LocalPath, Args, MoveTemp(CompleteHandler), MoveTemp(LineHandler), FConfiguration::Substitute(TEXT("$(root)")), GetTypedDefinition<FValidatorRunExecutableDefinition>()->bLaunchHidden, GetTypedDefinition<FValidatorRunExecutableDefinition>()->bLaunchReallyHidden);
	return true;
}

void FValidatorRunExecutable::OnProcessOutputLine(const FString& ProcessId, FString&& Line, const EProcessOutputType& OutputType)
{

	const FString& FormattedMessage = FString::Printf(TEXT("[%s] [%s] %s"), *GetValidatorName(), *ProcessId, *Line);
	if (OutputType == EProcessOutputType::ProcessError)
	{
		LogFailure(FormattedMessage);
		ProcessesData[ProcessId].ErrorList.Add(FormattedMessage);
		return;
	}

	for (const FRegexLockdownTrigger& LockdownTrigger : GetTypedDefinition<FValidatorRunExecutableDefinition>()->LogLockdownTriggers)
	{
		FRegexMatcher Regex = FRegexMatcher(*LockdownTrigger.BuiltRegex, Line);
		if (Regex.FindNext() && !LockdownIds.Contains(LockdownTrigger.LockdownId))
		{
			LockdownIds.Add(LockdownTrigger.LockdownId);
			OnLockdownIdsChanged.Broadcast(*this);
		}
	}

	bool bIsWarning = false;
	bool bIsError = IsLineAnError(Line, OutputType, bIsWarning, ProcessesData[ProcessId].bIgnoringOutputError, ProcessesData[ProcessId].bIgnoringWarningsAsErrors);

	if(bIsError)
	{
		// LogFailure already handles telemetry
		LogFailure(FormattedMessage);
		ProcessesData[ProcessId].ErrorList.Add(FormattedMessage);
	}
	else if (bIsWarning)
	{
		FScopeLock Lock(&ErrorLogsMutex);
		ProcessesData[ProcessId].WarningList.Add(FormattedMessage);
		WarningListCache.Add(FormattedMessage);

		UE_LOGF(LogValidatorsResult, Warning, "%ls", *FormattedMessage);
		UE_LOGF(LogValidators, Warning, "%ls", *FormattedMessage);

		// Handle telemetry message in the game thread for this warning
		UE::Tasks::Launch(UE_SOURCE_LOCATION, [this, FormattedMessage]() mutable
		{
			FString TelemetryMsg = FormattedMessage;
			ProcessSingleMessageForTelemetry(TelemetryMsg);

			FTelemetryService::Get()->CustomEvent(TEXT("SubmitTool.Task.Warning"), GetSingleMessageTelemetryAttributes(TelemetryMsg));
		}, LowLevelTasks::ETaskPriority::Normal, UE::Tasks::EExtendedTaskPriority::GameThreadNormalPri);
	}
	else
	{
		UE_LOGF(LogValidators, Log, "%ls", *FormattedMessage);
	}
}

bool FValidatorRunExecutable::IsLineAnError(const FString& InLine, const EProcessOutputType InOutputType, bool& OutbIsWarning, bool& InOutbIgnoringOutputErrors, bool& InOutbIgnoringWarningsAsErrors)
{
	const FValidatorRunExecutableDefinition* TypedDefinition = GetTypedDefinition<FValidatorRunExecutableDefinition>();


	if(!TypedDefinition->EnableOutputErrorsAnchor.IsEmpty() && InLine.Find(TypedDefinition->EnableOutputErrorsAnchor, ESearchCase::IgnoreCase) != INDEX_NONE)
	{
		InOutbIgnoringOutputErrors = false;
		return false;
	}
	
	if(!TypedDefinition->DisableOutputErrorsAnchor.IsEmpty() && InLine.Find(TypedDefinition->DisableOutputErrorsAnchor, ESearchCase::IgnoreCase) != INDEX_NONE)
	{
		InOutbIgnoringOutputErrors = true;
		return false;
	}

	if (!TypedDefinition->EnableWarningsAsErrorsAnchor.IsEmpty() && InLine.Find(TypedDefinition->EnableWarningsAsErrorsAnchor, ESearchCase::IgnoreCase) != INDEX_NONE)
	{
		InOutbIgnoringWarningsAsErrors = false;
		return false;
	}

	if (!TypedDefinition->DisableWarningsAsErrorsAnchor.IsEmpty() && InLine.Find(TypedDefinition->DisableWarningsAsErrorsAnchor, ESearchCase::IgnoreCase) != INDEX_NONE)
	{
		InOutbIgnoringWarningsAsErrors = true;
		return false;
	}

	if(InOutbIgnoringOutputErrors)
	{
		return false;
	}

	for(const FString& Message : TypedDefinition->IgnoredErrorMessages)
	{
		if(InLine.Contains(Message, ESearchCase::IgnoreCase))
		{
			return false;
		}
	}

	for(const FString& Message : TypedDefinition->ErrorMessages)
	{
		if(InLine.Contains(Message, ESearchCase::IgnoreCase))
		{
			return true;
		}
	}

	if (InOutputType == EProcessOutputType::STDErr)
	{
		return true;
	}

	{
		FRegexMatcher Regex = FRegexMatcher(*TypedDefinition->BuiltRegexError, InLine);
		if(Regex.FindNext())
		{
			return true;
		}
	}

	FRegexMatcher Regex = FRegexMatcher(*TypedDefinition->BuiltRegexWarning, InLine);
	if (Regex.FindNext())
	{
		OutbIsWarning = true;

		if(Definition->bTreatWarningsAsErrors && !InOutbIgnoringWarningsAsErrors)
		{
			return true;
		}
	}

	return false;
}

void FValidatorRunExecutable::OnProcessComplete(const FString& ProcessId, int32 ReturnCode)
{
	const FValidatorRunExecutableDefinition* Def = GetTypedDefinition<FValidatorRunExecutableDefinition>();
	const bool Success = Def->AllowedExitCodes.Contains(ReturnCode) && (Def->bOnlyLookAtExitCode || ProcessesData[ProcessId].ErrorList.IsEmpty());

	if (!Def->AllowedExitCodes.Contains(ReturnCode))
	{
		for (const FString& ErrorMsg : Def->NonZeroExitCodeErrorMessages)
		{
			LogFailure(FString::Printf(TEXT("[%s]: %s"), *GetValidatorName(), *ErrorMsg));
		}
	}

	if(Success)
	{
		UE_LOGF(LogValidators, Log, "[%ls] [%ls] Task process succeded (Exit code %d)", *GetValidatorName(), *ProcessId, ReturnCode);
	}
	else
	{
		LogFailure(FString::Printf(TEXT("[%s] [%s] Task process failed with exit code %d and %d log errors."), *GetValidatorName(), *ProcessId, ReturnCode, ProcessesData[ProcessId].ErrorList.Num()));
	}
}

void FValidatorRunExecutable::Tick(float DeltaTime)
{		
	FValidatorBase::Tick(DeltaTime);

	if (GetIsRunning() && Processes.Num() != 0)
	{	
		bool bAnyProcessRunning = false;
		bool bAllFinished = true;
		bool bAllSucceeded = true;

		const FValidatorRunExecutableDefinition* Def = GetTypedDefinition<FValidatorRunExecutableDefinition>();

		for (const FProcessWrapper& Process : Processes)
		{
			bAnyProcessRunning |= Process.IsRunning();
			bAllFinished &= Process.bIsComplete;

			if (Process.bIsComplete)
			{
				FProcessTrackingData& Data = ProcessesData.FindChecked(Process.GetProcessName());
				if (!Data.bSuccess.IsSet())
				{
					bool bSuccess = Def->AllowedExitCodes.Contains(Process.ExitCode) && (Def->bOnlyLookAtExitCode || Data.ErrorList.IsEmpty());
					if (!Def->AllowedExitCodes.Contains(Process.ExitCode))
					{
						FailureReason = EFailureReason::InvalidExitCode;
					}
					else
					{
						FailureReason = EFailureReason::Errors;
					}

					Data.bSuccess.Emplace(bSuccess);
					TRACE_END_REGION(*Process.GetProcessName());
				}
				bAllSucceeded &= Data.bSuccess.GetValue();
			}
		}

		if (bAllFinished)
		{
			ValidationFinished(bAllSucceeded);
		}
		else
		{
			for (FProcessWrapper& Process : Processes)
			{
				if (!Process.bStarted)
				{
					if (GetTypedDefinition<FValidatorRunExecutableDefinition>()->bAllowProcessConcurrency || !bAnyProcessRunning)
					{
						const bool ProcessStarted = Process.Start();
						bAnyProcessRunning = true;
						if (ProcessStarted)
						{
							TRACE_BEGIN_REGION(*Process.GetProcessName(), TEXT("Processes"));
							UE_LOGF(LogValidators, Log, "[%ls] [%ls] %ls", *GetValidatorName(), *Process.GetProcessName(), TEXT("Task process started."));
						}
						else
						{
							FString ErrorMessage = FString::Format(TEXT("Task process failed to start with Process path: '{0}' and arguments: '{1}'"),
								{
									*Process.GetExecutable(),
									*Process.GetArgs()
								});

							LogFailure(FString::Printf(TEXT("[%s] [%s] %s"), *GetValidatorName(), *Process.GetProcessName(),  *ErrorMessage));
						}
					}
				}
			}
		}

	}
}

const TArray<FAnalyticsEventAttribute> FValidatorRunExecutable::GetTelemetryAttributes() const
{
	TArray<FAnalyticsEventAttribute> Attributes = FValidatorBase::GetTelemetryAttributes();
	AppendAnalyticsEventAttributeArray(Attributes, 
		TEXT("ProcessCount"), Processes.Num()
	);

	for (size_t i = 0; i < Processes.Num(); ++i)
	{
		const FProcessWrapper& ProcessWrapper = Processes[i];
		if (i == 0)
		{
			AppendAnalyticsEventAttributeArray(Attributes,
				TEXT("Executable"), ProcessWrapper.GetExecutable(),
				TEXT("Args"), ProcessWrapper.GetArgs(),
				TEXT("Started"), ProcessWrapper.bStarted,
				TEXT("ExeExitCode"), ProcessWrapper.ExitCode,
				TEXT("ExeRunTime"), ProcessWrapper.ExecutingTime,
				TEXT("ErrorCount"), ProcessesData[ProcessWrapper.GetProcessName()].ErrorList.Num()
			);
		}
		else
		{
			AppendAnalyticsEventAttributeArray(Attributes,
				FString::Printf(TEXT("Executable #%i"), i) , ProcessWrapper.GetExecutable(),
				FString::Printf(TEXT("Args #%i"), i), ProcessWrapper.GetArgs(),
				FString::Printf(TEXT("Started #%i"), i), ProcessWrapper.bStarted,
				FString::Printf(TEXT("ExeExitCode #%i"), i), ProcessWrapper.ExitCode,
				FString::Printf(TEXT("ExeRunTime #%i"), i), ProcessWrapper.ExecutingTime,
				FString::Printf(TEXT("ErrorCount #%i"), i), ProcessesData[ProcessWrapper.GetProcessName()].ErrorList.Num()
			);
		}

	}	
	
	return Attributes;		
}

bool FValidatorRunExecutable::DoesExecutableNeedBuilding() const
{
	const FValidatorRunExecutableDefinition* ExeDefinition = GetTypedDefinition<FValidatorRunExecutableDefinition>();
	check(ExeDefinition != nullptr);

	bool bExeNeedsBuilding = false;

	if (IFileManager::Get().FileExists(*ExeDefinition->ExecutablePath))
	{
		FBuildVersion VersionInfo;
		if (FindBuildVersionForExecutable(ExeDefinition->ExecutablePath, VersionInfo))
		{
			if (!VersionInfo.BuildUrl.IsEmpty())
			{
				UE_LOGF(LogValidators, Log, "[%ls] BuildVersion info for '%ls' indicates that it is a precompiled binary", *ValidatorName, *ExeDefinition->ExecutablePath);
				return false;
			}
			else
			{
				UE_LOGF(LogValidators, Log, "[%ls] BuildVersion info for '%ls' indicates that it was built locally", *ValidatorName, *ExeDefinition->ExecutablePath);
				return true;
			}
		}
		else
		{
			UE_LOGF(LogValidators, Warning, "[%ls] Failed to retrieve BuildVersion info for '%ls', assuming that it was locally built", *ValidatorName, *ExeDefinition->ExecutablePath);
			return true;
		}
	}
	else
	{
		UE_LOGF(LogValidators, Log, "[%ls] Failed to find '%ls', so it will need to be built locally", *ValidatorName, *ExeDefinition->ExecutablePath);
		return true;
	}
}

bool FValidatorRunExecutable::FindBuildVersionForExecutable(const FString& ExecutablePath, FBuildVersion& OutBuildVersion) const
{
	const FString VersionPath = FPathViews::ChangeExtension(ExecutablePath, TEXTVIEW("version"));
	if (IFileManager::Get().FileExists(*VersionPath))
	{
		FBuildVersion BuildVersion;
		if (FBuildVersion::TryRead(VersionPath, BuildVersion))
		{
			OutBuildVersion = MoveTemp(BuildVersion);
			return true;
		}
		else
		{
			return false;
		}
	}

	const FString TargetPath = FPathViews::ChangeExtension(ExecutablePath, TEXTVIEW("target"));
	if (IFileManager::Get().FileExists(*TargetPath))
	{
		FString JsonString;
		if (!FFileHelper::LoadFileToString(JsonString, *TargetPath))
		{
			return false;
		}

		TSharedRef<TJsonReader<>> JsonReader = TJsonReaderFactory<>::Create(JsonString);
		TSharedPtr<FJsonObject> JsonRootObject;
		if (!FJsonSerializer::Deserialize(JsonReader, JsonRootObject))
		{
			return false;
		}

		const TSharedPtr<FJsonObject>* JsonVersionObject;
		if (!JsonRootObject->TryGetObjectField(TEXTVIEW("version"), JsonVersionObject))
		{
			return false;
		}

		FString JsonObjectString;
		TSharedRef<TJsonWriter<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>> JsonWriter = TJsonWriterFactory<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>::Create(&JsonObjectString, 0);
		if (!FJsonSerializer::Serialize((*JsonVersionObject).ToSharedRef(), JsonWriter, true))
		{
			return false;
		}

		FBuildVersion BuildVersion;
		if (FBuildVersion::TryReadFromString(JsonObjectString, BuildVersion))
		{
			OutBuildVersion = MoveTemp(BuildVersion);
			return true;
		}
		else
		{
			return false;
		}

	}

	return false;
}

void FValidatorRunExecutable::PrepareExecutableOptions()
{
	const FValidatorRunExecutableDefinition* TypedDefinition = GetTypedDefinition<FValidatorRunExecutableDefinition>();
	if(TypedDefinition->ExecutableCandidates.Num() > 0)
	{
		TMap<FString, FString> Options;
		Options.Reserve(TypedDefinition->ExecutableCandidates.Num());

		FValidatorRunExecutableDefinition* ModifyableDefinition = const_cast<FValidatorRunExecutableDefinition*>(TypedDefinition);

		FString SelectedOption;
		FString* UserSelectedOption = FSubmitToolUserPrefs::Get()->ValidatorOptions.Find(OptionsProvider.GetUserPrefsKey(ExecutableOptions));
		if(UserSelectedOption != nullptr && !UserSelectedOption->StartsWith(TEXT("Auto Select"), ESearchCase::IgnoreCase))
		{
			SelectedOption = *UserSelectedOption;
		}	

		FDateTime LastWriteAccess = FDateTime::MinValue();
		FFileManagerGeneric FileManager;

		FString NewestExecutable;	

		for(TPair<FString, FString>& ExecutableCandidate : ModifyableDefinition->ExecutableCandidates)
		{
			ExecutableCandidate.Value = FConfiguration::SubstituteAndNormalizeFilename(ExecutableCandidate.Value);
			Options.Add(ExecutableCandidate.Key, ExecutableCandidate.Value);

			if(FPaths::FileExists(*ExecutableCandidate.Value))
			{
				if(TypedDefinition->bUseLatestExecutable)
				{
					FFileStatData FileModifiedDate = FileManager.GetStatData(*ExecutableCandidate.Value);
					if(FileModifiedDate.ModificationTime > LastWriteAccess)
					{
						NewestExecutable = ExecutableCandidate.Key;
						LastWriteAccess = FileModifiedDate.ModificationTime;
					}
				}
				else if(SelectedOption.IsEmpty())
				{
					SelectedOption = ExecutableCandidate.Key;
				}
			}
		}

		if(TypedDefinition->bUseLatestExecutable && TypedDefinition->ExecutableCandidates.Contains(NewestExecutable))
		{
			const FString NewestExecutableOptionKey = FString::Printf(TEXT("Auto Select (%s)"), *NewestExecutable);
			Options.Add(NewestExecutableOptionKey, TypedDefinition->ExecutableCandidates[NewestExecutable]);

			if(UserSelectedOption == nullptr || UserSelectedOption->StartsWith(TEXT("Auto Select"), ESearchCase::IgnoreCase))
			{
				SelectedOption = NewestExecutableOptionKey;
			}
		}

		OptionsProvider.InitializeValidatorOptions(ExecutableOptions, Options, SelectedOption, EValidatorOptionType::FilePath);
	}
}
