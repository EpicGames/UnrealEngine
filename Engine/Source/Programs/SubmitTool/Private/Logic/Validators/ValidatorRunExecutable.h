// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ValidatorBase.h"
#include "Containers/StringFwd.h"

class FBuildVersion;

struct FProcessTrackingData
{
	TOptional<bool> bSuccess;
	bool bIgnoringOutputError = false;
	bool bIgnoringWarningsAsErrors = false;
	TArray<FString> ErrorList;
	TArray<FString> WarningList;
};

namespace SubmitToolParseConstants
{
	const FString CustomValidator = TEXT("CustomValidator");
	const FString ExecutableValidator = TEXT("Executable"); // Same as Custom Validator, renaming to keep in line with class
}

/**
 * @brief Runs an executable with optional arguments. Return code 0 on the executable means success.
 * Validator values: "ExecutablePath" (required), "ExecutableArguments" (optional)
 */
class FValidatorRunExecutable : public FValidatorBase
{
public:	
	FValidatorRunExecutable(const FName& InNameId, const FSubmitToolParameters& InParameters, TSharedRef<FSubmitToolServiceProvider> InServiceProvider, const FString& InDefinition);
	virtual void ParseDefinition(const FString& InDefinition) override;

	virtual void StartValidation() override;
	virtual bool Validate(const FString& InCLDescription, const TArray<FSCFileRef>& InFilteredFilesInCL, const TArray<const FTag*>& InTags) override;
	virtual const FString& GetValidatorTypeName() const override { return SubmitToolParseConstants::ExecutableValidator; }
	virtual bool Activate() override;

	virtual void StopInternalValidations() override;
	 
protected:
	virtual bool QueueProcess(const FString& ProcessId, const FString& LocalPath, const FString& Args);
	virtual void OnProcessOutputLine(const FString& ProcessId, FString&& Line, const EProcessOutputType& OutputType);
	virtual bool IsLineAnError(const FString& InLine, const EProcessOutputType OutputType, bool& OutbIsWarning, bool& InOutbIgnoringOutputErrors, bool& InOutbIgnoringWarningsAsErrors);
	virtual void OnProcessComplete(const FString& ProcessId, int32 ReturnCode);
	
	virtual void Tick(float DeltaTime) override;

	virtual const TArray<FAnalyticsEventAttribute> GetTelemetryAttributes() const override;

	bool DoesExecutableNeedBuilding() const;
	bool FindBuildVersionForExecutable(const FString& ExecutablePath, FBuildVersion& OutBuildVersion) const;

	void PrepareExecutableOptions();

	static FString GetProcessName(FStringView ExecutablePath);


protected:
	const FString ExecutableOptions = TEXT("SelectedExecutable");
	TMap<FString, FProcessTrackingData> ProcessesData;
	TArray<FProcessWrapper> Processes;

private:
	static int32 ProcessCounter;
};
