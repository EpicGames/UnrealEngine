// Copyright Epic Games, Inc. All Rights Reserved.

#include "ModelInterface.h"
#include "Logic/ChangelistService.h"
#include "Logic/TagService.h"
#include "Logic/P4LockdownService.h"
#include "Logic/CredentialsService.h"
#include "Algo/Compare.h"
#include "Models/SubmitToolUserPrefs.h"
#include "Logic/JiraService.h"
#include "Logic/PreflightService.h"
#include "Framework/Application/SlateApplication.h"
#include "Version/AppVersion.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Text/STextBlock.h"
#include "Containers/Ticker.h"
#include "Configuration/Configuration.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Async/Async.h"
#include "Misc/StringOutputDevice.h"
#include "Telemetry/TelemetryService.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"

#include "CommandLine/CmdLineParameters.h"
#include "Logic/Validators/ValidatorFactory.h"
#include "Logic/Services/CacheDataService.h"
#include "Logic/Services/SourceControl/SubmitToolPerforce.h"
#include "SubmitToolCoreUtils.h"

ESubmitToolAppState FModelInterface::SubmitToolState = ESubmitToolAppState::Initializing;
FOnStateChanged FModelInterface::OnStateChanged;

FModelInterface::FModelInterface(FSubmitToolParameters&& InParameters, const TSharedPtr<ISTSourceControlService>& InSourceControlService) :
	Parameters(MoveTemp(InParameters)), SourceControlService(InSourceControlService)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FModelInterface::FModelInterface);

	// initialize call backs
	CLReadyCallback = FOnChangeListReadyDelegate::CreateRaw(this, &FModelInterface::OnChangelistReady);
	CLRefreshCallback = FOnChangelistRefreshDelegate::CreateRaw(this, &FModelInterface::OnChangelistRefresh);

	ServiceProvider = MakeShared<FSubmitToolServiceProvider>();
	// Initialize services
	if (Parameters.GeneralParameters.CacheFile.IsEmpty())
	{
		ServiceProvider->RegisterService<ICacheDataService>(MakeShared<FNoOpCacheDataService>());
	}
	else
	{
		ServiceProvider->RegisterService<ICacheDataService>(MakeShared<FCacheDataService>(Parameters.GeneralParameters));
	}

	TArray<FString> DynamicLockdown;
	FCmdLineParameters::Get().GetValueArray(FSubmitToolCmdLine::EnableLocks, DynamicLockdown);
	FixedLockdownIds.Append(MoveTemp(DynamicLockdown));

	SourceControlService->InitializeParameters(Parameters);
	ServiceProvider->RegisterService<ISTSourceControlService>(SourceControlService.ToSharedRef());
	ValidationService = MakeShared<FTasksService>(Parameters.Validators, TEXT("SubmitTool.StandAlone.Validator"));
	ServiceProvider->RegisterService<FTasksService>(ValidationService.ToSharedRef(), TEXT("ValidationService"));
	PresubmitOperationsService = MakeShared<FTasksService>(Parameters.PresubmitOperations, TEXT("SubmitTool.StandAlone.PresubmitOperation"));
	ServiceProvider->RegisterService<FTasksService>(PresubmitOperationsService.ToSharedRef(), TEXT("PresubmitOperationsService"));
	CredentialsService = MakeShared<FCredentialsService>(Parameters);
	ServiceProvider->RegisterService<ICredentialsService>(CredentialsService.ToSharedRef());
	ChangelistService = MakeShared<FChangelistService>(Parameters.GeneralParameters, ServiceProvider, CLReadyCallback, CLRefreshCallback);
	ServiceProvider->RegisterService<IChangelistService>(ChangelistService.ToSharedRef());
	P4LockdownService = MakeShared<FP4LockdownService>(Parameters.P4LockdownParameters, ServiceProvider);
	ServiceProvider->RegisterService<ILockdownService>(P4LockdownService.ToSharedRef());
	TagService = MakeShared<FTagService>(Parameters, ChangelistService);
	ServiceProvider->RegisterService<ITagService>(TagService.ToSharedRef());
	SwarmService = MakeShared<FSwarmService>(ServiceProvider);
	ServiceProvider->RegisterService<FSwarmService>(SwarmService.ToSharedRef());
	PreflightService = MakeShared<FPreflightService>(Parameters.HordeParameters, this, ServiceProvider);
	ServiceProvider->RegisterService<FPreflightService>(PreflightService.ToSharedRef());
	JiraService = MakeShared<FJiraService>(Parameters.JiraParameters, 512, ServiceProvider);
	ServiceProvider->RegisterService<FJiraService>(JiraService.ToSharedRef());
	IntegrationService = MakeShared<FIntegrationService>(Parameters.IntegrationParameters, ServiceProvider);
	ServiceProvider->RegisterService<FIntegrationService>(IntegrationService.ToSharedRef());
	UpdateService = MakeShared<FUpdateService>(Parameters.HordeParameters, Parameters.AutoUpdateParameters, ServiceProvider);
	ServiceProvider->RegisterService<FUpdateService>(UpdateService.ToSharedRef());

	ParseTasksConfig(Parameters.Validators, ValidationService.ToSharedRef());
	ParseTasksConfig(Parameters.PresubmitOperations, PresubmitOperationsService.ToSharedRef());

	OnValidationStateUpdatedHandle = ValidationService->OnTasksRunResultUpdated.Add(FOnTaskRunStateChanged::FDelegate::CreateLambda([this](bool bIsValid) {
		ReevaluateSubmitToolTag();
		if (bIsValid)
		{
			bool bOptionalFailures = false;
			for (const TWeakPtr<const FValidatorBase>& Validator : ValidationService->GetTasks())
			{
				if (!Validator.Pin()->GetIsRunningOrQueued() && !Validator.Pin()->GetHasPassed())
				{
					bOptionalFailures = true;
				}
			}

			if (EvaluateControlledLockdownAllowed())
			{
				UE_LOGF(LogSubmitTool, Log, "The required local validation has succeeded, you're ALLOWED TO SUBMIT.")
			}

			if (ValidationService->GetIsAnyTaskRunning())
			{
				UE_LOGF(LogSubmitTool, Warning, "You still have optional validations running you might want to consider waiting for them to finish.")
			}

			if (bOptionalFailures)
			{
				UE_LOGF(LogSubmitTool, Warning, "You have optional validations that have failed, you can still proceed with the submission if you consider that these failures are not relevant. Please make sure this is the case.")
			}
		}
		}));

	OnSingleValidationFinishedHandle = ValidationService->OnSingleTaskFinished.AddLambda([this](const FValidatorBase& InTask) {
		ReevaluateSubmitToolTag();

		if (bPreflightQueued && CanLaunchPreflight())
		{
			bPreflightQueued = false;
			PreflightService->RequestPreflight();
		}
		});

	OnValidationFinishedHandle = ValidationService->OnTasksQueueFinished.Add(FOnTaskFinished::FDelegate::CreateLambda([this](bool bIsValid)
		{
			if (!MainTab.Pin()->GetParentWindow()->HasAnyUserFocusOrFocusedDescendants())
			{
				MainTab.Pin()->GetParentWindow()->DrawAttention(FWindowDrawAttentionParameters());
			}

			if (bIsValid)
			{
				if (!bPreflightQueued && bSubmitOnSuccessfulValidation && !IsIntegrationRequired())
				{
					bool bAllSucceedIncludingOptional = true;
					for (const TWeakPtr<const FValidatorBase>& Validator : ValidationService->GetTasks())
					{
						if (Validator.IsValid() && !Validator.Pin()->GetHasPassed())
						{
							bAllSucceedIncludingOptional = false;
						}
					}


					if (bAllSucceedIncludingOptional)
					{
						UE_LOGF(LogSubmitTool, Log, "User has opted in to automatically submit on validation successful. Proceeding with submission...");
						StartSubmitProcess(true);
					}
					else
					{
						UE_LOGF(LogSubmitTool, Warning, "User has opted in to automatically submit on validation successful but not all validations succeeded. Fix optional validation errors or submit manually if you want to bypass them.");
						FDialogFactory::ShowInformationDialog(FText::FromString(TEXT("Auto-Submit Cancelled")), FText::FromString(TEXT("Submit tool couldn't auto submit because there were optional validations that failed.\n\nFix these errors or manually submit if you are certain that you should ignore them.")));
						bSubmitOnSuccessfulValidation = false;
					}
				}
			}

		}));

	OnPresubmitFinishedHandle = PresubmitOperationsService->OnTasksQueueFinished.Add(FOnTaskFinished::FDelegate::CreateRaw(this, &FModelInterface::OnPresubmitOperationsComplete));

	FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateRaw(this, &FModelInterface::Tick));
}

FModelInterface::~FModelInterface()
{
	PrepareSubmitCallBack.Clear();
	FileRefreshedCallback.Clear();
	SubmitFinishedCallback.Clear();
	ValidationService->OnTasksRunResultUpdated.Remove(OnValidationStateUpdatedHandle);
	ValidationService->OnTasksQueueFinished.Remove(OnValidationFinishedHandle);
	ValidationService->OnSingleTaskFinished.Remove(OnSingleValidationFinishedHandle);
	PresubmitOperationsService->OnTasksQueueFinished.Remove(OnPresubmitFinishedHandle);


	for (const TWeakPtr<const FValidatorBase>& Task : ValidationService->GetTasks())
	{
		if (Task.IsValid())
		{
			const_cast<FValidatorBase*>(Task.Pin().Get())->OnLockdownIdsChanged.RemoveAll(this);
		}
	}

	ServiceProvider.Reset();
}

void FModelInterface::Dispose() const
{
	ChangelistService->CancelP4Operations();

	if (DescGenProcess != nullptr && DescGenProcess->IsRunning())
	{
		DescGenProcess->Stop();
	}

	if (GetInputEnabled())
	{
		ChangelistService->UpdateP4CLDescriptionSynchronously();
	}

	for (const TPair<FString, TSharedPtr<FIntegrationOptionBase>>& IntegrationOption : ServiceProvider->GetService<FIntegrationService>()->GetIntegrationOptions())
	{
		FString Value;
		if (IntegrationOption.Value->GetJiraValue(Value) && !Value.IsEmpty())
		{
			ServiceProvider->GetService<ICacheDataService>()->SetIntegrationFieldValue(GetCLID(), IntegrationOption.Key, Value);
		}
	}

	ServiceProvider->GetService<ICacheDataService>()->SaveCacheToDisk();
	ValidationService->StopTasks();
}

void FModelInterface::ParseTasksConfig(const TMap<FString, FString>& InTasksConfig, TSharedRef<FTasksService> InOutTaskService)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FModelInterface::ParseTasksConfig);
	TArray<TSharedRef<FValidatorBase>> Tasks;

	for (const TPair<FString, FString>& DefinitionPair : InTasksConfig)
	{
		FValidatorDefinition TaskDefinition;
		FStringOutputDevice Errors;
		FValidatorDefinition::StaticStruct()->ImportText(*DefinitionPair.Value, &TaskDefinition, nullptr, 0, &Errors, FValidatorDefinition::StaticStruct()->GetName());

		if (!Errors.IsEmpty())
		{
			UE_LOGF(LogSubmitTool, Error, "Error loading parameter file %ls", *Errors);
			FModelInterface::SetErrorState();
			continue;
		}

		if (TaskDefinition.Type.TrimStartAndEnd().IsEmpty())
		{
			UE_LOGF(LogSubmitTool, Error, "Task %ls didn't have a Type.", *DefinitionPair.Key);
			continue;
		}

		if (TaskDefinition.bIsDisabled)
		{
			UE_LOGF(LogSubmitToolDebug, Log, "Task %ls was disabled by configuration", *DefinitionPair.Key);
			continue;
		}

		TSharedPtr<FValidatorBase> Task = FValidatorFactory::Get().Create(TaskDefinition.Type, FName(DefinitionPair.Key), Parameters, ServiceProvider.ToSharedRef(), DefinitionPair.Value);

		if (Task == nullptr)
		{
			UE_LOGF(LogSubmitTool, Error, "[%ls] is not a recognized validator type.", *DefinitionPair.Key);
			continue;
		}

		Task->OnLockdownIdsChanged.AddRaw(this, &FModelInterface::DynamicValidatorLockdownRefreshed);
		Tasks.Add(Task.ToSharedRef());
	}

	InOutTaskService->InitializeTasks(Tasks);
}

void FModelInterface::GenerateCLDescription()
{
	if (!Parameters.GeneralParameters.DescriptionGenProcess.IsEmpty() && (DescGenProcess == nullptr || !DescGenProcess->IsRunning()))
	{
		GeneratedDescription.Empty();

		FOnOutputLine LineHandler = FOnOutputLine::CreateLambda([this](FString InOutput, const EProcessOutputType& InType)
		{
			if (InType == EProcessOutputType::ProcessError || InType == EProcessOutputType::STDErr)
			{
				UE_LOGF(LogSubmitTool, Error, "[Description Generation]: %ls", *InOutput);
			}
			else
			{

				if (InOutput.StartsWith(Parameters.GeneralParameters.DescriptionGenParsing))
				{
					UE_LOGF(LogSubmitToolDebug, Log, "[Description Generation]: %ls", *InOutput);
					FString Content = InOutput.RightChop(Parameters.GeneralParameters.DescriptionGenParsing.Len());

					// Try to parse as JSON for multi-line support
					TSharedPtr<FJsonObject> JsonObject;
					TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(Content);
					if (FJsonSerializer::Deserialize(Reader, JsonObject) && JsonObject.IsValid())
					{
						FString DescriptionValue;
						if (JsonObject->TryGetStringField(TEXT("description"), DescriptionValue))
						{
							GeneratedDescription = DescriptionValue;
						}
					}
					else
					{
						// Fallback to plain text for backwards compatibility
						GeneratedDescription = Content;
					}
				}
				else
				{
					UE_LOGF(LogSubmitTool, Log, "[Description Generation]: %ls", *InOutput);
				}
			}
		});

		FOnCompleted CompleteHandler = FOnCompleted::CreateLambda([this](const int& InExitCode)
		{
			FTelemetryService::Get()->CustomEvent(TEXT("SubmitTool.StandAlone.GenerateDescription.Complete"),
				MakeAnalyticsEventAttributeArray(
					TEXT("ExitCode"), InExitCode
				)
			);

			if(InExitCode == 0 && !GeneratedDescription.IsEmpty())
			{
				ChangelistService->SetCLDescription(GeneratedDescription);
				TagService->ReApplyTags();
				ValidateCLDescription();
			}
			else
			{
				UE_LOGF(LogSubmitTool, Error, "[Description Generation]: Process exited with code %d", InExitCode);
			}
		});

		DescGenProcess = MakeUnique<FProcessWrapper>(TEXT("Description Gen Proc"), FConfiguration::Substitute(Parameters.GeneralParameters.DescriptionGenProcess), FConfiguration::Substitute(Parameters.GeneralParameters.DescriptionGenArgs), CompleteHandler, LineHandler);
		DescGenProcess->Start();
	}
}

void FModelInterface::SetCLDescription(const FText& newDescription, bool DoNotInvalidate)
{
	if (ChangelistService->SetCLDescription(newDescription.ToString()))
	{
		TagService->ParseCLDescription();

		if (!DoNotInvalidate)
		{
			ValidationService->InvalidateForChanges(ETaskArea::Changelist);
		}
	}
}

void FModelInterface::SendDescriptionToP4() const
{
	if (GetInputEnabled())
	{
		if (IsP4OperationRunning())
		{
			UE_LOGF(LogSubmitToolP4, Log, "Attempted to send description to P4, but another operation is already running");
			return;
		}

		ChangelistService->UpdateP4CLDescription();
	}
}

bool FModelInterface::CanLaunchPreflight() const
{
	// Check Validators which are validating files, ignore changelist (description, valid tags) validators when we evaluate if we 
	// allow the user to trigger a preflight
	for (const TWeakPtr<const FValidatorBase>& Validator : ValidationService->GetTasks())
	{
		if (Validator.IsValid())
		{
			const TSharedPtr<const FValidatorBase> Pinned = Validator.Pin();
			if (Pinned->Definition->bBlocksPreflightStart)
			{
				if ((Pinned->Definition->IsRequired && !Pinned->GetHasPassed()) || (!Pinned->Definition->IsRequired && Pinned->GetIsRunningOrQueued()))
				{
					return false;
				}
			}
		}
	}

	return true;
}

void FModelInterface::EvaluateDisabledValidatorsTag()
{
	bool bUpdateDescription = false;

	FString DescriptionCopy = ChangelistService->GetCLDescription();
	const FString DisabledTag = TEXT("#DisabledValidations ");

	bUpdateDescription = RemoveTagFromDescription(DisabledTag, DescriptionCopy);

	TStringBuilder<256> DisabledValidatorsString;
	for (const TWeakPtr<const FValidatorBase>& Validator : ValidationService->GetTasks())
	{
		if (Validator.Pin()->Definition->bIsDisabled)
		{
			if (DisabledValidatorsString.Len() != 0)
			{
				DisabledValidatorsString << TEXT(", ");
			}

			DisabledValidatorsString << Validator.Pin()->GetValidatorNameId();
		}
	}

	if (DisabledValidatorsString.Len() != 0)
	{
		DisabledValidatorsString.InsertAt(0, DisabledTag);
		const FString FinalString = DisabledValidatorsString.ToString();
		UE_LOGF(LogSubmitToolDebug, Log, "Added Disabled validators tag %ls", *FinalString);
		DescriptionCopy.Append(TEXT("\n") + FinalString);
		bUpdateDescription = true;
	}


	if (bUpdateDescription)
	{
		TSharedPtr<SWidget> PinnedDescriptionBox = DescriptionBox.Pin();
		if (PinnedDescriptionBox != nullptr && PinnedDescriptionBox->HasKeyboardFocus())
		{
			FSlateApplication::Get().ClearKeyboardFocus();
		}

		ChangelistService->SetCLDescription(DescriptionCopy, true);
		TagService->ParseCLDescription();
	}
}


void FModelInterface::EvaluateFailedValidatorsTag()
{
	bool bUpdateDescription = false;

	FString DescriptionCopy = ChangelistService->GetCLDescription();
	const FString FailedTag = TEXT("#FailedValidations ");

	bUpdateDescription = RemoveTagFromDescription(FailedTag, DescriptionCopy);

	TStringBuilder<256> FailedValidatorsString;
	for (const TWeakPtr<const FValidatorBase>& Validator : ValidationService->GetTasks())
	{
		TSharedPtr<const FValidatorBase> ValidatorPtr = Validator.Pin();

		if (ValidatorPtr != nullptr &&
		ValidatorPtr->Definition->bRegisterFailureInCLDescription &&
		!ValidatorPtr->GetHasPassed() &&
		!ValidatorPtr->Definition->bIsDisabled)
		{
			if (FailedValidatorsString.Len() != 0)
			{
				FailedValidatorsString << TEXT(", ");
			}

			FailedValidatorsString << Validator.Pin()->GetValidatorNameId();
		}
	}

	if (FailedValidatorsString.Len() != 0)
	{
		FailedValidatorsString.InsertAt(0, FailedTag);
		const FString FinalString = FailedValidatorsString.ToString();
		UE_LOGF(LogSubmitToolDebug, Log, "Added Failed validators tag %ls", *FinalString);
		DescriptionCopy.Append(TEXT("\n") + FinalString);
		bUpdateDescription = true;
	}


	if (bUpdateDescription)
	{
		TSharedPtr<SWidget> PinnedDescriptionBox = DescriptionBox.Pin();
		if (PinnedDescriptionBox != nullptr && PinnedDescriptionBox->HasKeyboardFocus())
		{
			FSlateApplication::Get().ClearKeyboardFocus();
		}
		ChangelistService->SetCLDescription(DescriptionCopy, true);
		TagService->ParseCLDescription();
	}
}

void FModelInterface::ReevaluateSubmitToolTag()
{
	UpdateSubmitToolTag(ValidationService->GetIsRunSuccessful(!IsIntegrationRequired()));
}

void FModelInterface::UpdateSubmitToolTag(bool InbAdd)
{
	// add a special tag to the CL description
	FString SubmitToolTag = FString::Format(TEXT("#submittool {0}"), { FAppVersion::GetVersion() });
	FString DescriptionCopy = ChangelistService->GetCLDescription();

	if (InbAdd)
	{
		if (!HasSubmitToolTag())
		{
			TSharedPtr<SWidget> PinnedDescriptionBox = DescriptionBox.Pin();
			if (PinnedDescriptionBox != nullptr && PinnedDescriptionBox->HasKeyboardFocus())
			{
				FSlateApplication::Get().ClearKeyboardFocus();
			}
			UE_LOGF(LogSubmitToolDebug, Log, "Added Submit Tool tag");
			DescriptionCopy.Append(TEXT("\n") + SubmitToolTag);
			ChangelistService->SetCLDescription(DescriptionCopy, true);
			TagService->ParseCLDescription();
		}
	}
	else if (HasSubmitToolTag())
	{
		const FString VersionlessTag = TEXT("#submittool ");

		if (RemoveTagFromDescription(VersionlessTag, DescriptionCopy))
		{
			ChangelistService->SetCLDescription(DescriptionCopy, true);
			TagService->ParseCLDescription();
		}
	}
}


bool FModelInterface::HasSubmitToolTag() const
{
	// Only Checking that it has a submit tool tag, regardless of version.
	return ChangelistService->GetCLDescription().Find(TEXT("#submittool ")) != INDEX_NONE;
}


void FModelInterface::UpdateCLFromP4Async() const
{
	if (GetInputEnabled() || SubmitToolState == ESubmitToolAppState::Errored)
	{
		ChangelistService->FetchChangelistDataAsync();
	}
}

bool FModelInterface::GetInputEnabled()
{
	return SubmitToolState == ESubmitToolAppState::WaitingUserInput;
}

void FModelInterface::RequestPreflight(bool bForceStart)
{
	if (bForceStart || CanLaunchPreflight())
	{
		bPreflightQueued = false;
		PreflightService->RequestPreflight();
	}
	else
	{
		bPreflightQueued = true;
	}
}

void FModelInterface::ShowSwarmReview()
{
	if (HasSwarmReview() && SwarmService.IsValid())
	{
		FString Url;
		if (SwarmService->GetCurrentReviewUrl(Url))
		{
			UE_LOGF(LogSubmitTool, Log, "Swarm: Opening Swarm Review with URL: \"%ls\"", *Url);

			FPlatformProcess::LaunchURL(*Url, nullptr, nullptr);
		}
	}
}

void FModelInterface::RequestSwarmReview(const TArray<FString>& InReviewers)
{
	if (!HasSwarmReview() && SwarmService.IsValid())
	{
		if (!HasShelvedFiles())
		{
			ChangelistService->CreateShelvedFiles([this, InReviewers](const FSCCommand& InCmd)
				{
					if (InCmd.bSuccess && HasShelvedFiles())
					{
						RequestSwarmReview(InReviewers);
					}
					else
					{
						UE_LOGF(LogSubmitTool, Error, "Failed to shelve files, Swarm Review request is cancelled");
					}
				});

			return;
		}

		SwarmService->CreateReview(InReviewers, OnCreateReviewComplete::CreateRaw(this, &FModelInterface::OnSwarmCreateCompleted));
	}
}

void FModelInterface::StartSubmitProcess(bool bSkipShelfDialog)
{
	PresubmitOperationsService->ResetStates();

	// Check if any last minute file changes have come in that invalidated any validators.
	CheckForFileEdits();
	if (IsCLValid())
	{
		if (PrepareSubmitCallBack.IsBound())
		{
			PrepareSubmitCallBack.Broadcast();
		}

		if (!EvaluateControlledLockdownAllowed())
		{
			return;
		}

		TStringBuilder<1024> SubmitOverFailedValidatorsMessages;

		for (const TWeakPtr<const FValidatorBase>& Validator : ValidationService->GetTasks())
		{
			TSharedPtr<const FValidatorBase> ValidatorPtr = Validator.Pin();

			if (ValidatorPtr != nullptr)
			{
				// If it was an active validator that failed, and has a popup message set up, build the message
				if (!ValidatorPtr->Definition->PopupMessageWhenFailed.IsEmpty() &&
					!ValidatorPtr->Definition->bIsDisabled &&
					!ValidatorPtr->GetHasPassed())
				{
					if (SubmitOverFailedValidatorsMessages.Len() != 0)
					{
						SubmitOverFailedValidatorsMessages << TEXT("\n");
					}

					SubmitOverFailedValidatorsMessages << TEXT("- <RichTextBlock.Red>") << ValidatorPtr->GetValidatorName() << TEXT("</> - ") << ValidatorPtr->GetPopupMessageWhenFailedText() << TEXT("\n");
				}
			}
		}

		if (SubmitOverFailedValidatorsMessages.Len() > 0)
		{
			SubmitOverFailedValidatorsMessages.InsertAt(0, TEXT("You have failed important validations, please review each one carefully and confirm you want to continue:\n\n"));

			TArray<FString> Buttons{TEXT("Continue with submission"), TEXT("Cancel")};

			bool bConfirmedSubmission = false;

			TSharedPtr<SHorizontalBox> ConfirmSubmission = SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SCheckBox)
					.IsChecked_Lambda([&bConfirmedSubmission]() { return bConfirmedSubmission ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
					.OnCheckStateChanged_Lambda([&bConfirmedSubmission](ECheckBoxState InNewState)
						{
							bConfirmedSubmission = InNewState == ECheckBoxState::Checked;
						})
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SButton)
					.ButtonStyle(FAppStyle::Get(), "InvisibleButton")
					.IsFocusable(false)
					.OnClicked_Lambda([&bConfirmedSubmission]() { bConfirmedSubmission = !bConfirmedSubmission; return FReply::Handled(); })
					[
						SNew(STextBlock)
							.Justification(ETextJustify::Left)
							.MinDesiredWidth(60)
							.Text(FText::FromString(TEXT("I confirm I want to submit with these failed validations")))
					]
			];

			auto BtnEnabledFunc = [&bConfirmedSubmission](size_t InBtnIdx) 
			{ 
				if (InBtnIdx == static_cast<size_t>(EDialogFactoryResult::Confirm))
				{
					return bConfirmedSubmission;
				}

				return true;
			};

			EDialogFactoryResult DialogResult = FDialogFactory::ShowDialog(FText::FromString(TEXT("Submit with failed validators")), FText::FromString(SubmitOverFailedValidatorsMessages.ToString()), Buttons, ConfirmSubmission, BtnEnabledFunc);

			if (DialogResult != EDialogFactoryResult::Confirm)
			{
				return;
			}
		}

		EvaluateFailedValidatorsTag();
		EvaluateDisabledValidatorsTag();
		UpdateSubmitToolTag(true);

		if (Parameters.IncompatibleFilesParams.IncompatibleFileGroups.Num() > 0)
		{
			const TArray<FSCFileRef>& FilesInCL = ChangelistService->GetFilesInCL();

			for (const FIncompatibleFilesGroup& FileGroup : Parameters.IncompatibleFilesParams.IncompatibleFileGroups)
			{
				TArray<size_t, TInlineAllocator<8>> Indexes;

				for (const FSCFileRef& File : FilesInCL)
				{
					for (size_t i = 0; i < FileGroup.FileGroups.Num(); ++i)
					{
						const FString ReplacedPath = FileGroup.FileGroups[i].Replace(TEXT("$(StreamRoot)"), *GetCurrentStream());
						if (ReplacedPath.Equals(TEXT("*")) || File->GetDepotPath().Contains(ReplacedPath, ESearchCase::IgnoreCase))
						{
							if (!Indexes.Contains(i))
							{
								Indexes.Add(i);
							}
							break;
						}
					}
				}

				if (Indexes.Num() > 1)
				{
					const FText TextTitle = FText::FromString(FileGroup.Title);
					const FText TextDescription = FText::FromString(FileGroup.GetMessage().Replace(TEXT("$(StreamRoot)"), *GetCurrentStream()));

					if (FileGroup.bIsError)
					{
						FDialogFactory::ShowInformationDialog(TextTitle, TextDescription);
						UE_LOGF(LogSubmitTool, Log, "Submission canceled due to incompatible files");
						return;
					}
					else
					{
						if (FDialogFactory::ShowDialog(TextTitle, TextDescription, TArray<FString>{TEXT("Submit"), TEXT("Cancel")}) != EDialogFactoryResult::FirstButton)
						{
							UE_LOGF(LogSubmitTool, Log, "Submission canceled by user");
							return;
						}
					}
				}
			}
		}

		if (HasShelvedFiles())
		{
			if (!bSkipShelfDialog)
			{
				const TArray<FSCFileRef>& ShelvedFiles = ChangelistService->GetShelvedFilesInCL();
				const TArray<FSCFileRef>& LocalFiles = ChangelistService->GetFilesInCL();

				bool bShelfDiffers;

				if (ShelvedFiles.Num() != LocalFiles.Num())
				{
					bShelfDiffers = true;
				}
				else
				{
					// Sort and compare open vs. shelved files
					Algo::StableSortBy(LocalFiles, [](const FSCFileRef& File) { return File->GetDepotPath(); });
					Algo::StableSortBy(ShelvedFiles, [](const FSCFileRef& File) { return File->GetDepotPath(); });
					bShelfDiffers = !Algo::Compare(LocalFiles, ShelvedFiles,
						[](const FSCFileRef& A, const FSCFileRef& B)
						{
							return A->GetDepotPath().Equals(B->GetDepotPath(), ESearchCase::IgnoreCase) && A->GetState() == B->GetState();
						});
				}

				if (bShelfDiffers)
				{
					const FText TextTitle = NSLOCTEXT("SourceControl.SubmitWindow", "ShelveConflictTitle", "Shelve - Local conflict");

					const size_t MaxFilesToList = 5;
					TArray<FString> ShelvedList;
					TArray<FString> LocalList;
					for (size_t i = 0; i < MaxFilesToList; ++i)
					{
						if (i < ShelvedFiles.Num())
						{
							ShelvedList.Add(ShelvedFiles[i]->GetDepotPath());
						}

						if (i < LocalFiles.Num())
						{
							LocalList.Add(LocalFiles[i]->GetDepotPath());
						}
					}
					FString LocalListString = FSubmitToolCoreUtils::StringBuilderJoin<1024>(LocalList, TEXT("\n - "));
					if (LocalFiles.Num() > MaxFilesToList)
					{
						LocalListString = FString::Printf(TEXT("%s\n - And %d other files"), *LocalListString, LocalFiles.Num() - MaxFilesToList);
					}

					FString ShelveListString = FSubmitToolCoreUtils::StringBuilderJoin<1024>(ShelvedList, TEXT("\n - "));
					if (ShelvedFiles.Num() > MaxFilesToList)
					{
						ShelveListString = FString::Printf(TEXT("%s\n - And %d other files"), *ShelveListString, ShelvedFiles.Num() - MaxFilesToList);
					}

					const FString Description =
						FString::Printf(TEXT("The shelve filelist does not match the local filelist, due to p4 restrictions submit tool can only submit local content do you want to continue with the submit?\nLocal Files:\n - %s\n\nShelved Files:\n - %s"),
							*LocalListString,
							*ShelveListString);
					if (FDialogFactory::ShowDialog(TextTitle, FText::FromString(Description), TArray<FString>{TEXT("Delete Shelve and Submit"), TEXT("Cancel")}) != EDialogFactoryResult::Confirm)
					{
						UE_LOGF(LogSubmitTool, Log, "Submission canceled by user");
						return;
					}
				}
			}

			ChangeState(ESubmitToolAppState::Submitting, LockdownStatus.LockdownType != ELockdownType::None && LockdownStatus.bIsAllowlisted);
			ChangelistService->DeleteShelvedFiles(
				[this](const FSCCommand& InCmd) {
					check(SubmitToolState == ESubmitToolAppState::Submitting);
					if (InCmd.bSuccess)
					{
						Submit();
					}
					else
					{
						ChangeState(ESubmitToolAppState::WaitingUserInput);
					}
				}
			);
		}
		else
		{
			ChangeState(ESubmitToolAppState::Submitting, LockdownStatus.LockdownType != ELockdownType::None && LockdownStatus.bIsAllowlisted);
			Submit();
		}
	}
	else
	{
		UE_LOGF(LogSubmitTool, Warning, "Attempted to submit, but all validators have not passed. Aborting submit.");
	}
}

void FModelInterface::RequestIntegration()
{
	if (IsCLValid())
	{
		EvaluateFailedValidatorsTag();
		EvaluateDisabledValidatorsTag();
		UpdateSubmitToolTag(IsCLValid());
		IntegrationService->RequestIntegration(FOnBooleanValueChanged::CreateLambda([this](bool bSuccess)
			{
				if (bSuccess)
				{
					ChangeState(ESubmitToolAppState::Finished);
				}
			}));
	}
}
void FModelInterface::RefreshStateBasedOnFiles()
{
	TArray<FSCFileRef> LocalFiles = ChangelistService->GetFilesInCL();

	if (LocalFiles.IsEmpty())
	{
		const TArray<FSCFileRef>& ShelvedFiles = ChangelistService->GetShelvedFilesInCL();
		if (ShelvedFiles.IsEmpty())
		{
			UE_LOGF(LogSubmitTool, Error, "There are no files in CL %ls, SUBMIT IS DISABLED", *ChangelistService->GetCLID());
			ChangeState(ESubmitToolAppState::Errored);
		}
		else
		{
			if (!HasSubmitToolTag())
			{
				UE_LOGF(LogSubmitTool, Warning, "This CL hasn't been validated and there are no local files. You need to unshelve and run validations.");
				ChangeState(ESubmitToolAppState::Errored);
			}
			else
			{
				// Copy the shelved file depot paths on the main thread since they are not safe to reference from other threads
				UE::Tasks::Launch(UE_SOURCE_LOCATION, [this, ShelvedFilesDepotPaths = ChangelistService->GetShelvedFilesInCL()] {
					FSubmitToolLockdownData LockdownData = P4LockdownService->ArePathsInLockdown(ShelvedFilesDepotPaths, FixedLockdownIds);
					UE::Tasks::Launch(UE_SOURCE_LOCATION, [this, LockdownData = MoveTemp(LockdownData)]() mutable
					{
						LockdownStatus = MoveTemp(LockdownData);
						TitleMessage = FSubmitToolCoreUtils::StringBuilderJoin(LockdownStatus.GroupMessages, TEXT("\n"));
						if (LockdownStatus.LockdownType == ELockdownType::Hardcore)
						{
							UE_LOGF(LogSubmitTool, Log, "There are no local files in CL %ls, Submit is disabled but you can still request an Integration with your shelved files", *ChangelistService->GetCLID());
							ChangeState(ESubmitToolAppState::WaitingUserInput);
						}
						else
						{
							UE_LOGF(LogSubmitTool, Error, "There are no files in CL %ls, SUBMIT IS DISABLED", *ChangelistService->GetCLID());
							ChangeState(ESubmitToolAppState::Errored);
						}
					}, LowLevelTasks::ETaskPriority::Normal, UE::Tasks::EExtendedTaskPriority::GameThreadNormalPri);
				});
			}
		}
	}
	else
	{
		const TArray<FSCCStream*>& Streams = SourceControlService->GetClientStreams();
		if (!Streams.IsEmpty())
		{
			FString StreamsMsg = FString::JoinBy(Streams, TEXT(" -> "), [](const FSCCStream* InStr) { return InStr->Name; });
			for (const FSCFileRef& File : ChangelistService->GetFilesInCL())
			{
				bool bMappedToView = false;

				for (const FSCCStream* Str : Streams)
				{
					if (File->GetDepotPath().StartsWith(Str->Name))
					{
						bMappedToView = true;
						break;
					}

					for (const FString& ImportStream : Str->AdditionalImportPaths)
					{
						if (File->GetDepotPath().StartsWith(ImportStream))
						{
							bMappedToView = true;
							break;
						}
					}
				}

				if (!bMappedToView)
				{
					UE_LOGF(LogSubmitTool, Warning, "File %ls is not in the stream that the workspace is set to: %ls", *File->GetDepotPath(), *StreamsMsg);
				}
			}
		}

		// Copy the file depot paths on the main thread since they are not safe to reference from other threads
		UE::Tasks::Launch(UE_SOURCE_LOCATION, [this, FilesDepotPaths = ChangelistService->GetFilesInCL()] {
			FSubmitToolLockdownData LockdownData = P4LockdownService->ArePathsInLockdown(FilesDepotPaths, FixedLockdownIds);
			UE::Tasks::Launch(UE_SOURCE_LOCATION, [this, LockdownData = MoveTemp(LockdownData)]() mutable
			{
				LockdownStatus = MoveTemp(LockdownData);
				TitleMessage = FSubmitToolCoreUtils::StringBuilderJoin(LockdownStatus.GroupMessages, TEXT("\n"));
				ChangeState(ESubmitToolAppState::WaitingUserInput);
			}, LowLevelTasks::ETaskPriority::Normal, UE::Tasks::EExtendedTaskPriority::GameThreadNormalPri);
		});
	}
}

void FModelInterface::OnChangelistReady(bool bIsValid)
{
	if (SubmitToolState == ESubmitToolAppState::Initializing)
	{
		if (bIsValid)
		{
			UE_LOGF(LogSubmitTool, Log, "Retrieved information for CL %ls", *ChangelistService->GetCLID());
			PreflightService->FetchPreflightInfo(true);

			TagService->ParseCLDescription();
			SwarmService->FetchReview(OnGetReviewComplete::CreateRaw(this, &FModelInterface::OnGetUsersFromSwarmCompleted));

			RefreshStateBasedOnFiles();
			if (!ChangelistService->GetFilesInCL().IsEmpty())
			{
				UpdateSubmitToolTag(false);
				ValidationService->CheckForTagSkips();

				ETaskArea ValidateArea = ~ETaskArea::Changelist;
				for (const FTag* Tag : TagService->GetTagsArray())
				{
					if (Tag->GetValues().Num() != 0)
					{
						ValidateArea = ETaskArea::Everything;
						break;
					}
				}

				if (!UpdateService->CheckForNewVersion())
				{
					// Delay auto-queuing of validators to allow the UI to be initialized
					FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda([this, ValidateArea](float) { ValidateByArea(ValidateArea); return false; }), 0.1f);
				}
			}

			FileRefreshedCallback.Broadcast();
		}
		else
		{
			UE_LOGF(LogSubmitTool, Error, "Couldn't retrieve information for CL %ls", *ChangelistService->GetCLID());
			ChangeState(ESubmitToolAppState::Errored);
		}
	}
}

void FModelInterface::OnChangelistRefresh(ETaskArea InChangeType)
{
	// If we're waiting for a shelf, check again here to detect same file number but different content when files were refreshed in P4V
	if (!ChangelistService->IsShelfReady())
	{
		ChangelistService->CheckShelfIsUpToDate(nullptr);
	}

	if (InChangeType != ETaskArea::None)
	{
		if ((InChangeType & ETaskArea::Changelist) == ETaskArea::Changelist)
		{
			TagService->ParseCLDescription();
		}

		if ((InChangeType & ETaskArea::LocalFiles) == ETaskArea::LocalFiles)
		{
			FileRefreshedCallback.Broadcast();
		}

		if ((InChangeType & (ETaskArea::LocalFiles | ETaskArea::ShelvedFiles)) != ETaskArea::None)
		{
			RefreshStateBasedOnFiles();
		}

		ValidationService->InvalidateForChanges(InChangeType);
	}
}

void FModelInterface::Submit()
{
	if (PresubmitOperationsService->AreTasksPendingQueue())
	{
		// This will call submit again when it's done
		PresubmitOperationsService->CheckForTagSkips();
		if (PresubmitOperationsService->QueueAll())
		{
			return;
		}
	}

	auto AddendumAccumulator = [](const TArray<FString>& InAddendums, const FString& InDescription, FString& InOutAccumulated) {
		for (const FString& Str : InAddendums)
		{
			if (!InDescription.Contains(Str, ESearchCase::IgnoreCase))
			{
				InOutAccumulated += (TEXT("\n") + Str);
			}
		}
		};

	const FString& CLDescription = GetCLDescription();
	FString Addendums;
	AddendumAccumulator(ValidationService->GetAddendums(), CLDescription, Addendums);
	AddendumAccumulator(PresubmitOperationsService->GetAddendums(), CLDescription, Addendums);
	AddendumAccumulator(Parameters.GeneralParameters.AutomaticChangelistTags, CLDescription, Addendums);
	Addendums = FConfiguration::Substitute(Addendums);

	ChangelistService->Submit(Addendums, 
		[this](const FSCCommand& InCmd) {
			OnSubmitOperationComplete(InCmd);
		}		
	);
}

void FModelInterface::OnPresubmitOperationsComplete(bool bInSuccess)
{
	if (bInSuccess)
	{
		Submit();
	}
	else
	{
		UE_LOGF(LogSubmitTool, Warning, "Presubmit operations have failed, submission is not possible, please fix errors and try again.");
		PresubmitOperationsService->ResetStates();
	}
}


void FModelInterface::OnSubmitOperationComplete(const FSCCommand& InCmd)
{
	if (InCmd.bSuccess)
	{
		const FString* SubmittedCLString = nullptr;
		for (const FSCCRecord& Record : InCmd.Values)
		{
			SubmittedCLString = Record.Find(TEXT("submittedChange"));
			if (SubmittedCLString != nullptr)
			{
				UE_LOGF(LogSubmitToolP4, Log, "Submitted CL: %ls", **SubmittedCLString);
				break;
			}
		}

		for (const TWeakPtr<const FValidatorBase>& Task : ValidationService->GetTasks())
		{
			if (Task.Pin()->Definition->bIsDisabled)
			{
				FTelemetryService::Get()->CustomEvent(TEXT("SubmitTool.StandAlone.Submit.ValidatorDisabled"),
					MakeAnalyticsEventAttributeArray(
						TEXT("TaskId"), Task.Pin()->GetValidatorName()
					)
				);
			}
		}	

		FString SubmittedCL;
		int64 CLNumber;
		if (SubmittedCLString == nullptr || !LexTryParseString(CLNumber, **SubmittedCLString))
		{
			CLNumber = 0;
			SubmittedCL = TEXT("0");
		}
		else
		{
			SubmittedCL = *SubmittedCLString;
		}


		TArray<FAnalyticsEventAttribute> SubmitSuccessAttribs = MakeAnalyticsEventAttributeArray(
			TEXT("Stream"), SourceControlService->GetCurrentStreamName(),
			TEXT("SubmittedCL"), MoveTemp(SubmittedCL),
			TEXT("SubmittedCLNumber"), MoveTemp(CLNumber));

		for (const TWeakPtr<const FValidatorBase>& Validator : ValidationService->GetTasks())
		{
			const TSharedPtr<const FValidatorBase> Ptr = Validator.Pin();
			if (Ptr != nullptr)
			{
				EValidationStates State = Ptr->Definition->bIsStealthMode ? Ptr->GetStealthState() : Ptr->GetState();
				AppendAnalyticsEventAttributeArray(SubmitSuccessAttribs,
					FString::Printf(TEXT("V_%s"), *Ptr->GetValidatorNameId().ToString()), StaticEnum<EValidationStates>()->GetNameStringByValue(static_cast<int64>(State)).Replace(TEXT("_"), TEXT(" ")));
			}
		}

		FTelemetryService::Get()->SubmitSucceeded(MoveTemp(SubmitSuccessAttribs));
		
		// We've submitted, or tried to submit and failed so we only let the user close the app
		ChangeState(ESubmitToolAppState::Finished);
		if(FSubmitToolUserPrefs::Get()->bOpenJiraOnSubmit)
		{			
			if (SubmittedCLString != nullptr && SubmittedCLString->IsNumeric())
			{
				FPlatformApplicationMisc::ClipboardCopy(**SubmittedCLString);
				UE_LOGF(LogSubmitToolP4, Log, "Submitted CL copied to clipboard: %ls", **SubmittedCLString);
			}


			FTag* JiraTag = TagService->GetTagOfType(TEXT("JiraIssue"));
			if (JiraTag != nullptr && JiraTag->GetValues().Num() != 0)
			{
				for (const FString& JiraValue : JiraTag->GetValues())
				{
					if (!JiraValue.Equals(TEXT("none"), ESearchCase::IgnoreCase) && !JiraValue.Equals(TEXT("nojira"), ESearchCase::IgnoreCase))
					{
						FString Url = FString::Printf(TEXT("https://%s/browse/%s"), *Parameters.JiraParameters.ServerAddress, *JiraValue);
						FPlatformProcess::LaunchURL(*Url, nullptr, nullptr);
					}
				}
			}
		}

		SubmitFinishedCallback.Broadcast(true);

		if (FSubmitToolUserPrefs::Get()->bCloseOnSubmit)
		{
			MainTab.Pin()->RequestCloseTab();
		}
	}
	else
	{
		ChangeState(ESubmitToolAppState::WaitingUserInput);
		SubmitFinishedCallback.Broadcast(false);
	}
}

bool FModelInterface::Tick(float InDeltaTime)
{
	switch (SubmitToolState)
	{
		case ESubmitToolAppState::WaitingUserInput:
			if (SwarmService->IsRequestRunning() || JiraService->IsBlockingRequestRunning())
			{
				ChangeState(ESubmitToolAppState::P4BlockingOperation);
			}
			break;

		case ESubmitToolAppState::P4BlockingOperation:
			if (!SwarmService->IsRequestRunning() && !ChangelistService->HasP4OperationsRunning() && !JiraService->IsBlockingRequestRunning())
			{
				ChangeState(ESubmitToolAppState::WaitingUserInput);
			}
			break;

		default:
			break;
	}

	return true;
}

void FModelInterface::ChangeState(ESubmitToolAppState newState, bool bForce)
{
	ensure(IsInGameThread());
	if (!IsInGameThread())
	{
		AsyncTask(ENamedThreads::GameThread, [newState, bForce] { ChangeState(newState,bForce); });
		return;
	}

	const ESubmitToolAppState CurrentState = SubmitToolState;
	if (bForce)
	{
		UE_LOGF(LogSubmitToolDebug, Log, "Transitioned state from '%ls' to '%ls'", *StaticEnum<ESubmitToolAppState>()->GetNameStringByValue(static_cast<int64>(SubmitToolState)), *StaticEnum<ESubmitToolAppState>()->GetNameStringByValue(static_cast<int64>(newState)));
		SubmitToolState = newState;
		OnStateChanged.Broadcast(CurrentState, SubmitToolState);
	}
	else
	{
		if (SubmitToolAppState::AllowedTransitions.Contains(SubmitToolState))
		{
			const TArray<ESubmitToolAppState>& allowedStates = SubmitToolAppState::AllowedTransitions[SubmitToolState];
			if (allowedStates.Contains(newState))
			{
				UE_LOGF(LogSubmitToolDebug, Log, "Transitioned state from '%ls' to '%ls'", *StaticEnum<ESubmitToolAppState>()->GetNameStringByValue(static_cast<int64>(SubmitToolState)), *StaticEnum<ESubmitToolAppState>()->GetNameStringByValue(static_cast<int64>(newState)));
				SubmitToolState = newState;
				OnStateChanged.Broadcast(CurrentState, SubmitToolState);
			}
			else
			{
				UE_LOGF(LogSubmitToolDebug, Warning, "Invalid state transition requested from '%ls' to '%ls'", *StaticEnum<ESubmitToolAppState>()->GetNameStringByValue(static_cast<int64>(SubmitToolState)), *StaticEnum<ESubmitToolAppState>()->GetNameStringByValue(static_cast<int64>(newState)));
			}
		}
		else
		{
			UE_LOGF(LogSubmitToolDebug, Warning, "Transition not allowed from '%ls' to '%ls'", *StaticEnum<ESubmitToolAppState>()->GetNameStringByValue(static_cast<int64>(SubmitToolState)), *StaticEnum<ESubmitToolAppState>()->GetNameStringByValue(static_cast<int64>(newState)));
		}
	}
}

void FModelInterface::OnGetUsersFromSwarmCompleted(const TUniquePtr<FSwarmReview>& InReview, const FString& InErrorMessage)
{
	if (!InReview.IsValid())
	{
		UE_LOGF(LogSubmitTool, Log, "Could not retrieve swarm review for current changelist. %ls", *InErrorMessage);
		return;
	}

	TArray<const FTag*> TargetTags;

	for (const FTag* Tag : TagService->GetTagsArray())
	{
		if (Tag->Definition.InputSubType.Equals(TEXT("SwarmApproved"), ESearchCase::IgnoreCase))
		{
			TargetTags.Add(Tag);
		}
	}

	if (!TargetTags.IsEmpty())
	{
		TArray<FString> SwarmUserValues;

		for (const TPair<FString, FSwarmReviewParticipant>& Participant : InReview->Participants)
		{
			if (Participant.Key.Equals(InReview->Author, ESearchCase::IgnoreCase))
			{
				continue;
			}

			if (Participant.Value.Vote.Value == 1)
			{
				if (!SwarmUserValues.Contains(Participant.Key) && !SwarmUserValues.Contains(TEXT("@") + Participant.Key))
				{
					SwarmUserValues.Add(Participant.Key);
				}
			}
		}

		if (!SwarmUserValues.IsEmpty())
		{
			bool bApplied = false;

			for (const FTag* Tag : TargetTags)
			{
				if (Tag->GetValues() != SwarmUserValues)
				{
					SetTagValues(*Tag, SwarmUserValues);
					bApplied = true;
				}
			}

			if (bApplied)
			{
				UE_LOGF(LogSubmitTool, Log, "RB tag set to users that upvoted review '%d' Users: %ls", InReview->Id, *FSubmitToolCoreUtils::StringBuilderJoin<128>(SwarmUserValues, TEXT(", ")));
				UE_LOGF(LogSubmitToolDebug, Log, "Re-running Tag validator after applying the #rb from swarm");
				ValidateCLDescription();
			}
		}
	}
}

void FModelInterface::OnSwarmCreateCompleted(bool InResult, const FString& InErrorMessage)
{
	if (InResult)
	{
		OnGetUsersFromSwarmCompleted(SwarmService->GetReview(), InErrorMessage);
		ShowSwarmReview();
	}
}


void FModelInterface::ValidateChangelist() const
{
	TArray<const TWeakPtr<const FValidatorBase>> Validators;
	for (const TWeakPtr<const FValidatorBase>& Validator : ValidationService->GetTasks())
	{
		const TSharedPtr<const FValidatorBase> SharedValidator = Validator.Pin();
		if (SharedValidator != nullptr && !SharedValidator->Definition->bIsDisabled && SharedValidator->IsRelevantToCL())
		{
			Validators.Add(Validator);
		}
	}

	PreValidate(Validators);
	ValidationService->QueueAll();
}

void FModelInterface::ValidateSingle(const FName& ValidatorId, bool bForce) const
{
	TArray<const TWeakPtr<const FValidatorBase>> Validators;
	for (const TWeakPtr<const FValidatorBase>& Validator : ValidationService->GetTasks())
	{
		const TSharedPtr<const FValidatorBase> SharedValidator = Validator.Pin();
		if (SharedValidator != nullptr && SharedValidator->GetValidatorNameId() == ValidatorId && !SharedValidator->Definition->bIsDisabled && SharedValidator->IsRelevantToCL())
		{
			Validators.Add(Validator);
		}
	}

	PreValidate(Validators);
	ValidationService->QueueSingle(ValidatorId, bForce);
}

void FModelInterface::ValidateByArea(const ETaskArea InValidateArea) const
{
	TArray<const TWeakPtr<const FValidatorBase>> Validators;
	for (const TWeakPtr<const FValidatorBase>& Validator : ValidationService->GetTasks())
	{
		const TSharedPtr<const FValidatorBase> SharedValidator = Validator.Pin();
		if (SharedValidator != nullptr && (SharedValidator->Definition->TaskArea & InValidateArea) != ETaskArea::None && !SharedValidator->Definition->bIsDisabled && SharedValidator->IsRelevantToCL())
		{
			Validators.Add(Validator);
		}
	}
	PreValidate(Validators);
	ValidationService->QueueByArea(InValidateArea);
}

void FModelInterface::PreValidate(const TArray<const TWeakPtr<const FValidatorBase>>& InValidators) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FModelInterface::PreValidate);
	bool bRequiresShelf = false;
	for (const TWeakPtr<const FValidatorBase>& Validator : InValidators)
	{
		const TSharedPtr<const FValidatorBase> SharedValidator = Validator.Pin();
		if (SharedValidator != nullptr && !SharedValidator->Definition->bIsDisabled && SharedValidator->IsRelevantToCL())
		{
			bRequiresShelf |= SharedValidator->Definition->bRequireShelveOperation;
		}
	}

	if (bRequiresShelf)
	{
		ChangelistService->EnsureShelfIsCurrent();
	}
}

void FModelInterface::ToggleValidator(const FName& ValidatorId)
{
	ValidationService->ToggleEnabled(ValidatorId);
	EvaluateDisabledValidatorsTag();
}

bool FModelInterface::RemoveTagFromDescription(const FString& InTag, FString& InOutCLDescription) const
{
	int32 Loc = InOutCLDescription.Find(InTag);
	if (Loc != INDEX_NONE)
	{
		size_t EndPos = Loc + InTag.Len();
		while (EndPos < InOutCLDescription.Len())
		{
			if (InOutCLDescription[EndPos] == TCHAR('\n'))
			{
				++EndPos;
				break;
			}
			++EndPos;
		}

		if (Loc != 0 && InOutCLDescription[Loc - 1] == '\n' && (EndPos - InOutCLDescription.Len()) < 2)
		{
			Loc--;
		}

		UE_LOGF(LogSubmitToolDebug, Log, "Removed tag %ls", *InOutCLDescription.Mid(Loc, EndPos - Loc));
		InOutCLDescription.RemoveAt(Loc, EndPos - Loc);
		return true;
	}

	return false;
}

const FString& FModelInterface::GetTitleMessage() const
{
	if (TitleMessage.IsEmpty())
	{
		if (FDateTime::Now().GetHour() < Parameters.GeneralParameters.EarlySubmitHour24 || FDateTime::Now().GetHour() >= Parameters.GeneralParameters.LateSubmitHour24)
		{
			TitleMessage = Parameters.GeneralParameters.OutOfHoursSubmitMessage.Replace(TEXT("$(DateTime)"), *FDateTime::Now().ToFormattedString(TEXT("%H:%M")));
		}
	}

	return TitleMessage;
}

void FModelInterface::DynamicValidatorLockdownRefreshed(const FValidatorBase& InValidator)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FModelInterface::DynamicValidatorLockdownRefreshed);
	bool bNeedsUpdate = false;
	const TSet<FString>& NewSet = InValidator.GetLockdownIds();

	if (DynamicValidatorLockdownIds.Contains(InValidator.GetValidatorNameId()))
	{
		const TSet<FString>& CurrentSet = DynamicValidatorLockdownIds[InValidator.GetValidatorNameId()];
		bNeedsUpdate = NewSet.Num() != CurrentSet.Num() || !CurrentSet.Includes(NewSet);
	}
	else
	{
		bNeedsUpdate = !NewSet.IsEmpty();
	}

	if (bNeedsUpdate)
	{
		if (NewSet.IsEmpty())
		{
			DynamicValidatorLockdownIds.Remove(InValidator.GetValidatorNameId());
		}
		else
		{
			DynamicValidatorLockdownIds.FindOrAdd(InValidator.GetValidatorNameId()) = NewSet;
		}

		ReevaluateControlledLockdown();
	}
}

void FModelInterface::ReevaluateControlledLockdown()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FModelInterface::ReevaluateControlledLockdown);
	TSet<FString> NewLockdownIds = FixedLockdownIds;
	for (const TPair<FName, TSet<FString>>& ValidatorLockdownIds : DynamicValidatorLockdownIds)
	{
		NewLockdownIds.Append(ValidatorLockdownIds.Value);
	}

	UE::Tasks::Launch(UE_SOURCE_LOCATION, [this, FilesDepotPaths = ChangelistService->GetFilesInCL(), NewLockdownIds = MoveTemp(NewLockdownIds)]() mutable {
		FSubmitToolLockdownData LockdownData = P4LockdownService->ArePathsInLockdown(FilesDepotPaths, MoveTemp(NewLockdownIds));
		UE::Tasks::Launch(UE_SOURCE_LOCATION, [this, LockdownData = MoveTemp(LockdownData)]() mutable
			{
				LockdownStatus = MoveTemp(LockdownData);
				TitleMessage = FSubmitToolCoreUtils::StringBuilderJoin(LockdownStatus.GroupMessages, TEXT("\n"));
				EvaluateControlledLockdownAllowed();
			}, LowLevelTasks::ETaskPriority::Normal, UE::Tasks::EExtendedTaskPriority::GameThreadNormalPri);
		});
}

bool FModelInterface::EvaluateControlledLockdownAllowed()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FModelInterface::EvaluateControlledLockdownAllowed);
	if (LockdownStatus.LockdownType == ELockdownType::Controlled && !LockdownStatus.bIsAllowlisted)
	{
		bool bFillsRequirements = true;
		for (const TPair<FString, TSet<FString, FCaseInsensitiveStringKeyFuncs>>& RequiredTag : LockdownStatus.RequiredTags)
		{

			FTagDefinition Definition;
			Definition.TagId = RequiredTag.Key;
			FTag TransientTag(Definition);
			TransientTag.ParseTag(ChangelistService->GetCLDescription());

			bool bFillsTagRequirements = TransientTag.IsEnabled() && RequiredTag.Value.Num() == 0;
			for (const FString& Value : TransientTag.GetValues())
			{
				if (RequiredTag.Value.Contains(Value))
				{
					bFillsTagRequirements = true;
					break;
				}
			}

			if (!bFillsTagRequirements)
			{
				if (RequiredTag.Value.Num() == 0)
				{
					UE_LOGF(LogSubmitTool, Error, "Your CL contains changes under controlled lockdown %ls requiring the tag %ls", *FSubmitToolCoreUtils::StringBuilderJoin(LockdownStatus.GroupNames, TEXT(", ")), *RequiredTag.Key);
				}
				else
				{
					UE_LOGF(LogSubmitTool, Error, "Your CL contains changes under controlled lockdown %ls requiring the tag %ls followed by one of approved allowlisters:\n%ls", *FSubmitToolCoreUtils::StringBuilderJoin(LockdownStatus.GroupNames, TEXT(", ")), *RequiredTag.Key, *FSubmitToolCoreUtils::StringBuilderJoin(RequiredTag.Value, TEXT(", ")));
				}
			}

			bFillsRequirements &= bFillsTagRequirements;
		}

		if (bFillsRequirements || LockdownStatus.RequiredTags.Num() == 0)
		{
			bIsControlledLock = false;
			return !bIsControlledLock;
		}
		else
		{
			if (!bIsControlledLock)
			{
				UE_LOGF(LogSubmitTool, Error, "Additional info:\n%ls", *FSubmitToolCoreUtils::StringBuilderJoin(LockdownStatus.GroupMessages, TEXT("\n")));
			}

			bIsControlledLock = true;
			return !bIsControlledLock;
		}
	}

	bIsControlledLock = false;
	return !bIsControlledLock;
}
