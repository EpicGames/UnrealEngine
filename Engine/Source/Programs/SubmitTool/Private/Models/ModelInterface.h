// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Logic/TasksService.h"
#include "Logic/Services/Interfaces/ISTSourceControlService.h"
#include "Logic/Services/Interfaces/IChangelistService.h"
#include "Logic/Services/Interfaces/ITagService.h"
#include "Logic/Services/Interfaces/ILockdownService.h"
#include "Logic/Services/Interfaces/ICredentialsService.h"
#include "Logic/JiraService.h"
#include "Logic/IntegrationService.h"
#include "Logic/SwarmService.h"
#include "Logic/PreflightService.h"
#include "Logic/UpdateService.h"
#include "Logic/Services/SubmitToolServiceProvider.h"
#include "Models/Tag.h"
#include "Models/PreflightData.h"

#include "Parameters/SubmitToolParameters.h"

#include "ModelInterface.generated.h"
class SDockTab;

UENUM()
enum class ESubmitToolAppState : uint8 {
	None = 0,
	Initializing = 1,
	WaitingUserInput = 2,
	Errored = 3,
	P4BlockingOperation = 4,
	Submitting = 5,
	Finished = 7,
};

namespace SubmitToolAppState {

	using StateList = const TArray<ESubmitToolAppState>;

	// Allowed states to transition from the original state.
	const TMap<const ESubmitToolAppState, StateList> AllowedTransitions =
	{
		{ ESubmitToolAppState::Initializing, StateList {
				ESubmitToolAppState::WaitingUserInput,
				ESubmitToolAppState::P4BlockingOperation,
				ESubmitToolAppState::Errored
		}},
		{ ESubmitToolAppState::WaitingUserInput, StateList {
				ESubmitToolAppState::Submitting,
				ESubmitToolAppState::Errored,
				ESubmitToolAppState::P4BlockingOperation
		}},
		{ ESubmitToolAppState::Errored, StateList {
			ESubmitToolAppState::WaitingUserInput
		}},
		{ ESubmitToolAppState::P4BlockingOperation, StateList {
				ESubmitToolAppState::WaitingUserInput,
				ESubmitToolAppState::Errored,
				ESubmitToolAppState::Finished
		}},
		{ ESubmitToolAppState::Submitting, StateList {
				ESubmitToolAppState::WaitingUserInput,
				ESubmitToolAppState::Errored,
				ESubmitToolAppState::Finished
		}},
		{ ESubmitToolAppState::Finished, StateList {
			ESubmitToolAppState::Errored
		}}
	};
}

DECLARE_MULTICAST_DELEGATE(FPreSubmitCallBack)
DECLARE_MULTICAST_DELEGATE(FFilesRefresh)
DECLARE_MULTICAST_DELEGATE_OneParam(FOnSubmit, bool /*bSuccess*/)
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnStateChanged, const ESubmitToolAppState /*InFrom*/, const ESubmitToolAppState /*InTo*/)

class FModelInterface
{
public:
	FModelInterface(FSubmitToolParameters&& InParameters, const TSharedPtr<ISTSourceControlService>& InSourceControlService);
	~FModelInterface();
	void Dispose() const;
	void ParseTasksConfig(const TMap<FString, FString>& InTasksConfig, TSharedRef<FTasksService> InOutTaskService);

	void SetMainTab(TSharedPtr<SDockTab> InMainTab)									{ MainTab = InMainTab; }
	TWeakPtr<SDockTab> GetMainTab() const											{ return MainTab; }
	void SetDescriptionBox(const TWeakPtr<SWidget> InDescriptionBox)				{ DescriptionBox = InDescriptionBox; }

	void ApplyTag(const FString& tagID) const										{ TagService->ApplyTag(tagID);}
	void ApplyTag(const FTag& tag) const											{ TagService->ApplyTag(const_cast<FTag&>(tag)); ValidationService->InvalidateForChanges(ETaskArea::Changelist); }
	void RemoveTag(const FString& tagID) const										{ TagService->RemoveTag(tagID); }
	void RemoveTag(const FTag& tag) const											{ TagService->RemoveTag(const_cast<FTag&>(tag)); ValidationService->InvalidateForChanges(ETaskArea::Changelist); }
	void SetTagValues(const FString& tagID, const FString& values) const			{ TagService->SetTagValues(tagID, values); }
	void SetTagValues(const FTag& tag, const FString& values) const					{ TagService->SetTagValues(const_cast<FTag&>(tag), values); ValidationService->InvalidateForChanges(ETaskArea::Changelist); }
	void SetTagValues(const FTag& tag, const TArray<FString>& values) const			{ TagService->SetTagValues(const_cast<FTag&>(tag), values); ValidationService->InvalidateForChanges(ETaskArea::Changelist); }
	void UpdateTagsInCL() const														{ TagService->UpdateTagsInCL(); ValidationService->InvalidateForChanges(ETaskArea::Changelist); }
	const FTag* GetTag(const FString& tagID) const									{ return TagService->GetTag(tagID); };
	const TArray<const FTag*>& GetTagsArray() const									{ return TagService->GetTagsArray(); }
	void RegisterTagUpdatedCallback(const FTagUpdated::FDelegate Callback)			{ TagService->OnTagUpdated.Add(Callback); }

	void SetCLDescription(const FText& newDescription, bool DoNotInvalidate = false);
	void GenerateCLDescription();
	void SendDescriptionToP4() const;
	void UpdateCLFromP4Async() const;
	const FString& GetCLDescription() const											{ return ChangelistService->GetCLDescription(); }
	const FString GetCLID() const													{ return ChangelistService->GetCLID(); }

	FOnCLDescriptionUpdated& GetCLDescriptionUpdatedDelegate()						{ return ChangelistService->OnCLDescriptionUpdated; }
	
	const TArray<FSCFileRef>& GetFilesInCL() const									{ return ChangelistService->GetFilesInCL(); }
	bool HasShelvedFiles() const													{ return ChangelistService->HasShelvedFiles(); }

	bool IsP4OperationRunning() const												{ return ChangelistService->HasP4OperationsRunning(); }
	bool IsBlockingOperationRunning() const											{ return SwarmService->IsRequestRunning() || JiraService->IsBlockingRequestRunning() || P4LockdownService->IsBlockingOperationRunning(); }
	void CancelP4Operations(FName OperationName = FName()) const					{ ChangelistService->CancelP4Operations(); SwarmService->CancelOperations(); }

	void ValidateChangelist() const;
	void ValidateSingle(const FName& ValidatorId, bool bForce = true) const;
	void ValidateByArea(const ETaskArea InValidateArea) const;
	void ToggleValidator(const FName& ValidatorId);
	void ValidateCLDescription() const												{ ValidationService->StopTasksByArea(ETaskArea::Changelist); ValidationService->QueueByArea(ETaskArea::Changelist); }
	bool IsCLValid() const															{ return ValidationService->GetIsRunSuccessful(!IsIntegrationRequired()); }
	bool CanLaunchPreflight() const;
	void EvaluateDisabledValidatorsTag();
	void EvaluateFailedValidatorsTag();
	void ReevaluateSubmitToolTag();
	void UpdateSubmitToolTag(bool InbAdd);
	bool HasSubmitToolTag() const;
	bool IsValidationRunning() const												{ return ValidationService->GetIsAnyTaskRunning(); }
	const TArray<TWeakPtr<const FValidatorBase>>& GetValidators() const				{ return ValidationService->GetTasks(); }
	const TArray<TWeakPtr<const FValidatorBase>>& GetPreSubmitOperations() const	{ return PresubmitOperationsService->GetTasks(); }
	void CancelValidations(const FName& InValidatorId = FName(), bool InbAsFailed = false) const { ValidationService->StopTasks(InValidatorId, InbAsFailed); }
	void CheckForFileEdits() const													{ ValidationService->CheckForLocalFileEdit(); }

	bool IsDescriptionGenerationRunning() const										{ return DescGenProcess != nullptr && DescGenProcess->IsRunning(); }

	FDelegateHandle AddSingleValidatorFinishedCallback
		(const FOnSingleTaskFinished::FDelegate& Callback) const					{ return ValidationService->OnSingleTaskFinished.Add(Callback); }

	void RemoveSingleValidatorFinishedCallback(const FDelegateHandle& Handle) const { ValidationService->OnSingleTaskFinished.Remove(Handle); }

	FDelegateHandle AddValidationFinishedCallback
		(const FOnTaskFinished::FDelegate& Callback) const							{ return ValidationService->OnTasksQueueFinished.Add(Callback); }

	void RemoveValidationFinishedCallback(const FDelegateHandle& Handle) const		{ ValidationService->OnTasksQueueFinished.Remove(Handle); }
	
	FDelegateHandle AddValidationUpdatedCallback
		(const FOnTaskRunStateChanged::FDelegate& Callback) const					{ return ValidationService->OnTasksRunResultUpdated.Add(Callback); }
	void RemoveValidationUpdatedCallback(const FDelegateHandle& Handle) const		{ ValidationService->OnTasksRunResultUpdated.Remove(Handle); }

	void GetUsers(const FOnUsersGet::FDelegate& Callback) const						{ return SourceControlService->GetUsers(Callback); }
	const TArray<TSharedPtr<FUserData>>& GetRecentUsers() const						{ return SourceControlService->GetRecentUsers(); }
	void AddRecentUser(TSharedPtr<FUserData>& User)									{ SourceControlService->AddRecentUser(User); }

	void GetGroups(const FOnGroupsGet::FDelegate& Callback) const					{ return SourceControlService->GetGroups(Callback); }
	const TArray<TSharedPtr<FString>>& GetRecentGroups() const						{ return SourceControlService->GetRecentGroups(); }
	void AddRecentGroup(TSharedPtr<FString>& Group)									{ SourceControlService->AddRecentGroup(Group); }
	FString GetUsername() const														{ return CredentialsService->GetUsername(); }
	const FString GetRootStreamName() const											{ return SourceControlService->GetRootStreamName(); }
	const FString GetCurrentStream() const											{ return SourceControlService->GetCurrentStreamName(); }

	void SetLogin(const FString& InUsername, const FString& InPassword)				{ CredentialsService->SetLogin(InUsername, InPassword); }

	static bool GetInputEnabled();
	static void SetErrorState()														{ ChangeState(ESubmitToolAppState::Errored);}
	static const ESubmitToolAppState GetState()										{ return SubmitToolState; }
	void DeleteShelvedFiles() const													{ ChangelistService->DeleteShelvedFiles(); }
	void ReplaceShelvedFiles() const												{ ChangelistService->ReplaceShelvedFiles(); }


	void RequestPreflight(bool bForceStart = false);
	bool IsPreflightRequestInProgress() const										{ return PreflightService->IsRequestInProgress(); }
	bool IsShelfReady() const														{ return ChangelistService->IsShelfReady(); }
	bool IsPreflightQueued() const													{ return bPreflightQueued; }
	void RefreshPreflightInformation() const										{ PreflightService->FetchPreflightInfo(); UE_LOGF(LogSubmitTool, Log, "Requesting preflight information..."); }
	const TUniquePtr<FPreflightList>& GetPreflightData()							{ return PreflightService->GetPreflightData(); }


	void ShowSwarmReview();
	void RequestSwarmReview(const TArray<FString>& InReviewers);
	FDelegateHandle AddPreflightUpdateCallback
	(const FOnPreflightDataUpdated::FDelegate& Callback) const						{ return PreflightService->OnPreflightDataUpdated.Add(Callback); }

	void RemovePreflightUpdateCallback(const FDelegateHandle& Handle) const			{ PreflightService->OnPreflightDataUpdated.Remove(Handle); }

	void StartSubmitProcess(bool bSkipShelfDialog = false);

	TSharedRef<FSubmitToolServiceProvider> GetServiceProvider() { return ServiceProvider.ToSharedRef(); }

	TWeakPtr<FJiraService> GetJiraService() { return this->JiraService; }
	TWeakPtr<FSwarmService> GetSwarmService() { return this->SwarmService; }
	TWeakPtr<FPreflightService> GetPreflightService() { return this->PreflightService; }

	FSubmitToolLockdownData LockdownStatus;
	TSet<FString> FixedLockdownIds;
	TMap<FName, TSet<FString>> DynamicValidatorLockdownIds;
	void DynamicValidatorLockdownRefreshed(const FValidatorBase& InValidator);
	void ReevaluateControlledLockdown();
	bool EvaluateControlledLockdownAllowed();

	const FSubmitToolParameters& GetParameters() const { return Parameters; }

	const FString& GetTitleMessage() const;
	bool IsIntegrationRequired() const { return LockdownStatus.LockdownType == ELockdownType::Hardcore; };
	bool IsSubmissionBlocked() const { return bIsControlledLock; }
	void RequestIntegration();
	const TMap<FString, TSharedPtr<FIntegrationOptionBase>>& GetIntegrationOptions() { return IntegrationService->GetIntegrationOptions(); }
	bool ValidateIntegrationOptions(bool bSilent) const	
	{
		if (IntegrationService->ValidateIntegrationOptions(bSilent))
		{
			if (!HasSubmitToolTag())
			{
				ValidateCLDescription();
			}

			return true;
		}
		return false;
	}


	static FOnStateChanged OnStateChanged;
	FOnSubmit SubmitFinishedCallback;
	FPreSubmitCallBack PrepareSubmitCallBack;
	FFilesRefresh FileRefreshedCallback;
	bool bSubmitOnSuccessfulValidation = false;

	const TUniquePtr<FSwarmReview>& GetSwarmReview() const		{ return SwarmService->GetReview(); }
	const void RefreshSwarmReview()								{ return SwarmService->FetchReview(OnGetReviewComplete::CreateRaw(this, &FModelInterface::OnGetUsersFromSwarmCompleted)); }

	bool GetSwarmReviewUrl(FString& OutUrl) const	{ return SwarmService->GetCurrentReviewUrl(OutUrl); }

	// AUTO-UPDATE
	bool IsAutoUpdateOn() const { return Parameters.AutoUpdateParameters.bIsAutoUpdateOn; }
	bool CheckForNewVersion() { return UpdateService->CheckForNewVersion(); }
	FString GetDeployId() const { return UpdateService->GetDeployId(); }
	FString GetLocalVersion() const { return UpdateService->GetLocalVersion(); }
	FString GetLatestVersion() const { return UpdateService->GetLatestVersion(); }
	void InstallLatestVersion() { UpdateService->InstallLatestVersion(); }
	void CancelInstallLatestVersion() { UpdateService->Cancel(); }
	const FString GetDownloadMessage() { return UpdateService->GetDownloadMessage(); }

	const bool IsSwarmServiceValid() { return SwarmService.IsValid(); }
	const bool HasSwarmReview()
	{
		if (SwarmService.IsValid())
		{
			const TUniquePtr<FSwarmReview>& Review = SwarmService->GetReview();
			if(Review.IsValid())
			{
				return Review->Id != 0;
			}
		}
		return false;
	}

private:	
	const FSubmitToolParameters Parameters;
	void OnChangelistRefresh(ETaskArea InChangeType);
	void PreValidate(const TArray<const TWeakPtr<const FValidatorBase>>& InValidators) const;
	void RefreshStateBasedOnFiles();
	void OnChangelistReady(bool bIsValid);
	void Submit();
	void OnPresubmitOperationsComplete(bool bInSuccess);
	void OnSubmitOperationComplete(const FSCCommand& InCmd);
	void OnGetUsersFromSwarmCompleted(const TUniquePtr<FSwarmReview>& InReview, const FString& InErrorMessage);
	void OnSwarmCreateCompleted(bool InResult, const FString& InErrorMessage);
	bool RemoveTagFromDescription(const FString& InTag, FString& InOutCLDescription) const;

	FOnChangeListReadyDelegate CLReadyCallback;
	FOnChangelistRefreshDelegate CLRefreshCallback;
	FDelegateHandle OnValidationStateUpdatedHandle;
	FDelegateHandle OnValidationFinishedHandle;
	FDelegateHandle OnPresubmitFinishedHandle;
	FDelegateHandle OnSingleValidationFinishedHandle;
	TUniquePtr<FProcessWrapper> DescGenProcess;
	FString GeneratedDescription;

	TSharedPtr<ISTSourceControlService> SourceControlService;
	TSharedPtr<IChangelistService> ChangelistService;
	TSharedPtr<ILockdownService> P4LockdownService;
	TSharedPtr<ITagService> TagService;
	TSharedPtr<FTasksService> ValidationService;
	TSharedPtr<FJiraService> JiraService;
	TSharedPtr<FPreflightService> PreflightService;
	TWeakPtr<SDockTab> MainTab;
	TWeakPtr<SWidget> DescriptionBox;
	TSharedPtr<FTasksService> PresubmitOperationsService;
	TSharedPtr<FIntegrationService> IntegrationService;
	TSharedPtr<FSwarmService> SwarmService;
	TSharedPtr<ICredentialsService> CredentialsService;
	TSharedPtr<FUpdateService> UpdateService;
	TSharedPtr<FSubmitToolServiceProvider> ServiceProvider;
	bool bPreflightQueued = false;
	bool bIsControlledLock = false;
	mutable FString TitleMessage;
	bool Tick(float InDeltaTime);

	static ESubmitToolAppState SubmitToolState;

	static void ChangeState(ESubmitToolAppState newState, bool bForce = false);
};
