// Copyright Epic Games, Inc. All Rights Reserved.

#include "SubmitToolParametersBuilder.h"

#include "SubmitToolCoreUtils.h"
#include "Configuration/Configuration.h"

#include "CoreGlobals.h"
#include "Models/ModelInterface.h"
#include "HAL/FileManager.h"
#include "Logging/SubmitToolLog.h"
#include "CommandLine/CmdLineParameters.h"
#include "Misc/ConfigContext.h"
#include "Misc/StringOutputDevice.h"

FSubmitToolParametersBuilder::FSubmitToolParametersBuilder()
{
	ConfigHierarchy.Add({ TEXT("Base"), TEXT("{ENGINE}/Config/Base.ini") });
	ConfigHierarchy.Add( { TEXT("SubmitToolBase"), TEXT("{PROJECT}/Config/{TYPE}.ini") } );
	ConfigHierarchy.Add( { TEXT("Platform"), TEXT("{PROJECT}/Config/{PLATFORM}/{PLATFORM}{TYPE}.ini") } );

#ifdef SUBMIT_TOOL_CONFIG_DEVOVERRIDE
	ConfigHierarchy.Add({ TEXT("SubmitToolVersionDevOverride"), TEXT("{PROJECT}/Config/{TYPE}DevOverride.ini") });
	ConfigHierarchy.Add({ TEXT("PlatformVersionDevOverride"), TEXT("{PROJECT}/Config/{PLATFORM}/{PLATFORM}{TYPE}DevOverride.ini") });
#endif

	FString RootDir;
	FCmdLineParameters::Get().GetValue(FSubmitToolCmdLine::RootDir, RootDir);
	FPaths::NormalizeDirectoryName(RootDir);

	// allocate a string since the FConfigLayer expects static TCHAR*, so we can't let the string go away
	FString* RootBase = new FString(FPaths::Combine(RootDir, TEXT("/Config/{TYPE}.ini")));
	ConfigHierarchy.Add({ TEXT("RootBase"), **RootBase, EConfigLayerFlags::NoExpand });

	FString* RootPlatform = new FString(FPaths::Combine(RootDir, TEXT("/Config/{PLATFORM}/{PLATFORM}{TYPE}.ini")));
	ConfigHierarchy.Add({ TEXT("RootPlatform"), **RootPlatform });

	if(!RootDir.IsEmpty())
	{
		FString* EngineBase = new FString(FPaths::Combine(RootDir, TEXT("/Engine/Restricted/NotForLicensees/Config/{TYPE}.ini")));
		ConfigHierarchy.Add({ TEXT("EngineBase"), **EngineBase, EConfigLayerFlags::NoExpand });

		FString* EnginePlatform = new FString(FPaths::Combine(RootDir, TEXT("Engine/Restricted/NotForLicensees/Config/{PLATFORM}/{PLATFORM}{TYPE}.ini")));
		ConfigHierarchy.Add({ TEXT("EnginePlatform"), **EnginePlatform });

		IFileManager::Get().IterateDirectory(*FConfiguration::Substitute(TEXT("$(root)")),
			[this](const TCHAR* FileOrDir, bool bIsDir) -> bool
			{
				if(bIsDir)
				{
					FString Dir = { FileOrDir };
					if(Dir != TEXT("SubmitTool") && !Dir.Contains(TEXT("Engine"), ESearchCase::IgnoreCase))
					{
						const FString Extension = TEXT(".uproject");
						TArray<FString> UProjects;
						IFileManager::Get().FindFiles(UProjects, FileOrDir, *Extension);

						if(UProjects.Num() != 0)
						{
							// allocate a string since the FConfigLayer expects static TCHAR*, so we can't let the string go away
							FString* ProjectIni = new FString(FPaths::Combine(Dir, TEXT("/Config/{TYPE}.ini")));
							ConfigHierarchy.Add( { TEXT("Project"), **ProjectIni, EConfigLayerFlags::NoExpand } );

							FString* ProjectPlatform = new FString(FPaths::Combine(Dir, TEXT("/Config/{PLATFORM}/{PLATFORM}{TYPE}.ini")));
							ConfigHierarchy.Add({ TEXT("ProjectPlatform"), **ProjectPlatform });

							// Let's hold on to this info for the preflight parameters
							for (const FString& UProjectName : UProjects)
							{
								ProjectNames.Add(UProjectName.LeftChop(Extension.Len()));
							}
						}
					}
				}

				return true;
			});
	}
		
	// allocate a string since the FConfigLayer expects static TCHAR*, so we can't let the string go away
	FString* UserIni = new FString(FPaths::Combine(FSubmitToolCoreUtils::GetLocalAppDataPath(), "SubmitTool", "SubmitTool.ini"));
	ConfigHierarchy.Add( { TEXT("User"), **UserIni, EConfigLayerFlags::NoExpand } );
}

FSubmitToolParameters FSubmitToolParametersBuilder::LoadConfigFromFiles()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FSubmitToolParametersBuilder::LoadConfigFromFiles);
	FConfigContext Context = FConfigContext::ReadIntoGConfig();
	Context.OverrideLayers = ConfigHierarchy;

	FString IniFilename;
	Context.Load(TEXT("SubmitTool"), IniFilename);
	FConfigFile* SubmitToolConfig = GConfig->FindConfigFile(IniFilename);

	UE_LOGF(LogSubmitTool, Verbose, "Loading config from the following files:");
	for(const TPair<int32, FUtf8String>& OverrideLayer : Context.Branch->Hierarchy)
	{
		FString FullString(OverrideLayer.Value);
		if(IFileManager::Get().FileExists(*FullString))
		{
			UE_LOGF(LogSubmitTool, Verbose, "Priority: %d - File:%ls", OverrideLayer.Key, *FPaths::ConvertRelativePathToFull(FullString));
		}
	}

	FSubmitToolParameters Parameters;

	if (SubmitToolConfig == nullptr)
	{
		UE_LOGF(LogSubmitTool, Error, "Failed to load config file");
		return Parameters;
	}

	Parameters.GeneralParameters = BuildGeneralParameters(*SubmitToolConfig);
	Parameters.JiraParameters = BuildJiraParameters(*SubmitToolConfig);
	Parameters.Telemetry = GetTelemetryParameters(*SubmitToolConfig);
	Parameters.IntegrationParameters = BuildIntegrationParameters(*SubmitToolConfig);
	Parameters.AvailableTags = BuildAvailableTags(*SubmitToolConfig);
	Parameters.Validators = BuildValidators(*SubmitToolConfig);
	Parameters.PresubmitOperations = BuildPresubmitOperations(*SubmitToolConfig);
	Parameters.CopyLogParameters = BuildCopyLogParameters(*SubmitToolConfig);
	Parameters.P4LockdownParameters = BuildP4LockdownParameters(*SubmitToolConfig);
	Parameters.OAuthParameters = BuildOAuthParameters(*SubmitToolConfig);
	Parameters.IncompatibleFilesParams = BuildIncompatibleFilesParameters(*SubmitToolConfig);
	Parameters.HordeParameters = BuildHordeParameters(*SubmitToolConfig);
	Parameters.AutoUpdateParameters = BuildAutoUpdateParameters(*SubmitToolConfig);
	return Parameters;
}

FGeneralParameters FSubmitToolParametersBuilder::ReadGeneralParametersFromLocalConfig()
{
	FConfigFile LocalConfigFile;
	FConfigContext Context = FConfigContext::ReadSingleIntoLocalFile(LocalConfigFile);

	FString IniFilename;
	Context.Load(TEXT("SubmitTool"), IniFilename);

	return BuildGeneralParameters(LocalConfigFile);
}

FGeneralParameters FSubmitToolParametersBuilder::BuildGeneralParameters(const FConfigFile& InConfigFile)
{
	const FConfigSection* Section = InConfigFile.FindSection(TEXT("SubmitTool.General"));
	FGeneralParameters Output;

	if(Section != nullptr)
	{
		FStringOutputDevice Errors;
		FGeneralParameters::StaticStruct()->ImportText(*SectionToText(*Section), &Output, nullptr, 0, &Errors, FGeneralParameters::StaticStruct()->GetName());

		if(!Errors.IsEmpty())
		{
			UE_LOGF(LogSubmitTool, Error, "Error loading parameter file %ls", *Errors);
			FModelInterface::SetErrorState();
		}
		else
		{
			for(const TPair<FString,FString>& Pair : Output.PathOverrides)
			{				
				FConfiguration::AddOrUpdateEntry(Pair.Key, Pair.Value);
			}

			if (!FPaths::IsStaged())
			{
				for (const TPair<FString, FString>& Pair : Output.PathOverridesInSourceBuild)
				{
					FConfiguration::AddOrUpdateEntry(Pair.Key, Pair.Value);
				}
			}

			Output.CacheFile = FConfiguration::SubstituteAndNormalizeFilename(Output.CacheFile);
		}
	}

	return Output;
}

FJiraParameters FSubmitToolParametersBuilder::BuildJiraParameters(const FConfigFile& InConfigFile)
{
	const FConfigSection* Section = InConfigFile.FindSection(TEXT("SubmitTool.Jira"));
	FJiraParameters Output;

	if(Section != nullptr)
	{
		FStringOutputDevice Errors;
		FJiraParameters::StaticStruct()->ImportText(*SectionToText(*Section), &Output, nullptr, 0, &Errors, FJiraParameters::StaticStruct()->GetName());

		if(!Errors.IsEmpty())
		{
			UE_LOGF(LogSubmitTool, Error, "Error loading parameter file %ls", *Errors);
			FModelInterface::SetErrorState();
		}
	}

	return Output;
}

 FTelemetryParameters FSubmitToolParametersBuilder::GetTelemetryParameters(const FConfigFile& InConfigFile)
{
	 const FConfigSection* Section = InConfigFile.FindSection(TEXT("SubmitTool.Telemetry"));
	 FTelemetryParameters Output;

	 if (Section != nullptr)
	 {
		 FStringOutputDevice Errors;
		 FTelemetryParameters::StaticStruct()->ImportText(*SectionToText(*Section), &Output, nullptr, 0, &Errors, FTelemetryParameters::StaticStruct()->GetName());

		 if (!Errors.IsEmpty())
		 {
			 UE_LOGF(LogSubmitTool, Error, "Error loading parameter file %ls", *Errors);
			 FModelInterface::SetErrorState();
		 }
	 }

	 return Output;
}

 FIntegrationParameters FSubmitToolParametersBuilder::BuildIntegrationParameters(const FConfigFile& InConfigFile)
 {
	 const FConfigSection* Section = InConfigFile.FindSection(TEXT("SubmitTool.FNIntegration"));
	 FIntegrationParameters Output;

	 if(Section != nullptr)
	 {
		 FStringOutputDevice Errors;
		 FIntegrationParameters::StaticStruct()->ImportText(*SectionToText(*Section), &Output, nullptr, 0, &Errors, FIntegrationParameters::StaticStruct()->GetName());

		 if(!Errors.IsEmpty())
		 {
			 UE_LOGF(LogSubmitTool, Error, "Error loading parameter file %ls", *Errors);
			 FModelInterface::SetErrorState();
		 }
	 }

	 return Output;
 }

TArray<FTagDefinition> FSubmitToolParametersBuilder::BuildAvailableTags(const FConfigFile& InConfigFile)
{
	static const TCHAR* TagsSectionName = TEXT("Tags.");
	TArray<FTagDefinition> Output;

	for(const TPair<FString, FConfigSection>& Section : InConfigFile)
	{
		if(Section.Key.StartsWith(TagsSectionName))
		{
			FTagDefinition Definition;
			FStringOutputDevice Errors;
			FTagDefinition::StaticStruct()->ImportText(*SectionToText(Section.Value), &Definition, nullptr, 0, &Errors, FValidatorDefinition::StaticStruct()->GetName());

			if (Definition.bIsDisabled)
			{
				UE_LOGF(LogSubmitToolDebug, Verbose, "Skipped tag due to it being disabled %ls", *Definition.TagId);
				continue;
			}

			if(!Definition.DocumentationUrl.IsEmpty())
			{
				Definition.ToolTip += TEXT("\nClick the icon for more information.");
			}

			if(!Errors.IsEmpty())
			{
				UE_LOGF(LogSubmitTool, Error, "Error loading parameter file %ls", *Errors);
				FModelInterface::SetErrorState();
			}
			else
			{
				UE_LOGF(LogSubmitToolDebug, Verbose, "Added Tag %ls", *Definition.TagId);
				Output.Add(Definition);
			}
		}
	}

	Output.Sort([](const FTagDefinition& A, const FTagDefinition& B)
		{
			return A.OrdinalOverride <= B.OrdinalOverride;
		});

	return Output;
}

TMap<FString, FString> FSubmitToolParametersBuilder::BuildValidators(const FConfigFile& InConfigFile)
{
	static const TCHAR* ValidatorsSectionName = TEXT("Validator.");

	TMap<FString, FString> Output;

	for(const TPair<FString, FConfigSection>& Section : InConfigFile)
	{
		if(Section.Key.StartsWith(ValidatorsSectionName))
		{
			Output.Add(Section.Key.Replace(ValidatorsSectionName, TEXT("")), SectionToText(Section.Value));
		}
	}

	return Output;
}

TMap<FString, FString> FSubmitToolParametersBuilder::BuildPresubmitOperations(const FConfigFile& InConfigFile)
{
	static const TCHAR* PresubmitOperationsSectionName = TEXT("PresubmitOperation.");

	TMap<FString, FString> Output;

	for(const TPair<FString, FConfigSection>& Section : InConfigFile)
	{
		if(Section.Key.StartsWith(PresubmitOperationsSectionName))
		{
			Output.Add(Section.Key.Replace(PresubmitOperationsSectionName, TEXT("")), SectionToText(Section.Value));
		}
	}

	return Output;
}

FCopyLogParameters FSubmitToolParametersBuilder::BuildCopyLogParameters(const FConfigFile& InConfigFile)
{
	const FConfigSection* Section = InConfigFile.FindSection(TEXT("SubmitTool.CopyLog"));
	FCopyLogParameters Output;
	
	if(Section != nullptr)
	{
		FStringOutputDevice Errors;
		FCopyLogParameters::StaticStruct()->ImportText(*SectionToText(*Section), &Output, nullptr, 0, &Errors, FCopyLogParameters::StaticStruct()->GetName());

		if(!Errors.IsEmpty())
		{
			UE_LOGF(LogSubmitTool, Error, "Error loading parameter file %ls", *Errors);
			FModelInterface::SetErrorState();
		}
	}

	return Output;
}
FP4LockdownParameters FSubmitToolParametersBuilder::BuildP4LockdownParameters(const FConfigFile& InConfigFile)
{
	const FConfigSection* Section = InConfigFile.FindSection(TEXT("SubmitTool.P4Lockdown"));
	FP4LockdownParameters Output;

	if(Section != nullptr)
	{
		FStringOutputDevice Errors;
		FP4LockdownParameters::StaticStruct()->ImportText(*SectionToText(*Section), &Output, nullptr, 0, &Errors, FP4LockdownParameters::StaticStruct()->GetName());

		if(!Errors.IsEmpty())
		{
			UE_LOGF(LogSubmitTool, Error, "Error loading parameter file %ls", *Errors);
			FModelInterface::SetErrorState();
		}
	}

	return Output;
}

FOAuthTokenParams FSubmitToolParametersBuilder::BuildOAuthParameters(const FConfigFile& InConfigFile)
{
	const FConfigSection* Section = InConfigFile.FindSection(TEXT("SubmitTool.OAuthToken"));
	FOAuthTokenParams Output;

	if(Section != nullptr)
	{
		FStringOutputDevice Errors;
		FOAuthTokenParams::StaticStruct()->ImportText(*SectionToText(*Section), &Output, nullptr, 0, &Errors, FOAuthTokenParams::StaticStruct()->GetName());

		if(!Errors.IsEmpty())
		{
			UE_LOGF(LogSubmitTool, Error, "Error loading parameter file %ls", *Errors);
			FModelInterface::SetErrorState();
		}
		else
		{
			Output.OAuthFile = FConfiguration::Substitute(Output.OAuthFile);
			Output.OAuthTokenTool = FConfiguration::Substitute(Output.OAuthTokenTool);
			Output.OAuthArgs = FConfiguration::Substitute(Output.OAuthArgs); 
		}
	}

	return Output;
}


FIncompatibleFilesParams FSubmitToolParametersBuilder::BuildIncompatibleFilesParameters(const FConfigFile& InConfigFile)
{
	const FConfigSection* Section = InConfigFile.FindSection(TEXT("SubmitTool.IncompatibleFiles"));
	FIncompatibleFilesParams Output;

	if(Section != nullptr)
	{
		FStringOutputDevice Errors;
		FIncompatibleFilesParams::StaticStruct()->ImportText(*SectionToText(*Section), &Output, nullptr, 0, &Errors, FIncompatibleFilesParams::StaticStruct()->GetName());

		if(!Errors.IsEmpty())
		{
			UE_LOGF(LogSubmitTool, Error, "Error loading parameter file %ls", *Errors);
			FModelInterface::SetErrorState();
		}
	}

	return Output;
}

FHordeParameters FSubmitToolParametersBuilder::BuildHordeParameters(const FConfigFile& InConfigFile)
{
	const FConfigSection* Section = InConfigFile.FindSection(TEXT("SubmitTool.Horde"));
	FHordeParameters Output;

	if(Section != nullptr)
	{
		FStringOutputDevice Errors;
		FHordeParameters::StaticStruct()->ImportText(*SectionToText(*Section), &Output, nullptr, 0, &Errors, FHordeParameters::StaticStruct()->GetName());

		if (!Errors.IsEmpty())
		{
			UE_LOGF(LogSubmitTool, Error, "Error loading parameter file %ls", *Errors);
			FModelInterface::SetErrorState();
		}
	}
	return Output;
}

FAutoUpdateParameters FSubmitToolParametersBuilder::BuildAutoUpdateParameters(const FConfigFile& InConfigFile)
{
	const FConfigSection* Section = InConfigFile.FindSection(TEXT("SubmitTool.AutoUpdate"));
	FAutoUpdateParameters Output;

	if(Section != nullptr)
	{
		FStringOutputDevice Errors;
		FAutoUpdateParameters::StaticStruct()->ImportText(*SectionToText(*Section), &Output, nullptr, 0, &Errors, FAutoUpdateParameters::StaticStruct()->GetName());

		if (!Errors.IsEmpty())
		{
			UE_LOGF(LogSubmitTool, Error, "Error loading parameter file %ls", *Errors);
			FModelInterface::SetErrorState();
		}
		else
		{
			Output.DeployIdFilePath = FConfiguration::Substitute(Output.DeployIdFilePath);
			Output.LocalDownloadZip = FConfiguration::Substitute(Output.LocalDownloadZip);
			Output.LocalVersionFile = FConfiguration::Substitute(Output.LocalVersionFile);
			Output.AutoUpdateScript = FConfiguration::Substitute(Output.AutoUpdateScript);
			Output.LocalAutoUpdateScript = FConfiguration::Substitute(Output.LocalAutoUpdateScript);
		}

	}

	return Output;
}

FString FSubmitToolParametersBuilder::SectionToText(const FConfigSection& InSection) const
{
	TArray<FString> lines;
	for(const TPair<FName, FConfigValue>& Item : InSection.Array())
	{
		FString Value = Item.Value.GetValue();

		// If it's an array/map/struct, we only need to quote the key, otherwise quote key and value
		if((Value.IsNumeric() && !Value.Equals(TEXT("-"))) || (Value.StartsWith(TEXT("(")) && Value.EndsWith(TEXT(")")) && !Item.Key.ToString().Contains(TEXT("Regex"), ESearchCase::IgnoreCase)))
		{
			lines.Add(FString::Printf(TEXT("\"%s\"=%s"), *Item.Key.ToString(), *Item.Value.GetValue()));
		}
		else
		{
			lines.Add(FString::Printf(TEXT("\"%s\"=\"%s\""), *Item.Key.ToString(), *Item.Value.GetValue()));
		}
	}

	FString FinalText = TEXT("(") + FString::Join(lines, TEXT(",")) + TEXT(")");
	return FinalText;
}

