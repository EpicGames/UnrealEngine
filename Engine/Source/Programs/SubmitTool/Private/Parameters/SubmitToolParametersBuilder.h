// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#include "Misc/ConfigCacheIni.h"
#include "Parameters/SubmitToolParameters.h"

class FSubmitToolParametersBuilder
{
public:
	FSubmitToolParametersBuilder();
	FSubmitToolParameters LoadConfigFromFiles();

	FGeneralParameters ReadGeneralParametersFromLocalConfig();

private:
	TArray<FConfigLayer> ConfigHierarchy;
	TArray<FString> ProjectNames;
private:
	FGeneralParameters BuildGeneralParameters(const FConfigFile& InConfigFile);
	FJiraParameters BuildJiraParameters(const FConfigFile& InConfigFile);
	FTelemetryParameters GetTelemetryParameters(const FConfigFile& InConfigFile);
	FIntegrationParameters BuildIntegrationParameters(const FConfigFile& InConfigFile);
	TArray<FTagDefinition> BuildAvailableTags(const FConfigFile& InConfigFile);
	TMap<FString, FString> BuildValidators(const FConfigFile& InConfigFile);
	TMap<FString, FString> BuildPresubmitOperations(const FConfigFile& InConfigFile);
	FCopyLogParameters BuildCopyLogParameters(const FConfigFile& InConfigFile);
	FP4LockdownParameters BuildP4LockdownParameters(const FConfigFile& InConfigFile);
	FOAuthTokenParams BuildOAuthParameters(const FConfigFile& InConfigFile);
	FIncompatibleFilesParams BuildIncompatibleFilesParameters(const FConfigFile& InConfigFile);
	FHordeParameters BuildHordeParameters(const FConfigFile& InConfigFile);
	FAutoUpdateParameters BuildAutoUpdateParameters(const FConfigFile& InConfigFile);

	
	FString SectionToText(const FConfigSection& InSection) const;
};
