// Copyright Epic Games, Inc. All Rights Reserved.

#include "SubmitToolApp.h"
#include "Misc/CommandLine.h"
#include "UnixCommonStartup.h"

int RunSubmitToolWrapper(const TCHAR* Commandline)
{
	FCommandLine::Set(Commandline);
    return RunSubmitTool(Commandline);
}

int main(int argc, char *argv[])
{
	return CommonUnixMain(argc, argv, &RunSubmitToolWrapper);
}
