// Copyright Epic Games, Inc. All Rights Reserved.

#include "SubmitToolApp.h"
#include "Mac/MacProgramDelegate.h"
#include "Misc/CommandLine.h"
#include "LaunchEngineLoop.h"

int RunSubmitToolWrapper(const TCHAR* Commandline)
{
	FCommandLine::Set(Commandline);
	return RunSubmitTool(Commandline);
}

int main(int argc, char *argv[])
{
	[MacProgramDelegate mainWithArgc:argc argv:argv programMain:RunSubmitToolWrapper programExit:FEngineLoop::AppExit];
}
