// Copyright Epic Games, Inc. All Rights Reserved.

#include "SubmitToolApp.h"
#include "Misc/CommandLine.h"
#include "CoreMinimal.h"
#include "Windows/WindowsHWrapper.h"
#include "HAL/ExceptionHandling.h"
#include "Misc/OutputDeviceError.h"
/**
 * WinMain, called when the application is started
 */

int WINAPI WinMain( _In_ HINSTANCE hInInstance, _In_opt_ HINSTANCE hPrevInstance, _In_ LPSTR, _In_ int nCmdShow )
{
	int ErrorLevel = 0;
#if !PLATFORM_SEH_EXCEPTIONS_DISABLED
	__try
#endif
	{
		hInstance = hInInstance;
		GIsGuarded = 1;

		FString CmdLine = FCommandLine::BuildFromArgV(nullptr, __argc, __argv, "");
		FCommandLine::Set(*CmdLine);

		RunSubmitTool(*CmdLine);
		GIsGuarded = 0;
	}
#if !PLATFORM_SEH_EXCEPTIONS_DISABLED
	__except(ReportCrash(GetExceptionInformation()))
	{
		ErrorLevel = 1;
		GError->HandleError();
		FPlatformMisc::RequestExit(true);
	}
#endif
	return ErrorLevel;
}