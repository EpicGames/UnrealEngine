// Copyright Epic Games, Inc. All Rights Reserved.

#include "SubmitToolApp.h"
#include "CoreMinimal.h"
#include "SubmitToolCoreUtils.h"
#include "Models/SubmitToolUserPrefs.h"
#include "Logic/Services/SourceControl/SubmitToolPerforce.h"
#include "Logging/SubmitToolOutputLogHistory.h"

#include "GenericPlatform/GenericPlatformMisc.h"
#include "RequiredProgramMainCPPInclude.h"
#include "StandaloneRenderer.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Docking/TabManager.h"
#include "Stats/StatsSystem.h"

#include "CommandLine/CmdLineParameters.h"

#include "View/SubmitToolWindow.h"
#include "View/SubmitToolStyle.h"
#include "View/SubmitToolCommandHandler.h"
#include "View/SubmitToolMenu.h"

#include "Models/ModelInterface.h"

#include "Parameters/SubmitToolParameters.h"
#include "Parameters/SubmitToolParametersBuilder.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"

#include "Telemetry/TelemetryService.h"
#include "Version/AppVersion.h"

#include "Configuration/Configuration.h"

#if !UE_BUILD_SHIPPING
#include "Tests/RPCLibrary.h"

#include "Logging/LogScopedVerbosityOverride.h"
#endif // !UE_BUILD_SHIPPING

IMPLEMENT_APPLICATION(SubmitTool, "SubmitTool");

#define LOCTEXT_NAMESPACE "SubmitTool"

class FSlateUI
{
public:
	FSlateUI(FModelInterface* InModelInterface)
		: ModelInterface(InModelInterface)
		, Window(ModelInterface)
		, MainDockTab(SNew(SDockTab))
		, CommandList(MakeShared<FUICommandList>())
	{ }
	void Build(FSubmitToolUserPrefs* UserPrefs, TSharedPtr<FSubmitToolOutputLogHistory> InLogHistory);

private:

	FModelInterface* ModelInterface;
	FSubmitToolCommandHandler CommandHandler;
	SubmitToolWindow Window;
	TSharedRef<SDockTab> MainDockTab;
	TSharedRef<FUICommandList> CommandList;
	TSharedPtr<FTabManager> TabManager;
	TSharedPtr<SWidget> MenuWidget;
	TSharedPtr<SDockTab> Tab;
};

TUniquePtr<FSlateUI> MakeSlateUI(FModelInterface* ModelInterface)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MakeSlateUI);
	return MakeUnique<FSlateUI>(ModelInterface);
}

void FSlateUI::Build(FSubmitToolUserPrefs* UserPrefs, TSharedPtr<FSubmitToolOutputLogHistory> InLogHistory)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(BuildSlateUI);

	// Build the slate UI for the program window
	TabManager = FGlobalTabmanager::Get()->NewTabManager(MainDockTab);
	// set the application name
	FGlobalTabmanager::Get()->SetApplicationTitle(LOCTEXT("AppTitle", "SubmitTool"));
	TabManager->SetCanDoDragOperation(false);

	// set the main menu commands and interface
	CommandHandler.AddToCommandList(ModelInterface, CommandList);

	FMenuBarBuilder MenuBarBuilder = FMenuBarBuilder(CommandList);
	MenuBarBuilder.AddPullDownMenu(LOCTEXT("MainMenu", "Main Menu"), LOCTEXT("OpensMainMenu", "Opens Main Menu"), FNewMenuDelegate::CreateStatic(&FSubmitToolMenu::FillMainMenuEntries));
#if !UE_BUILD_SHIPPING
	MenuBarBuilder.AddPullDownMenu(LOCTEXT("Debug Tools", "Debug"), LOCTEXT("OpensDebugMenu", "Opens Debug Menu"), FNewMenuDelegate::CreateStatic(&FSubmitToolMenu::FillDebugMenuEntries));
#endif

	MenuWidget = MenuBarBuilder.MakeWidget();
	TabManager->SetAllowWindowMenuBar(true);
	TabManager->SetMenuMultiBox(MenuBarBuilder.GetMultiBox(), MenuWidget);

	FName TabName = FName("Submit Tool");

	TabManager->RegisterTabSpawner(TabName, FOnSpawnTab::CreateLambda([&Window = Window, InLogHistory](const FSpawnTabArgs& SpawnArgs) mutable { return Window.BuildMainTab(SpawnArgs.GetOwnerWindow(), MoveTemp(InLogHistory)); }));
	TabManager->RegisterDefaultTabWindowSize(TabName, FVector2D(1024, 600));

	Tab = TabManager->TryInvokeTab(TabName);
	FWindowSizeLimits WindowLimits;
	WindowLimits.SetMinWidth(600);
	WindowLimits.SetMinHeight(400);
	Tab->GetParentWindow()->SetSizeLimits(WindowLimits);

	if(!UserPrefs->WindowPosition.IsZero())
	{
		FDisplayMetrics DisplayMetrics;
		FSlateApplicationBase::Get().GetCachedDisplayMetrics(DisplayMetrics);
		const FPlatformRect& VirtualDisplayRect = DisplayMetrics.VirtualDisplayRect;

		if(UserPrefs->WindowPosition.X >= VirtualDisplayRect.Left &&
			UserPrefs->WindowPosition.X < VirtualDisplayRect.Right &&
			UserPrefs->WindowPosition.Y >= VirtualDisplayRect.Top &&
			UserPrefs->WindowPosition.Y < VirtualDisplayRect.Bottom)
		{
			Tab->GetParentWindow()->MoveWindowTo(UserPrefs->WindowPosition);
		}
	}

	if(!UserPrefs->WindowSize.IsZero())
	{
		Tab->GetParentWindow()->Resize(UserPrefs->WindowSize);
	}
	else
	{
		Tab->GetParentWindow()->Resize(FDeprecateSlateVector2D(1024, 768));
	}

	if (UserPrefs->bWindowMaximized)
	{
		Tab->GetParentWindow()->Maximize();
	}
}

static void EngineLoop(FModelInterface* ModelInterface)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(EngineLoop);

	double DeltaTime = 0.0;
	double LastTime = FPlatformTime::Seconds();
	const float IdealFrameTime = 1.0f / 60;
	const float BackgroundFrameTime = 1.0f / 4;

	// Loop the engine
	while (!IsEngineExitRequested())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(Tick);

		BeginExitIfRequested();

		FTaskGraphInterface::Get().ProcessThreadUntilIdle(ENamedThreads::GameThread);
		FTSTicker::GetCoreTicker().Tick(DeltaTime);
		FSlateApplication::Get().PumpMessages();
		FSlateApplication::Get().Tick();

		if (IsEngineExitRequested())
		{
			FTelemetryService::Get()->Exit();
			// Dispose here so SCCProvider has time to clean up
			ModelInterface->Dispose();
		}

		const float FrameTime = !FPlatformApplicationMisc::IsThisApplicationForeground() && (FPlatformTime::Seconds() - FSlateApplication::Get().GetLastUserInteractionTime()) > 5 ? BackgroundFrameTime : IdealFrameTime;
		FPlatformProcess::Sleep(FMath::Max<float>(0.f, FrameTime - static_cast<float>(FPlatformTime::Seconds() - LastTime)));

		DeltaTime = FPlatformTime::Seconds() - LastTime;
		LastTime = FPlatformTime::Seconds();

		UE::Stats::FStats::AdvanceFrame(false);
		FCoreDelegates::OnEndFrame.Broadcast();

		GFrameCounter++;
	}
}

int RunSubmitTool(const TCHAR* CommandLine)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(RunSubmitTool);
	FTaskTagScope TaskTagScope(ETaskTag::EGameThread);

	FPlatformOutputDevices::SetupOutputDevices();
	TSharedPtr<FSubmitToolOutputLogHistory> LogHistory = MakeShared<FSubmitToolOutputLogHistory>();

	// need to make sure the cwd is correct before doing anything else
	FPlatformProcess::SetCurrentWorkingDirectoryToBaseDir();

	// setup command line
	const FCmdLineParameters& CmdLineParameters = FCmdLineParameters::Get();
	CommandLine = CmdLineParameters.InitializeParameters();

	// start up the main loop
	GEngineLoop.PreInit(CommandLine);

#if !UE_BUILD_SHIPPING
	if (FCmdLineParameters::Get().Contains(FSubmitToolCmdLine::TestMode))
	{
		// silence most logging in test mode except fatal errors
		LogSubmitToolP4Debug.SetVerbosity(ELogVerbosity::Fatal);
		LogSubmitToolDebug.SetVerbosity(ELogVerbosity::Fatal);
		LogSubmitTool.SetVerbosity(ELogVerbosity::Fatal);
	}
#endif // !UE_BUILD_SHIPPING

	// ensure that the backlog is enabled
	if(GLog)
	{
		GLog->EnableBacklog(true);
	}

	UE_LOGF(LogSubmitToolDebug, Log, "%ls", CommandLine);

	if(!CmdLineParameters.ValidateParameters())
	{
		UE_LOGF(LogSubmitTool, Error, "Command line is not valid");
		CmdLineParameters.LogParameters();
		FModelInterface::SetErrorState();
	}

	// Make sure all UObject classes are registered and default properties have been initialized
	ProcessNewlyLoadedUObjects();

	// Tell the module manager it may now process newly-loaded UObjects when new C++ modules are loaded
	FModuleManager::Get().StartProcessingNewlyLoadedObjects();

	// Load optional restricted module if present — silently no-ops if not found
	FModuleManager::Get().LoadModule(TEXT("SubmitToolRestricted"));

	// initialize the configuration system
	FConfiguration::Init();

	// App Scope
	{
		UE_LOGF(LogSubmitTool, Log, "Version %ls", *FAppVersion::GetVersion());
		
		// Sync up the config as early as possible
		UE::Tasks::TTask<TUniquePtr<FModelInterface>> InitializeTask = SyncConfigAndInitializeModelInterface();
		TUniquePtr<FSubmitToolUserPrefs> UserPrefs = FSubmitToolUserPrefs::Initialize(GetUserPrefsPath());

		// allow running multiple parallel instances of UAT
		FPlatformMisc::SetEnvironmentVar(TEXT("uebp_UATMutexNoWait"), TEXT("1"));
		UE_LOGF(LogSubmitToolDebug, Log, "Set environment variable uebp_UATMutexNoWait=1 to allow running multiple parallel instances of UAT.");

		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FSubmitToolApp::InitializeUI);
			// crank up a normal Slate application using the platform's standalone renderer while the sync is in progress
			FSlateApplication::InitializeAsStandaloneApplication(GetStandardStandaloneRenderer());
			FAppStyle::SetAppStyleSet(FSubmitToolStyle::Get());
			FSlateApplication::InitHighDPI(true);
		}

		// Wait for the sync since everything else needs the config.
		TUniquePtr<FModelInterface> ModelInterface = MoveTemp(InitializeTask.GetResult());

		// Build the slate ui
		TUniquePtr<FSlateUI> SlateUI = MakeSlateUI(ModelInterface.Get());
		SlateUI->Build(UserPrefs.Get(), MoveTemp(LogHistory));

		// Test mode, attach to debugger, only non-shipping
#if !UE_BUILD_SHIPPING
		if (FCmdLineParameters::Get().Contains(FSubmitToolCmdLine::WaitForDebugger))
		{
			int MaxWaitSeconds = 300;
			while (!FPlatformMisc::IsDebuggerPresent() && MaxWaitSeconds > 0)
			{
				MaxWaitSeconds--;
				FPlatformProcess::Sleep(1.f);
			}
			UE_DEBUG_BREAK();
		}

		if (FCmdLineParameters::Get().Contains(FSubmitToolCmdLine::TestMode))
		{
			USubmitToolRPCLibrary* RPCLibrary = NewObject<USubmitToolRPCLibrary>();
			RPCLibrary->SetModelInterface(ModelInterface.Get());
			RPCLibrary->RegisterLibraryRPCs();

			LogSubmitToolP4Debug.SetVerbosity(ELogVerbosity::All);
			LogSubmitToolDebug.SetVerbosity(ELogVerbosity::All);
			LogSubmitTool.SetVerbosity(ELogVerbosity::All);
		}
#endif

		// record that the application has started
		// do this after initializing UI to give ISTSourceControlService time to populate GetCurrentStreamName

		const FString StreamName = ModelInterface->GetServiceProvider()->GetService<ISTSourceControlService>()->GetCurrentStreamName();
		FTelemetryService::Init(ModelInterface->GetParameters().Telemetry, StreamName);
		FTelemetryService::Get()->Start(StreamName);

		// Tick the application until shutdown
		EngineLoop(ModelInterface.Get());

		// ensure all the telemetry events are flushed before unloading modules
		TRACE_CPUPROFILER_EVENT_SCOPE(FlushTelemetry);
		FTelemetryService::BlockFlush(5.f);
		FTelemetryService::Shutdown();
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(Shutdown);
	FSlateApplication::Shutdown();

	FEngineLoop::AppPreExit();
	FModuleManager::Get().UnloadModulesAtShutdown();
	FEngineLoop::AppExit();

	return 0;
}

UE::Tasks::TTask<TUniquePtr<FModelInterface>> SyncConfigAndInitializeModelInterface()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FSubmitToolApp::PreloadModules);

	FModuleManager::LoadModuleChecked<FHttpModule>("HTTP");

	return UE::Tasks::Launch(UE_SOURCE_LOCATION, [] {

		TSharedPtr<ISTSourceControlService> SourceControlService = MakeShared<FSubmitToolPerforce>();
		FSubmitToolParametersBuilder ParametersBuilder;
		FGeneralParameters GeneralParameters = ParametersBuilder.ReadGeneralParametersFromLocalConfig();
		if(GeneralParameters.SyncOnStartupPaths.Num() > 0)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FSubmitToolApp::SyncOnStartupFiles);
			const FString Root = FConfiguration::Substitute(TEXT("$(root)"));

			TArray<FString> Files;
			Files.Reserve(GeneralParameters.SyncOnStartupPaths.Num());
			for (const FString& Path : GeneralParameters.SyncOnStartupPaths)
			{
				Files.Add(FConfiguration::SubstituteAndNormalizeFilename(Path));
			}

			SourceControlService->RunSyncCommand(TEXT("sync"), MoveTemp(Files), true);
		}

		TRACE_CPUPROFILER_EVENT_SCOPE(FSubmitToolApp::InitializeModelInterface);
		// Create a new instance of model Interface so that UI can communicate
		return MakeUnique<FModelInterface>(ParametersBuilder.LoadConfigFromFiles(), SourceControlService);
	});
}

FString GetUserPrefsPath()
{
	return FPaths::Combine(FSubmitToolCoreUtils::GetLocalAppDataPath(), "SubmitTool", "SubmitToolPrefs.ini");
}
//Test comment for submit tool testing (remove if needed) x6
#undef LOCTEXT_NAMESPACE
