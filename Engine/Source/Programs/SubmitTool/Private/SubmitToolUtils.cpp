// Copyright Epic Games, Inc. All Rights Reserved.

#include "SubmitToolUtils.h"

#include "Misc/Paths.h"
#include "Logging/SubmitToolLog.h"
#include "Widgets/SWindow.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Text/STextBlock.h"
#include "Framework/Application/SlateApplication.h"

#if PLATFORM_WINDOWS
#include "Windows/AllowWindowsPlatformTypes.h"
#include <ShlObj.h>
#include "Windows/HideWindowsPlatformTypes.h"
#endif

#define LOCTEXT_NAMESPACE "FSubmitToolUtils"

TMap<FString, TMap<bool, TSet<FString>>> FSubmitToolUtils::HierarchyWildcardsCache;

void FSubmitToolUtils::EnsureWindowIsInView(TSharedRef<SWindow> InWindow, bool bSingleWindow)
{
	FDeprecateSlateVector2D WinPos = InWindow->GetPositionInScreen();

	FDisplayMetrics DisplayMetrics;
	FSlateApplicationBase::Get().GetCachedDisplayMetrics(DisplayMetrics);
	const FPlatformRect& VirtualDisplayRect = bSingleWindow ? DisplayMetrics.PrimaryDisplayWorkAreaRect : DisplayMetrics.VirtualDisplayRect;

	if(WinPos.X < VirtualDisplayRect.Left ||
		WinPos.X >= VirtualDisplayRect.Right ||
		WinPos.Y < VirtualDisplayRect.Top ||
		WinPos.Y >= VirtualDisplayRect.Bottom)
	{
		FDeprecateSlateVector2D ClampedPosition;
		ClampedPosition.X = FMath::Clamp(WinPos.X, VirtualDisplayRect.Left, VirtualDisplayRect.Right - InWindow->GetSizeInScreen().X);
		ClampedPosition.Y = FMath::Clamp(WinPos.Y, VirtualDisplayRect.Top, VirtualDisplayRect.Bottom - InWindow->GetSizeInScreen().Y);
		InWindow->MoveWindowTo(ClampedPosition);
	}	
}

TSharedRef<SHorizontalBox> FSubmitToolUtils::BuildUserPrefCheckboxUI(bool& InUserPrefOption, const FText&& InText)
{
	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(SCheckBox)
				.IsChecked_Lambda([&InUserPrefOption]() { return InUserPrefOption ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
				.OnCheckStateChanged_Lambda([&InUserPrefOption](ECheckBoxState InNewState)
					{
						InUserPrefOption = InNewState == ECheckBoxState::Checked;
					})
		]
	+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "InvisibleButton")
				.IsFocusable(false)
				.OnClicked_Lambda([&InUserPrefOption, MoveTemp(InText)]() { InUserPrefOption = !InUserPrefOption; return FReply::Handled(); })
				[
					SNew(STextBlock)
						.Justification(ETextJustify::Left)
						.MinDesiredWidth(60)
						.Text(InText)
				]
		];
}

#undef LOCTEXT_NAMESPACE
