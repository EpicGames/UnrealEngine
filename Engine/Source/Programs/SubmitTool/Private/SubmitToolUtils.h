// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Widgets/SWindow.h"

class FString;

class FSubmitToolUtils
{
public:
	static void EnsureWindowIsInView(TSharedRef<SWindow> InWindow, bool bSingleWindow);
	static TSharedRef<SHorizontalBox> BuildUserPrefCheckboxUI(bool& InUserPrefOption, const FText&& InText);
private:
	static TMap<FString, TMap<bool, TSet<FString>>> HierarchyWildcardsCache;
};
