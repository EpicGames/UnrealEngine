// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

struct FAnalyticsEventAttribute;

class ITelemetry
{
public:
	virtual void Start(const FString& InCurrentStream) const = 0;
	virtual void BlockFlush(float InTimeout) const = 0;
	virtual void SubmitSucceeded(TArray<FAnalyticsEventAttribute>&& InAttribs) const = 0;
	virtual void CustomEvent(const FString& InEventId, const TArray<FAnalyticsEventAttribute>& InAttribs) const = 0;
	virtual void Exit() = 0;


protected:
	bool bExiting = false;
};