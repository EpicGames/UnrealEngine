// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ITelemetry.h"

class FNullTelemetry : public ITelemetry
{
public:
	virtual void Start(const FString& InCurrentStream) const override {}
	virtual void BlockFlush(float InTimeout) const {}
	virtual void SubmitSucceeded(TArray<FAnalyticsEventAttribute>&& InAttribs) const override {}
	virtual void CustomEvent(const FString& InEventId, const TArray<FAnalyticsEventAttribute>& InAttribs) const override {}
	virtual void Exit() {}

};