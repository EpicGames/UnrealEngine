// Copyright Epic Games, Inc. All Rights Reserved.

#include "StandAloneTelemetry.h"
#include "Misc/Build.h"
#include "Logging/SubmitToolLog.h"
#include "Version/AppVersion.h"
#include "AnalyticsEventAttribute.h"

FStandAloneTelemetry::FStandAloneTelemetry(const FString& InUrl, const FGuid& InSessionID)
{
	FAnalyticsET::Config Config;
	
#if defined(SUBMIT_TOOL_PRERELEASE)
	Config.APIKeyET = TEXT("SubmitToolStandalone.PreRelease");
#elif defined(SUBMIT_TOOL_RELEASE)
	Config.APIKeyET = TEXT("SubmitToolStandalone.Live");
#else
	Config.APIKeyET = TEXT("SubmitToolStandalone.Source");
#endif

	Config.APIServerET = InUrl;

	// This will become the AppVersion URL parameter. It can be whatever makes sense for your app.
	Config.AppVersionET = FAppVersion::GetVersion();

	// This will become the Environment URL parameter. It can be arbitrary.
	Config.AppEnvironment = TEXT("SubmitTool.Standalone");

	// There are other things to configure, but the default are usually fine.
	
	Provider = FAnalyticsET::Get().CreateAnalyticsProvider(Config);
	checkf(Provider.IsValid(), TEXT("Failure constructing analytics provider!"));

	Provider->SetUserID(InSessionID.ToString());
}

FStandAloneTelemetry::~FStandAloneTelemetry()
{

}

void FStandAloneTelemetry::Start(const FString& InCurrentStream) const
{
	if (Provider == nullptr)
	{
		return;
	}

	Provider->RecordEvent(
		TEXT("SubmitTool.StandAlone.Start"),
		MakeAnalyticsEventAttributeArray(
			TEXT("Version"), FAppVersion::GetVersion(),
			TEXT("Stream"), InCurrentStream
		)
	);
}

void FStandAloneTelemetry::BlockFlush(float InTimeout) const
{
	if (Provider == nullptr)
	{
		return;
	}

	Provider->BlockUntilFlushed(InTimeout);
}
void FStandAloneTelemetry::CustomEvent(const FString& InEventId, const TArray<FAnalyticsEventAttribute>& InAttribs) const
{
	if(Provider == nullptr || bExiting)
	{
		return;
	}

	Provider->RecordEvent(InEventId, InAttribs);
}

void FStandAloneTelemetry::Exit()
{
	bExiting = true;

	if (Provider == nullptr)
	{
		return;
	}

	Provider->RecordEvent(
		TEXT("SubmitTool.StandAlone.Exit"),
		MakeAnalyticsEventAttributeArray(
			TEXT("Version"), FAppVersion::GetVersion()
		)
	);
}

void FStandAloneTelemetry::SubmitSucceeded(TArray<FAnalyticsEventAttribute>&& InAttribs) const
{
	if (Provider == nullptr || bExiting)
	{
		return;
	}

	Provider->RecordEvent(
		TEXT("SubmitTool.StandAlone.Submit.Succeeded"),
		AppendAnalyticsEventAttributeArray(
			InAttribs,
			TEXT("Version"), FAppVersion::GetVersion()
		)
	);
}