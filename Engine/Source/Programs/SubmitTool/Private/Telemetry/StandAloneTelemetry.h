// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AnalyticsProviderConfigurationDelegate.h"
#include "AnalyticsET.h"
#include "IAnalyticsProviderET.h"
#include "ITelemetry.h"

class FStandAloneTelemetry : public ITelemetry
{
public:
	FStandAloneTelemetry(const FString& InUrl, const FGuid& InSessionID);
	virtual ~FStandAloneTelemetry();
	virtual void Start(const FString& InCurrentStream) const override;
	virtual void BlockFlush(float InTimeout) const override;
	virtual void SubmitSucceeded(TArray<FAnalyticsEventAttribute>&& InAttribs) const override;
	virtual void CustomEvent(const FString& InEventId, const TArray<FAnalyticsEventAttribute>& InAttribs) const override;
	virtual void Exit() override;


private:
	TSharedPtr<IAnalyticsProviderET> Provider;
};