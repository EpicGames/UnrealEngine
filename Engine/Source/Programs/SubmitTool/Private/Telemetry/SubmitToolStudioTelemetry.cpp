// Copyright Epic Games, Inc. All Rights Reserved.

#include "SubmitToolStudioTelemetry.h"
#include "Logging/SubmitToolLog.h"
#include "Version/AppVersion.h"
#include "AnalyticsEventAttribute.h"
#include "StudioTelemetry.h"

const FName SpanId = TEXT("SubmitTool.Session");

FSubmitToolStudioTelemetry::FSubmitToolStudioTelemetry()
{
	FStudioTelemetry::Get().StartSession();	
	FStudioTelemetry::Get().StartSpan(SpanId);
}

FSubmitToolStudioTelemetry::~FSubmitToolStudioTelemetry()
{
	FStudioTelemetry::Get().EndSpan(SpanId);
	FStudioTelemetry::Get().EndSession();
}

void FSubmitToolStudioTelemetry::Start(const FString& InCurrentStream) const
{
	FStudioTelemetry::Get().RecordEvent(
		TEXT("SubmitTool.StandAlone.Start"),
		MakeAnalyticsEventAttributeArray(
			TEXT("Version"), FAppVersion::GetVersion(),
			TEXT("Stream"), InCurrentStream
		)
	);
}

void FSubmitToolStudioTelemetry::BlockFlush(float InTimeout) const
{
	FStudioTelemetry::Get().FlushEvents();
}
void FSubmitToolStudioTelemetry::CustomEvent(const FString& InEventId, const TArray<FAnalyticsEventAttribute>& InAttribs) const
{
	if (!bExiting)
	{
		FStudioTelemetry::Get().RecordEvent(InEventId, InAttribs);
	}
}

void FSubmitToolStudioTelemetry::Exit()
{
	bExiting = true;
	FStudioTelemetry::Get().RecordEvent(
		TEXT("SubmitTool.StandAlone.Exit"),
		MakeAnalyticsEventAttributeArray(
			TEXT("Version"), FAppVersion::GetVersion()
		)
	);
}

void FSubmitToolStudioTelemetry::SubmitSucceeded(TArray<FAnalyticsEventAttribute>&& InAttribs) const
{
	if (!bExiting)
	{ 
		FStudioTelemetry::Get().RecordEvent(
			TEXT("SubmitTool.StandAlone.Submit.Succeeded"),
			AppendAnalyticsEventAttributeArray(
				InAttribs,
				TEXT("Version"), FAppVersion::GetVersion()
			)
		);
	}
}