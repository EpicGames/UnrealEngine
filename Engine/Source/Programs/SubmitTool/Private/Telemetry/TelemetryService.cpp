// Copyright Epic Games, Inc. All Rights Reserved.

#include "TelemetryService.h"

#include "Logging/SubmitToolLog.h"
#include "Parameters/SubmitToolParameters.h"
#include "NullTelemetry.h"
#include "StandAloneTelemetry.h"
#include "SubmitToolStudioTelemetry.h"

FCriticalSection FTelemetryService::InstanceCriticalSection;
TSharedPtr<ITelemetry> FTelemetryService::TelemetryInstance = nullptr;

const TSharedPtr<ITelemetry>& FTelemetryService::Get()
{
	return TelemetryInstance;
}

void FTelemetryService::Init(const FTelemetryParameters& InTelemetryParameters, const FString& InStreamName)
{
	if (!InTelemetryParameters.RegexStreamAllowlist.IsEmpty())
	{
		FRegexPattern Pattern = FRegexPattern(InTelemetryParameters.RegexStreamAllowlist, ERegexPatternFlags::CaseInsensitive);
		FRegexMatcher regex = FRegexMatcher(Pattern, InStreamName);

		if (!regex.FindNext())
		{
			Set(MakeShared<FNullTelemetry>());
			return;
		}
	}

	if (InTelemetryParameters.Instance.Equals(TEXT("StandAlone"), ESearchCase::IgnoreCase))
	{
		if (InTelemetryParameters.Url.IsEmpty())
		{
			Set(MakeShared<FNullTelemetry>());
		}
		else
		{
			Set(MakeShared<FStandAloneTelemetry>(InTelemetryParameters.Url, FGuid()));
		}
	}
	else if (InTelemetryParameters.Instance.Equals(TEXT("StudioTelemetry"), ESearchCase::IgnoreCase))
	{
		Set(MakeShared<FSubmitToolStudioTelemetry>());
	}
	else
	{
		Set(MakeShared<FNullTelemetry>());
	}
}

void FTelemetryService::Set(TSharedPtr<ITelemetry> InInstance)
{
	if (InInstance == nullptr)
	{
		UE_LOGF(LogSubmitTool, Warning, "Trying to set a null telemetry instance. Operation aborted.");
		return;
	}

	InstanceCriticalSection.Lock();
	
	TelemetryInstance = InInstance;

	InstanceCriticalSection.Unlock();
}

void FTelemetryService::BlockFlush(float InTimeout)
{
	TelemetryInstance->BlockFlush(InTimeout);
}

void FTelemetryService::Shutdown()
{
	TelemetryInstance = nullptr;
}
