// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "StandAloneTelemetry.h"
#include "ITelemetry.h"

struct FTelemetryParameters;

class FTelemetryService
{
public:
	static const TSharedPtr<ITelemetry> & Get();
	static void Init(const FTelemetryParameters& InTelemetryParameters, const FString& InStreamName);
	static void Shutdown();
	static void BlockFlush(float InTimeout);
private:
	static void Set(TSharedPtr<ITelemetry> InInstance);
	static TSharedPtr<ITelemetry> TelemetryInstance;
	static FCriticalSection InstanceCriticalSection;
};