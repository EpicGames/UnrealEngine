// Copyright Epic Games, Inc. All Rights Reserved.

#if !UE_BUILD_SHIPPING

#include "Tests/RPCLibrary.h"

DEFINE_LOG_CATEGORY_STATIC(LogSubmitToolRPCLibrary, Log, All);

USubmitToolRPCLibrary::USubmitToolRPCLibrary()
{
}

void USubmitToolRPCLibrary::RegisterLibraryRPCs()
{
#if WITH_RPC_REGISTRY
	StartListening();
	Initialize();
	RegisterRPC(
		TEXT("RunValidator"),
		FHttpPath(TEXT("/submittool/runvalidator")),
		EHttpServerRequestVerbs::VERB_POST,
		FHttpRequestHandler::CreateUObject(this, &USubmitToolRPCLibrary::HandleRunValidatorRoute));
	RegisterRPC(
		TEXT("GetValidatorState"),
		FHttpPath(TEXT("/submittool/getvalidatorstate")),
		EHttpServerRequestVerbs::VERB_POST, // GET doesn't work because of .NET not allowing payload for this verb
		FHttpRequestHandler::CreateUObject(this, &USubmitToolRPCLibrary::HandleGetValidatorStateRoute));
	RegisterRPC(
		TEXT("SetChangelistDescription"),
		FHttpPath(TEXT("/submittool/setchangelistdescription")),
		EHttpServerRequestVerbs::VERB_POST,
		FHttpRequestHandler::CreateUObject(this, &USubmitToolRPCLibrary::HandleSetChangelistDescriptionRoute));
	RegisterRPC(
		TEXT("CloseApplication"),
		FHttpPath(TEXT("/submittool/closeapplication")),
		EHttpServerRequestVerbs::VERB_POST,
		FHttpRequestHandler::CreateUObject(this, &USubmitToolRPCLibrary::HandleCloseApplicationRoute));
	RegistrationFinished = true;
#endif //WITH_RPC_REGISTRY
}

#if WITH_RPC_REGISTRY
bool USubmitToolRPCLibrary::HandleRunValidatorRoute(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete)
{
	check(FCmdLineParameters::Get().Contains(FSubmitToolCmdLine::TestMode));

	FString ResponseStr;
	TSharedRef<TJsonWriter<>> JsonWriter = TJsonWriterFactory<>::Create(&ResponseStr);
	JsonWriter->WriteObjectStart();

	TSharedPtr<FJsonObject> RootObject;
	FUTF8ToTCHAR RequestBodyByteBuffer(reinterpret_cast<const ANSICHAR*>(Request.Body.GetData()), Request.Body.Num());
	const FString RequestBody = FString::ConstructFromPtrSize(RequestBodyByteBuffer.Get(), RequestBodyByteBuffer.Length());
	TSharedRef<TJsonReader<>> JsonReader = TJsonReaderFactory<>::Create(RequestBody);

	bool status = DeserializeRequest(RootObject, JsonReader, JsonWriter);

	if (status)
	{
		FString ValidatorName;
		status = GetStringField(RootObject, TEXT("ValidatorName"), ValidatorName, JsonWriter);
		if (status)
		{
			UE_LOGF(LogSubmitToolRPCLibrary, Log, "Running validator: %ls", *ValidatorName);

			OnSingleValidatorFinishedHandle = ModelInterface->AddSingleValidatorFinishedCallback(FOnSingleTaskFinished::FDelegate::CreateLambda([this](const FValidatorBase& InValidator) { OnSingleValidatorFinished(InValidator); }));
			ModelInterface->ValidateSingle(FName(ValidatorName), true);
			ValidatorStates.Add(ValidatorName, EValidationStates::Not_Applicable);

			JsonWriter->WriteValue(TEXT("IsStarted"), true);
			JsonWriter->WriteObjectEnd();
			JsonWriter->Close();
			status = true;
		}
	}

	auto Response = CreateSimpleResponse(true, ResponseStr, false);
	OnComplete(MoveTemp(Response));
	return status;
}

void USubmitToolRPCLibrary::OnSingleValidatorFinished(const FValidatorBase& InValidator)
{
	UE_LOGF(LogSubmitToolRPCLibrary, Display, "Validator %ls finished.", *InValidator.GetValidatorNameId().ToString());
	ValidatorStates.Emplace(InValidator.GetValidatorNameId().ToString(), InValidator.GetState());
}

bool USubmitToolRPCLibrary::HandleGetValidatorStateRoute(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete)
{
	FString ResponseStr;
	TSharedRef<TJsonWriter<>> JsonWriter = TJsonWriterFactory<>::Create(&ResponseStr);
	JsonWriter->WriteObjectStart();

	TSharedPtr<FJsonObject> RootObject;
	FUTF8ToTCHAR RequestBodyByteBuffer(reinterpret_cast<const ANSICHAR*>(Request.Body.GetData()), Request.Body.Num());
	const FString RequestBody = FString::ConstructFromPtrSize(RequestBodyByteBuffer.Get(), RequestBodyByteBuffer.Length());
	TSharedRef<TJsonReader<>> JsonReader = TJsonReaderFactory<>::Create(RequestBody);

	bool status = DeserializeRequest(RootObject, JsonReader, JsonWriter);

	if (status)
	{
		FString ValidatorName;
		status = GetStringField(RootObject, TEXT("ValidatorName"), ValidatorName, JsonWriter);
		if (status)
		{
			UE_LOGF(LogSubmitToolRPCLibrary, Log, "Running validator: %ls", *ValidatorName);
			TArray<TWeakPtr<const FValidatorBase>> ValidatorsArray = ModelInterface->GetValidators();

			EValidationStates ValidatorState = EValidationStates::Not_Applicable;

			for (const TWeakPtr<const FValidatorBase>& Validator : ValidatorsArray)
			{
				if (Validator.Pin()->GetValidatorNameId().ToString() == ValidatorName)
				{
					ValidatorState = Validator.Pin()->GetState();
					break;
				}
			}

			if (ValidatorState == EValidationStates::Not_Applicable && ValidatorStates.Contains(ValidatorName))
			{
				ValidatorState = *ValidatorStates.Find(ValidatorName);
			}

			JsonWriter->WriteValue(TEXT("ValidatorState"), *(StaticEnum<EValidationStates>()->GetNameStringByValue((int64)ValidatorState)));
			JsonWriter->WriteObjectEnd();
			JsonWriter->Close();
			status = true;
		}
	}

	auto Response = CreateSimpleResponse(true, ResponseStr, false);
	OnComplete(MoveTemp(Response));

	return status;
}

bool USubmitToolRPCLibrary::HandleSetChangelistDescriptionRoute(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete)
{
	FString ResponseStr;
	TSharedRef<TJsonWriter<>> JsonWriter = TJsonWriterFactory<>::Create(&ResponseStr);
	JsonWriter->WriteObjectStart();

	TSharedPtr<FJsonObject> RootObject;
	FUTF8ToTCHAR RequestBodyByteBuffer(reinterpret_cast<const ANSICHAR*>(Request.Body.GetData()), Request.Body.Num());
	const FString RequestBody = FString::ConstructFromPtrSize(RequestBodyByteBuffer.Get(), RequestBodyByteBuffer.Length());
	TSharedRef<TJsonReader<>> JsonReader = TJsonReaderFactory<>::Create(RequestBody);

	bool status = DeserializeRequest(RootObject, JsonReader, JsonWriter);

	if (status)
	{
		FString Description;
		status = GetStringField(RootObject, TEXT("Description"), Description, JsonWriter);
		if (status)
		{
			UE_LOGF(LogSubmitToolRPCLibrary, Log, "Setting CL description to: %ls", *Description);
			ModelInterface->SetCLDescription(FText::FromString(Description));
		}
	}

	auto Response = CreateSimpleResponse(true, ResponseStr, false);
	OnComplete(MoveTemp(Response));

	return true;
}

bool USubmitToolRPCLibrary::HandleCloseApplicationRoute(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete)
{
	FString ResponseStr;

	auto Response = CreateSimpleResponse(true, ResponseStr, false);
	OnComplete(MoveTemp(Response));

	FPlatformMisc::RequestExitWithStatus(true, 0);

	return true;
}

#endif // WITH_RPC_REGISTRY
#endif // !UE_BUILD_SHIPPING