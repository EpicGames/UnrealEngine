// Copyright Epic Games, Inc. All Rights Reserved.

#include "AppVersion.h"

#include "Misc/Build.h"

#include "AppVersionDefines.h"

#if UE_BUILD_SHIPPING 
constexpr const TCHAR* BuildType = TEXT("Shipping");
#elif  UE_BUILD_DEVELOPMENT
constexpr const TCHAR* BuildType = TEXT("Development");
#elif  UE_BUILD_DEBUG
constexpr const TCHAR* BuildType = TEXT("Debug");
#elif  UE_BUILD_TEST
constexpr const TCHAR* BuildType = TEXT("Test");
#endif

unsigned constexpr const_hash(char const* input) {
	return *input ?
		static_cast<unsigned int>(*input) + 33 * const_hash(input + 1) :
		5381;
}

constexpr const unsigned GetBuildId()
{
	return const_hash(__DATE__) + const_hash(__TIMESTAMP__);
}

FString FAppVersion::Version;

FString FAppVersion::GetVersion()
{
	if (Version.IsEmpty())
	{
		TMap<FString, FStringFormatArg> FormatArgs =
		{
			{ TEXT("Application"), SUBMIT_TOOL_APPNAME },
			{ TEXT("ApplicatonVersion"), SUBMIT_TOOL_VERSION_STRING },
			{ TEXT("BuildType"), BuildType },
			{ TEXT("BuildId"), SUBMIT_TOOL_CHANGELIST_STRING }
		};

		Version = FString::Format(TEXT("{Application}-{BuildType}-{ApplicatonVersion}.{BuildId}"), FormatArgs);
	}

	return Version;
}