// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Macros for encoding strings
#define VERSION_TEXT(x) TEXT(x)
#define VERSION_STRINGIFY_2(x) VERSION_TEXT(#x)
#define VERSION_STRINGIFY(x) VERSION_STRINGIFY_2(x)

#define SUBMIT_TOOL_APPNAME TEXT("SubmitTool-Source")

#ifdef SUBMIT_TOOL_PRERELEASE
#undef SUBMIT_TOOL_APPNAME
#define SUBMIT_TOOL_APPNAME TEXT("SubmitTool-PreRelease")
#endif

#ifdef SUBMIT_TOOL_RELEASE
#undef SUBMIT_TOOL_APPNAME
#define SUBMIT_TOOL_APPNAME TEXT("SubmitTool")
#endif

#define SUBMIT_TOOL_VERSION_MAJOR 6
#define SUBMIT_TOOL_VERSION_MINOR 8
#define SUBMIT_TOOL_VERSION_PATCH 0

#ifndef SUBMIT_TOOL_CHANGELIST
#define SUBMIT_TOOL_CHANGELIST_STRING TEXT("NoCL")
#endif

#ifdef SUBMIT_TOOL_CHANGELIST
#define SUBMIT_TOOL_CHANGELIST_STRING TEXT("CL-") VERSION_STRINGIFY(SUBMIT_TOOL_CHANGELIST)
#endif

#define SUBMIT_TOOL_VERSION_STRING \
	VERSION_STRINGIFY(SUBMIT_TOOL_VERSION_MAJOR)\
	TEXT(".")\
	VERSION_STRINGIFY(SUBMIT_TOOL_VERSION_MINOR)\
	TEXT(".")\
	VERSION_STRINGIFY(SUBMIT_TOOL_VERSION_PATCH)

