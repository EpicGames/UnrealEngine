// Copyright Epic Games, Inc. All Rights Reserved.

#include "SubmitToolCommandHandler.h"
#include "SubmitToolCommandList.h"
#include "SubmitToolMenu.h"
#include "Models/ModelInterface.h"
#include "Widgets/SubmitToolHelpWidget.h"
#include "Logging/SubmitToolLog.h"
#include "Logging/LogMacros.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/UICommandList.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "FSubmitToolCommandHandler"

#if !UE_BUILD_SHIPPING
#include "ISlateReflectorModule.h"
#endif

void FSubmitToolCommandHandler::AddToCommandList(FModelInterface * InModelInterface, TSharedRef<FUICommandList> CommandList)
{
	FSubmitToolCommandList::Register();
	ModelInterface = InModelInterface;

#if !UE_BUILD_SHIPPING
	CommandList->MapAction(
		FSubmitToolCommandList::Get().ForceCrashCommandInfo,
		FExecuteAction::CreateRaw(this, &FSubmitToolCommandHandler::OnForceCrashCommandPressed),
		FCanExecuteAction::CreateLambda([]() {return true; }),
		FIsActionChecked::CreateLambda([]() {return false; })
	);
	CommandList->MapAction(
		FSubmitToolCommandList::Get().WidgetReflectCommandInfo,
		FExecuteAction::CreateRaw(this, &FSubmitToolCommandHandler::OnWidgetReflectCommandPressed),
		FCanExecuteAction::CreateLambda([]() {return true; }),
		FIsActionChecked::CreateLambda([]() {return false; })
	);
#endif
	CommandList->MapAction(
		FSubmitToolCommandList::Get().HelpCommandInfo,
		FExecuteAction::CreateRaw(this, &FSubmitToolCommandHandler::OnHelpCommandPressed),
		FCanExecuteAction::CreateLambda([]() {return true; }),
		FIsActionChecked::CreateLambda([]() {return false;})
	);

	CommandList->MapAction(
		FSubmitToolCommandList::Get().ExitCommandInfo,
		FExecuteAction::CreateRaw(this, &FSubmitToolCommandHandler::OnExitCommandPressed),
		FCanExecuteAction::CreateLambda([]() {return true; }),
		FIsActionChecked::CreateLambda([]() {return false; })
	);
}

#if !UE_BUILD_SHIPPING
void FSubmitToolCommandHandler::OnForceCrashCommandPressed()
{
	UE_FORCE_CRASH();
}

void FSubmitToolCommandHandler::OnWidgetReflectCommandPressed()
{
	FModuleManager::LoadModuleChecked<ISlateReflectorModule>("SlateReflector").DisplayWidgetReflector();
}
#endif

void FSubmitToolCommandHandler::OnHelpCommandPressed()
{
	UE_LOGF(LogSubmitToolDebug, Log, "OnHelpCommandPressed");

	FSlateApplication& SlateApplicaton = FSlateApplication::Get();

	if (SlateApplicaton.GetActiveModalWindow() != nullptr)
	{
		return;
	}

	TSharedPtr<SWidget> ParentWidget = SlateApplicaton.GetUserFocusedWidget(0);
	if (!ensure(ParentWidget))
	{
		return;
	}

	// Determine the position of the window so that it will spawn near the mouse, but not go off the screen.
	const FVector2D CursorPos = FSlateApplication::Get().GetCursorPos();
	FSlateRect Anchor(CursorPos.X, CursorPos.Y, CursorPos.X, CursorPos.Y);
	FVector2D AdjustedSummonLocation = FSlateApplication::Get().CalculatePopupWindowPosition(Anchor, FVector2D(441, 537), true, FVector2D::ZeroVector, Orient_Horizontal);

	TSharedPtr<SWindow> Window = SNew(SWindow)
		.AutoCenter(EAutoCenter::None)
		.ScreenPosition(AdjustedSummonLocation)
		.SupportsMaximize(false)
		.SupportsMinimize(false)
		.SizingRule(ESizingRule::Autosized)
		.Title(LOCTEXT("WindowHeader", "Help"));

	Window->SetContent(SNew(SSubmitToolHelpWidget)
	.ModelInterface(ModelInterface)
	);

	FSlateApplication::Get().AddModalWindow(Window.ToSharedRef(), ParentWidget);
}

void FSubmitToolCommandHandler::OnExitCommandPressed()
{
	UE_LOGF(LogSubmitToolDebug, Log, "OnExitCommandPressed");

	FSlateApplication& SlateApplicaton = FSlateApplication::Get();

	SlateApplicaton.CloseAllWindowsImmediately();
}

#undef LOCTEXT_NAMESPACE
