// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"


class FModelInterface;
class FUICommandList;

class FSubmitToolCommandHandler
{
public:
	void AddToCommandList(FModelInterface * InModelInterface, TSharedRef<FUICommandList> CommandList);

private:

#if !UE_BUILD_SHIPPING
	void OnWidgetReflectCommandPressed();
	void OnForceCrashCommandPressed();
#endif

	void OnHelpCommandPressed();
	void OnExitCommandPressed();

private:
	FModelInterface* ModelInterface;
};