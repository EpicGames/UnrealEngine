// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"
#include "SubmitToolStyle.h"


#define LOCTEXT_NAMESPACE "FSubmitToolCommandList"

class FSubmitToolCommandList : public TCommands<FSubmitToolCommandList>
{
public:
	FSubmitToolCommandList()
		: TCommands<FSubmitToolCommandList>("SubmitToolCommands", LOCTEXT("SubmitToolCommands","Submit Tool Commands"), NAME_None, FSubmitToolStyle::Get().GetStyleSetName())
	{

	}
public:

#if !UE_BUILD_SHIPPING
	TSharedPtr<FUICommandInfo> ForceCrashCommandInfo;
	TSharedPtr<FUICommandInfo> WidgetReflectCommandInfo;
#endif
	TSharedPtr<FUICommandInfo> HelpCommandInfo;
	TSharedPtr<FUICommandInfo> ExitCommandInfo;

public:
	void RegisterCommands()
	{

#if !UE_BUILD_SHIPPING
		UI_COMMAND(WidgetReflectCommandInfo, "Widget Reflector", "Open the Widget Reflector Tab.", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(ForceCrashCommandInfo, "Force Crash", "Force A Crash", EUserInterfaceActionType::Button, FInputChord());
#endif
		UI_COMMAND(HelpCommandInfo, "Help", "Show the help page for the SubmitTool.", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(ExitCommandInfo, "Exit", "Close the application.", EUserInterfaceActionType::Button, FInputChord());
	}
};

#undef LOCTEXT_NAMESPACE