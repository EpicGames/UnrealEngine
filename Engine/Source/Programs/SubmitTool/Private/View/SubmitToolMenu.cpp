// Copyright Epic Games, Inc. All Rights Reserved.

#include "SubmitToolMenu.h"

#define LOCTEXT_NAMESPACE "FSubmitToolMenu"

void FSubmitToolMenu::FillMainMenuEntries(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.BeginSection("Main");
	{
		MenuBuilder.AddMenuEntry(FSubmitToolCommandList::Get().HelpCommandInfo);
		MenuBuilder.AddMenuEntry(FSubmitToolCommandList::Get().ExitCommandInfo);
	}
	MenuBuilder.EndSection();
}

#if !UE_BUILD_SHIPPING
void FSubmitToolMenu::FillDebugMenuEntries(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.BeginSection("Debug");
	{
		MenuBuilder.AddMenuEntry(FSubmitToolCommandList::Get().ForceCrashCommandInfo);
		MenuBuilder.AddMenuEntry(FSubmitToolCommandList::Get().WidgetReflectCommandInfo);
	}
	MenuBuilder.EndSection();
}
#endif

#undef LOCTEXT_NAMESPACE