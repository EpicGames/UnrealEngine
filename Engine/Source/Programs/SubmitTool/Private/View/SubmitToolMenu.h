// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "SubmitToolCommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"

class FSubmitToolMenu
{
public:
	static void FillMainMenuEntries(FMenuBuilder& MenuBuilder);

#if !UE_BUILD_SHIPPING
	static void FillDebugMenuEntries(FMenuBuilder& MenuBuilder);
#endif
};
