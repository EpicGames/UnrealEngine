// Copyright Epic Games, Inc. All Rights Reserved.

#include "SubmitToolStyle.h"

#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateTypes.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/StarshipCoreStyle.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/StyleColors.h"
#include "Styling/SlateStyleMacros.h"

FName FSubmitToolStyle::StyleName("SubmitToolStyle");
TUniquePtr<FSubmitToolStyle> FSubmitToolStyle::Inst(nullptr);

const FName& FSubmitToolStyle::GetStyleSetName() const
{
	return StyleName;
}

const FSubmitToolStyle& FSubmitToolStyle::Get()
{
	if (!Inst.IsValid())
	{
		Inst = TUniquePtr<FSubmitToolStyle>(new FSubmitToolStyle());
	}
	return *Inst;
}

void FSubmitToolStyle::Shutdown()
{
	Inst.Reset();
}

FSubmitToolStyle::FSubmitToolStyle() : FSlateStyleSet(StyleName)
{
	SetParentStyleName(FAppStyle::GetAppStyleSetName());
	SetContentRoot(FPaths::EngineContentDir() / TEXT("Editor/Slate"));
	SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));
	
	Set("AppIcon", new IMAGE_BRUSH("Icons/EditorAppIcon", FVector2D(20, 20)));
	Set("AppIcon.Small", new IMAGE_BRUSH("Icons/EditorAppIcon", FVector2D(10, 10)));

	const FTextBlockStyle NormalLogText = FTextBlockStyle(GetWidgetStyle<FTextBlockStyle>("NormalText"))
		.SetFont(DEFAULT_FONT("Mono", 8))
		.SetColorAndOpacity(FStyleColors::Foreground)
		.SetSelectedBackgroundColor(FStyleColors::Highlight)
		.SetHighlightColor(FStyleColors::Black);

	const FTextBlockStyle BoldText = FTextBlockStyle(GetWidgetStyle<FTextBlockStyle>("NormalText"))
		.SetFont(DEFAULT_FONT("Bold", 11))
		.SetColorAndOpacity(FStyleColors::Foreground)
		.SetSelectedBackgroundColor(FStyleColors::Highlight)
		.SetHighlightColor(FStyleColors::Black);
	Set("BoldText", BoldText);


	const FTextBlockStyle BoldTextNormalSize = FTextBlockStyle(GetWidgetStyle<FTextBlockStyle>("NormalText"))
		.SetFont(DEFAULT_FONT("Bold", 9))
		.SetColorAndOpacity(FStyleColors::Foreground)
		.SetSelectedBackgroundColor(FStyleColors::Highlight)
		.SetHighlightColor(FStyleColors::Black);
	Set("BoldTextNormalSize", BoldTextNormalSize);

	FSlateFontInfo LargerFont = GetFontStyle("StandardDialog.LargeFont");

	LargerFont.Size = 16;
	Set("StandardDialog.TitleFont", LargerFont);

	Set("Log.Normal", NormalLogText);

	Set("Log.Warning", FTextBlockStyle(NormalLogText)
		.SetColorAndOpacity(FStyleColors::Warning)
	);

	Set("Log.Error", FTextBlockStyle(NormalLogText)
		.SetColorAndOpacity(FStyleColors::Error)
	);
	
	Set("Log.Success", FTextBlockStyle(NormalLogText)
		.SetColorAndOpacity(FStyleColors::Success)
	);
	
	Set("Log.TextBox", FEditableTextBoxStyle(GetWidgetStyle<FEditableTextBoxStyle>("NormalEditableTextBox"))
		.SetTextStyle(NormalLogText)
		.SetBackgroundImageNormal(BOX_BRUSH("Common/WhiteGroupBorder", FMargin(4.0f / 16.0f)))
		.SetBackgroundImageHovered(BOX_BRUSH("Common/WhiteGroupBorder", FMargin(4.0f / 16.0f)))
		.SetBackgroundImageFocused(BOX_BRUSH("Common/WhiteGroupBorder", FMargin(4.0f / 16.0f)))
		.SetBackgroundImageReadOnly(BOX_BRUSH("Common/WhiteGroupBorder", FMargin(4.0f / 16.0f)))
		.SetBackgroundColor(FStyleColors::Recessed)
	);

	FTextBlockStyle NormalText = GetParentStyle()->GetWidgetStyle<FTextBlockStyle>("NormalText");

	Set("RichTextBlock.TextHighlight", FTextBlockStyle(NormalText)
		.SetColorAndOpacity(FLinearColor(1.0f, 1.0f, 1.0f)));
	Set("RichTextBlock.Bold", FTextBlockStyle(NormalText)
		.SetFont(DEFAULT_FONT("Bold", FCoreStyle::RegularTextSize)));
	Set("RichTextBlock.BoldHighlight", FTextBlockStyle(NormalText)
		.SetFont(DEFAULT_FONT("Bold", FCoreStyle::RegularTextSize))
		.SetColorAndOpacity(FLinearColor(1.0f, 1.0f, 1.0f)));
	Set("RichTextBlock.Italic", FTextBlockStyle(NormalText)
		.SetFont(DEFAULT_FONT("Italic", FCoreStyle::RegularTextSize)));
	Set("RichTextBlock.ItalicHighlight", FTextBlockStyle(NormalText)
		.SetFont(DEFAULT_FONT("Italic", FCoreStyle::RegularTextSize))
		.SetColorAndOpacity(FLinearColor(1.0f, 1.0f, 1.0f)));
	Set("RichTextBlock.Red", FTextBlockStyle(NormalText)
		.SetColorAndOpacity(FLinearColor(1.0f, 0.1f, 0.1f)));

	Set("ValidatorStateWarning", FStyleColors::Warning.GetSpecifiedColor());
	Set("ValidatorStateFail", FStyleColors::Error.GetSpecifiedColor());
	Set("ValidatorStateSuccess", FStyleColors::Success.GetSpecifiedColor());
	Set("ValidatorStateDisabled", FStyleColors::AccentGray.GetSpecifiedColor());
	Set("ValidatorStateNormal", FStyleColors::AccentBlue.GetSpecifiedColor());
	static const FSlateColor& TabFlashColor = MakeShared<FLinearColor>(COLOR("18A0FBFF"));
	FLinearColor DockColor_Active(FColor(62, 62, 62));
	
	// Panel Tab
	Set("Docking.Tab", FDockTabStyle(GetWidgetStyle<FDockTabStyle>("Docking.Tab"))
		.SetColorOverlayTabBrush(BOX_BRUSH("/Docking/Tab_ColorOverlay", 4 / 16.0f))
		.SetContentAreaBrush(FSlateColorBrush(DockColor_Active))
		.SetFlashColor(TabFlashColor)
	);

	// Navigation defaults
	const FLinearColor NavHyperlinkColor(0.03847f, 0.33446f, 1.0f);
	const FTextBlockStyle NavigationHyperlinkText = FTextBlockStyle(NormalText)
		.SetFont(DEFAULT_FONT("Regular", 10))
		.SetColorAndOpacity(NavHyperlinkColor);
	
	const FButtonStyle NavigationHyperlinkButton = FButtonStyle()
		.SetNormal(BORDER_BRUSH("Old/HyperlinkDotted", FMargin(0, 0, 0, 3 / 16.0f), NavHyperlinkColor))
		.SetPressed(FSlateNoResource())
		.SetHovered(BORDER_BRUSH("Old/HyperlinkUnderline", FMargin(0, 0, 0, 3 / 16.0f), NavHyperlinkColor));

	FHyperlinkStyle NavigationHyperlink = FHyperlinkStyle()
		.SetUnderlineStyle(NavigationHyperlinkButton)
		.SetTextStyle(NavigationHyperlinkText)
		.SetPadding(FMargin(0.0f));

	Set("NavigationHyperlink", NavigationHyperlink);

	Set("AppIcon.DocumentationHelp", new IMAGE_BRUSH("Icons/Help/icon_Help_Documentation_16x", FVector2D(16, 16)));
	Set("AppIcon.Refresh", new IMAGE_BRUSH("Icons/refresh_12x", FVector2D(12, 12)));
	Set("AppIcon.OpenExternal", new IMAGE_BRUSH("Icons/Help/icon_Help_support_16x", FVector2D(12, 12)));
	Set("AppIcon.Star16", new IMAGE_BRUSH("Icons/Star_16x", FVector2D(16, 16)));
	
	// We use a custom teal color for the branched icons
	BranchedColor = FLinearColor::FromSRGBColor(FColor::FromHex("#00E4A0"));
	StatusCheckedOutColor = FLinearColor::FromSRGBColor(FColor::FromHex("#1FE44B"));

	Set("AppIcon.Generate", new CORE_IMAGE_BRUSH_SVG("Starship/Common/AutomationTools", CoreStyleConstants::Icon16x16));
	Set("RevisionControl.CheckedOut", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/SCC_CheckedOut", CoreStyleConstants::Icon16x16, StatusCheckedOutColor));
	Set("RevisionControl.OpenForAdd", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/RC_MarkedForAdd", CoreStyleConstants::Icon16x16, FStyleColors::AccentBlue));
	Set("RevisionControl.MarkedForDelete", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/SCC_MarkedForDelete", CoreStyleConstants::Icon16x16, FStyleColors::Error));
	Set("RevisionControl.Branched", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/SCC_Branched", CoreStyleConstants::Icon16x16, BranchedColor));

	FSlateStyleRegistry::RegisterSlateStyle(*this);
}

FSubmitToolStyle::~FSubmitToolStyle()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*this);
}
