// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Styling/SlateStyle.h"

class FSubmitToolStyle : public FSlateStyleSet
{
public:
	virtual const FName& GetStyleSetName() const override;

	static const FSubmitToolStyle& Get();
	static void Shutdown();

	~FSubmitToolStyle();

private:
	FSubmitToolStyle();

	/** The specific color we use for all the "Branched" icons */
	FSlateColor BranchedColor;
	/** The specific colors we use for all the "Status" icons */
	FSlateColor StatusCheckedOutColor;

	static FName StyleName;
	static TUniquePtr<FSubmitToolStyle> Inst;
};
