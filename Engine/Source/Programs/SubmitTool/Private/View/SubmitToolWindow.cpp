// Copyright Epic Games, Inc. All Rights Reserved.

#include "SubmitToolWindow.h"
#include "HAL/FileManager.h"
#include "Widgets/AutoUpdateWidget.h"
#include "Widgets/SubmitToolWidget.h"
#include "Widgets/SWindow.h"
#include "Models/ModelInterface.h"
#include "Models/SubmitToolUserPrefs.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SButton.h"
#include "Framework/Application/SlateApplication.h"

SubmitToolWindow::SubmitToolWindow(FModelInterface* modelInterface) : ModelInterface(modelInterface)
{}

TSharedRef<SDockTab> SubmitToolWindow::BuildMainTab(TSharedPtr<SWindow> InParentWindow, TSharedPtr<FSubmitToolOutputLogHistory> InLogHistory)
{
	//~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~
	// Create a submit files window
	MainTab = SNew(SDockTab)
		.TabRole(ETabRole::MajorTab)
		.OnCanCloseTab(SDockTab::FCanCloseTab::CreateRaw(this, &SubmitToolWindow::OnCanCloseTab))
		.OnTabClosed_Lambda([](TSharedRef<SDockTab> Tab){
							if(!Tab->GetParentWindow()->IsWindowMaximized())
							{
								FSubmitToolUserPrefs::Get()->WindowPosition = Tab->GetParentWindow()->GetPositionInScreen();
								FSubmitToolUserPrefs::Get()->WindowSize = Tab->GetParentWindow()->GetClientSizeInScreen();
								FSubmitToolUserPrefs::Get()->bWindowMaximized = Tab->GetParentWindow()->IsWindowMaximized();
							}
						})
		.Label_Lambda([&ModelInterface = ModelInterface]() {return FText::FromString(FString::Printf(TEXT("Changelist: %s"), *ModelInterface->GetCLID())); });

	if (ModelInterface->CheckForNewVersion())
	{
		CreateAutoUpdateSubmitToolContent(InParentWindow, InLogHistory);
		if (FSubmitToolUserPrefs::Get()->bAutoUpdate)
		{
			ModelInterface->InstallLatestVersion();
		}
	}
	else
	{
		CreateMainSubmitToolContent(InParentWindow, InLogHistory);
	}

	return MainTab.ToSharedRef();
}

void SubmitToolWindow::CreateAutoUpdateSubmitToolContent(TSharedPtr<SWindow> InParentWindow, TSharedPtr<FSubmitToolOutputLogHistory> InLogHistory)
{
	TSharedRef<SAutoUpdateWidget> AutoUpdateWidget =
		SNew(SAutoUpdateWidget)
		.ModelInterface(ModelInterface)
		.OnAutoUpdateCancelled_Lambda([InParentWindow, this, InLogHistory]() mutable { CreateMainSubmitToolContent(InParentWindow, MoveTemp(InLogHistory)); });

	ModelInterface->SetMainTab(MainTab);
	MainTab->SetContent(AutoUpdateWidget);
}

void SubmitToolWindow::CreateMainSubmitToolContent(TSharedPtr<SWindow> InParentWindow, TSharedPtr<FSubmitToolOutputLogHistory> InLogHistory)
{
	TSharedRef<SubmitToolWidget> SourceControlWidget =
		SNew(SubmitToolWidget, InLogHistory)
		.ParentWindow(InParentWindow)
		.ParentTab(MainTab)
		.ModelInterface(ModelInterface);

	ModelInterface->SetMainTab(MainTab);
	MainTab->SetContent(SourceControlWidget);
}

bool SubmitToolWindow::OnCanCloseTab()
{
	if (ModelInterface->IsBlockingOperationRunning() || ModelInterface->IsP4OperationRunning())
	{
		TSharedPtr<SWindow> WaitForP4Dialog = SNew(SWindow)
			.Title(FText::FromString("Closing requested"))
			.SizingRule(ESizingRule::Autosized)
			.SupportsMaximize(false)
			.SupportsMinimize(false)
			.HasCloseButton(false);

		WaitForP4Dialog->SetContent(
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			.Padding(5)
			.VAlign(VAlign_Fill)
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(FText::FromString("The window will close automatically once the current p4 operation is finished."))
				]
				+SVerticalBox::Slot()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					SNew(SButton)
					.Text(FText::FromString(TEXT("Cancel Operations")))
					.OnClicked_Lambda([this]() { ModelInterface->CancelP4Operations(); return FReply::Handled(); })
				]
			]
		);

		FSlateApplication::Get().AddModalWindow(WaitForP4Dialog.ToSharedRef(), nullptr, true);
		WaitForP4Dialog->ShowWindow();

		FTSTicker::GetCoreTicker().AddTicker(
			FTickerDelegate::CreateLambda([WaitForP4Dialog, this](float InDeltaTime)
			{
				if (ModelInterface->IsBlockingOperationRunning() || ModelInterface->IsP4OperationRunning())
				{
					return true;
				}

				WaitForP4Dialog->RequestDestroyWindow();
				MainTab->RequestCloseTab();

				return false;
			})
		);

		return false;
	}

	return true;
}

