// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SWindow.h"
#include "Widgets/Docking/SDockTab.h"

class FModelInterface;
class FSubmitToolOutputLogHistory;

class SubmitToolWindow
{
public:
	SubmitToolWindow(FModelInterface* modelInterface);

	TSharedRef<SDockTab> BuildMainTab(TSharedPtr<SWindow> InParentWindow, TSharedPtr<FSubmitToolOutputLogHistory> InLogHistory);

private:
	TSharedPtr<SDockTab> MainTab;
	
	bool OnCanCloseTab();
	
	void CreateAutoUpdateSubmitToolContent(TSharedPtr<SWindow> InParentWindow, TSharedPtr<FSubmitToolOutputLogHistory> InLogHistory);
	void CreateMainSubmitToolContent(TSharedPtr<SWindow> InParentWindow, TSharedPtr<FSubmitToolOutputLogHistory> InLogHistory);

	FModelInterface* ModelInterface;
};
