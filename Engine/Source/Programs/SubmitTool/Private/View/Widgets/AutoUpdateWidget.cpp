// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutoUpdateWidget.h"

#include "Widgets/Input/SButton.h"
#include "Widgets/Input/STextComboBox.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/SToolTip.h"

#include "Models/SubmitToolUserPrefs.h"
#include "Models/ModelInterface.h"
#include "View/SubmitToolStyle.h"

#define LOCTEXT_NAMESPACE "AutoUpdateWidget"

void SAutoUpdateWidget::Construct(const FArguments& InArgs)
{
	ModelInterface = InArgs._ModelInterface;
	OnAutoUpdateCancelled = InArgs._OnAutoUpdateCancelled;

	TSharedPtr<SVerticalBox> Contents;
	ChildSlot
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			[
				SNew(SBox)
				.HAlign(EHorizontalAlignment::HAlign_Center)
				.VAlign(EVerticalAlignment::VAlign_Center)
				[
					SAssignNew(Contents, SVerticalBox)
				]
			]
		];

	Contents->AddSlot()
		.AutoHeight()
		[
			SNew(SBox)
			.Padding(3, 3)
			[
				SNew(STextBlock)
				.TextStyle(FAppStyle::Get(), "BoldText")
				.Text(FText::FromString(TEXT("New Submit tool version available")))
				.Justification(ETextJustify::Center)
			]
		];

	// DEPLOY ID
	Contents->AddSlot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(3, 3)
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Font(FAppStyle::Get().GetFontStyle("SmallFont"))
				.MinDesiredWidth(80)
				.Justification(ETextJustify::Left)
				.Text(FText::FromString("Deploy Id"))
			]
			+ SHorizontalBox::Slot()
			.Padding(0, 3)
			.FillWidth(1.f)
			.VAlign(VAlign_Center)
			[
				SNew(SEditableTextBox)
				.IsReadOnly(true)
				.MinDesiredWidth(200)
				.Justification(ETextJustify::Left)
				.Text_Lambda([this]() { return FText::FromString(ModelInterface->GetDeployId()); })
			]
		];

	// LOCAL VERSION
	Contents->AddSlot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(3, 3)
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Font(FAppStyle::Get().GetFontStyle("SmallFont"))
				.MinDesiredWidth(80)
				.Justification(ETextJustify::Left)
				.Text(FText::FromString("Local Version"))
			]
			+ SHorizontalBox::Slot()
			.Padding(0, 3)
			.FillWidth(1.f)
			.VAlign(VAlign_Center)
			[
				SNew(SEditableTextBox)
				.IsReadOnly(true)
				.MinDesiredWidth(200)
				.Justification(ETextJustify::Left)
				.Text_Lambda([this]() { return FText::FromString(ModelInterface->GetLocalVersion()); })
			]
		];

	// LATEST VERSION
	Contents->AddSlot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(3, 3)
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Font(FAppStyle::Get().GetFontStyle("SmallFont"))
				.MinDesiredWidth(80)
				.Justification(ETextJustify::Left)
				.Text(FText::FromString("Latest Version"))
			]
			+ SHorizontalBox::Slot()
			.Padding(0, 3)
			.FillWidth(1.f)
			.VAlign(VAlign_Center)
			[
				SNew(SEditableTextBox)
				.IsReadOnly(true)
				.MinDesiredWidth(200)
				.Justification(ETextJustify::Left)
				.Text_Lambda([this]() { return FText::FromString(ModelInterface->GetLatestVersion()); })
			]
		];

	// BUTTONS
	Contents->AddSlot()
		.Padding(0, 5)
		.AutoHeight()
		[
			SNew(SUniformGridPanel)
			.SlotPadding(FAppStyle::GetMargin("StandardDialog.SlotPadding"))
			.MinDesiredSlotWidth(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotWidth"))
			+ SUniformGridPanel::Slot(0, 0)
			[				
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "PrimaryButton")
				.ToolTip(SNew(SToolTip).Text(FText::FromString(TEXT("Pushing this button will download the latest version, close the SubmitTool, install it and restart the SubmitTool with the current parameters."))))
				.IsEnabled_Lambda([this]() { return ModelInterface->GetDownloadMessage().IsEmpty(); })
				.OnClicked_Lambda([this]() { ModelInterface->InstallLatestVersion(); return FReply::Handled(); })						
				[
					SNew(STextBlock)
					.MinDesiredWidth(130)
					.Justification(ETextJustify::Center)
					.Text(FText::FromString("Download"))
				]
			]
			+ SUniformGridPanel::Slot(1, 0)
			[
				SNew(SButton)
				.ToolTip(SNew(SToolTip).Text(FText::FromString(TEXT("Pushing this button will cancel the current download and close the window to go back to the regular SubmitTool UI."))))
				.OnClicked_Lambda([this]() { Cancel(); return FReply::Handled(); })		
				[
					SNew(STextBlock)
					.MinDesiredWidth(130)
					.Justification(ETextJustify::Center)
					.Text_Lambda([this](){ return ModelInterface->GetDownloadMessage().IsEmpty() ? FText::FromString(TEXT("Use Current Version")) : FText::FromString(TEXT("Cancel Download")); })
				]
			]
		];

	Contents->AddSlot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.Padding(3, 3)
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SCheckBox)
					.IsChecked_Lambda([this](){ return FSubmitToolUserPrefs::Get()->bAutoUpdate ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;})
					.OnCheckStateChanged_Lambda([this](ECheckBoxState InCheckBoxState){
						FSubmitToolUserPrefs::Get()->bAutoUpdate = !FSubmitToolUserPrefs::Get()->bAutoUpdate;
					})
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SButton)
					.ButtonStyle(FAppStyle::Get(), "InvisibleButton")
					.IsFocusable(false)	
					.OnClicked_Lambda([]() { FSubmitToolUserPrefs::Get()->bAutoUpdate = !FSubmitToolUserPrefs::Get()->bAutoUpdate; return FReply::Handled(); })
					[
						SNew(STextBlock)
						.Justification(ETextJustify::Left)
						.Font(FAppStyle::Get().GetFontStyle("SmallFont"))
						.MinDesiredWidth(60)
						.Text(FText::FromString(TEXT("Auto Update")))
					]
				]
		];

	Contents->AddSlot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
				.Padding(3, 3)
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.MinDesiredWidth(70)
					.Justification(ETextJustify::Left)
					.Visibility_Lambda([this]() { return (ModelInterface->GetDownloadMessage().IsEmpty()) ? EVisibility::Hidden : EVisibility::All; })
					.Text_Lambda([this]() {	return FText::FromString(ModelInterface->GetDownloadMessage()); })
				]
		];

}

void SAutoUpdateWidget::Cancel()
{
	ModelInterface->CancelInstallLatestVersion();
	OnAutoUpdateCancelled.ExecuteIfBound();
}

#undef LOCTEXT_NAMESPACE