// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Styling/SlateTypes.h"
#include "Logic/SwarmService.h"
#include "Containers/ObservableArray.h"

class FModelInterface;

DECLARE_DELEGATE(FOnAutoUpdateCancelledSignature)

class SAutoUpdateWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SAutoUpdateWidget) {}
		SLATE_ARGUMENT(FModelInterface*, ModelInterface)
		SLATE_EVENT(FOnAutoUpdateCancelledSignature, OnAutoUpdateCancelled)
	SLATE_END_ARGS()

	virtual ~SAutoUpdateWidget() {}

	void Construct(const FArguments& InArgs);
private:
	void Cancel();
	FString HumanReadableSize(long bytes);

private:
	FModelInterface* ModelInterface;
	FOnAutoUpdateCancelledSignature OnAutoUpdateCancelled;
};