// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConfirmDialogWidget.h"

#include "View/SubmitToolStyle.h"
#include "Widgets/SWindow.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Text/SRichTextBlock.h"

void SConfirmDialogWidget::Construct(const FArguments& InArgs)
{
	ResultsCallback = InArgs._ResultCallback;
	IsBtnEnabled = InArgs._IsBtnEnabled;
	TSharedPtr<SHorizontalBox> ButtonBox;
	
	FText DescriptionText = InArgs._DescriptionText;
	FString DescriptionString = InArgs._DescriptionText.ToString();
	if (DescriptionString.Contains(TEXT("<a")))
	{
		DescriptionText = FText::FromString(DescriptionString.Replace(TEXT("\'"), TEXT("\"")));
	}

	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		.Padding(5)
		.VAlign(VAlign_Fill)
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(5)
			[
				SNew(SRichTextBlock)
				.Text(DescriptionText)
				.DecoratorStyleSet(&FAppStyle::Get())
				+SRichTextBlock::HyperlinkDecorator(TEXT("browser"), FSlateHyperlinkRun::FOnClick::CreateLambda(
					[](const FSlateHyperlinkRun::FMetadata& Metadata)
					{
						const FString* UrlPtr = Metadata.Find(TEXT("href"));
						if(UrlPtr)
						{
							FPlatformProcess::LaunchURL(**UrlPtr, nullptr, nullptr);
						}
					}))
			]
			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(5)
			[
				InArgs._AdditionalContent == nullptr ? SNullWidget::NullWidget : InArgs._AdditionalContent.ToSharedRef()
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0, 5, 0, 0)
			.HAlign(HAlign_Right)
			[
				SAssignNew(ButtonBox, SHorizontalBox)
			]
		]
	];

	
	for(size_t i=0;i<InArgs._Buttons.Num();++i)
	{
		ConstructButton(ButtonBox.ToSharedRef(), i, InArgs._Buttons[i], i == 0);
	}
}

void SConfirmDialogWidget::ConstructButton(const TSharedRef<SHorizontalBox> Container, size_t Idx, const FString& ButtonText, bool IsPrimary)
{	
	Container->AddSlot()
		.AutoWidth()
		.HAlign(HAlign_Right)
		.Padding(5, 0, 0, 0)
		[
			SNew(SButton)
			.Text(FText::FromString(ButtonText))
			.IsEnabled_Lambda([Idx, this]{ return IsBtnEnabled == nullptr || IsBtnEnabled(Idx);})
			.OnClicked_Lambda([this, Idx]()
			{
				ResultsCallback.ExecuteIfBound(Idx);
				return FReply::Handled();
			})
			.ButtonStyle(FSubmitToolStyle::Get(), IsPrimary ? "PrimaryButton" : "Button")
		];
}
