// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

DECLARE_DELEGATE_OneParam(FOnResult, size_t)

class SHorizontalBox;

class SConfirmDialogWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SConfirmDialogWidget) {}
		SLATE_ARGUMENT(FText, Title)
		SLATE_ARGUMENT(FText, DescriptionText)
		SLATE_ARGUMENT(TSharedPtr<SWidget>, AdditionalContent)
		SLATE_ARGUMENT(TArray<FString>, Buttons)
		SLATE_ARGUMENT(TFunction<bool(size_t)>, IsBtnEnabled)
		SLATE_EVENT(FOnResult, ResultCallback)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

private:
	FOnResult ResultsCallback;
	TFunction<bool(size_t)> IsBtnEnabled = nullptr;
	
	void ConstructButton(
		const TSharedRef<SHorizontalBox> Container,
		size_t Idx,
		const FString& ButtonText,
		bool IsPrimary);
};