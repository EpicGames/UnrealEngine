// Copyright Epic Games, Inc. All Rights Reserved.

#include "SIntegrationWidget.h"
#include "View/SubmitToolStyle.h"
#include "View/Widgets/SelectSourceControlUserWidget.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"
#include "Widgets/Input/SHyperlink.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Styling/StyleColors.h"

#include "Models/ModelInterface.h"
#include "Models/SubmitToolUserPrefs.h"
#include "Models/IntegrationOptions.h"
#include "SubmitToolUtils.h"

void SIntegrationWidget::Construct(const FArguments& InArgs)
{
	ModelInterface = InArgs._ModelInterface;
	MainWindow = InArgs._MainWindow;
	
	ModelInterface->RegisterTagUpdatedCallback(FTagUpdated::FDelegate::CreateLambda([this](const FTag& InTag) {
		if(ParentWindow.IsValid() && ParentWindow->IsVisible() && InTag.Definition.TagId.Equals(TEXT("#jira"), ESearchCase::IgnoreCase))
		{
			bAreFieldsValid = ModelInterface->ValidateIntegrationOptions(true);
		}
		}));
	
	TSharedRef<SScrollBox> ScrollBox = SNew(SScrollBox)
		.Orientation(EOrientation::Orient_Vertical)
		+ SScrollBox::Slot()
		.Padding(5, 5)
		.AutoSize()
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.Padding(FAppStyle::GetMargin("StandardDialog.SlotPadding").Left, 5)
			.AutoHeight()
			[
				SNew(STextBlock)
				.Font(FAppStyle::Get().GetFontStyle("StandardDialog.TitleFont"))
				.Text(FText::FromString(TEXT("Integration Options")))
			]
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				BuildOptions()
			]
		]
		+SScrollBox::Slot()
		.Padding(5, 5)
		.AutoSize()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(FAppStyle::GetMargin("StandardDialog.SlotPadding"))
			.AutoWidth()
			[
				SNew(SHyperlink)
				.Style(FSubmitToolStyle::Get(), TEXT("NavigationHyperlink"))
				.IsEnabled_Lambda([&ModelInterface = ModelInterface]{ return ModelInterface->GetSwarmReview().IsValid(); })
				.Text_Lambda([this]() { return FText::FromString(GetSwarmLinkText()); })
				.ToolTipText_Lambda([this]() { FString Url; return ModelInterface->GetSwarmReviewUrl(Url) ? FText::FromString(Url) : FText::FromString("No Swarm Review Specified");  })
				.OnNavigate_Lambda([this]() { FString Url; if(ModelInterface->GetSwarmReviewUrl(Url)) { FPlatformProcess::LaunchURL(*Url, nullptr, nullptr); }})
			]
		]
		+SScrollBox::Slot()
		.Padding(5, 5)
		.AutoSize()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(FAppStyle::GetMargin("StandardDialog.SlotPadding"))
			.AutoWidth()
			[
				SNew(STextBlock)
				.ColorAndOpacity(FStyleColors::Error)
				.Visibility_Lambda([&ModelInterface = ModelInterface]{ return ModelInterface->HasSubmitToolTag() ? EVisibility::Collapsed : EVisibility::All; })
				.Text(FText::FromString(TEXT("Required validations have failed or need to finish running.")))
			]
		]
		+SScrollBox::Slot()
		.Padding(5, 5)
		.AutoSize()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.FillWidth(1.f)
			[
				SNew(SSpacer)
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Right)
			[		
				SNew(SButton)
				.Text(FText::FromString(TEXT("Request Integration")))
				.ButtonStyle(FAppStyle::Get(), "PrimaryButton")
				.IsEnabled_Lambda([this]
					{
						return ModelInterface->GetState() != ESubmitToolAppState::Finished && bAreFieldsValid && !ModelInterface->IsBlockingOperationRunning() && ModelInterface->HasSubmitToolTag() && ModelInterface->IsIntegrationRequired();
					})
				.OnClicked(this, &SIntegrationWidget::OnRequestIntegrationClicked)
			]
		];
		
	ChildSlot
		[
			SNew(SBorder)
			.Padding(5,10)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			[
				ScrollBox
			]
		];
}

SIntegrationWidget::~SIntegrationWidget() {}

void SIntegrationWidget::Open()
{
	if(!ParentWindow.IsValid())
	{
		ParentWindow = SNew(SWindow)
			.SizingRule(ESizingRule::UserSized)
			.SupportsMaximize(false)
			.SupportsMinimize(false)
			.MinWidth(300)
			.MinHeight(200);

		ParentWindow->SetRequestDestroyWindowOverride(FRequestDestroyWindowOverride::CreateLambda([](const TSharedRef<SWindow>& Window) { Window->HideWindow(); }));
		FSlateApplication::Get().AddWindowAsNativeChild(ParentWindow.ToSharedRef(), MainWindow.ToSharedRef(), false);

		ParentWindow->SetContent(AsShared());
	}

	ParentWindow->SetTitle(FText::FromString(FString::Printf(TEXT("Integration information for CL %s"), *ModelInterface->GetCLID())));

	FDeprecateSlateVector2D NewPosition = MainWindow->GetPositionInScreen();
	NewPosition.X += MainWindow->GetSizeInScreen().X;
	ParentWindow->MoveWindowTo(NewPosition);

	float ySize = ParentWindow->IsWindowMaximized() ? 300 : MainWindow->GetSizeInScreen().Y - 40;
	ParentWindow->Resize(FDeprecateSlateVector2D(400, ySize));

	FSubmitToolUtils::EnsureWindowIsInView(ParentWindow.ToSharedRef(), true);

	ParentWindow->BringToFront();
	ParentWindow->ShowWindow();

	bAreFieldsValid = ModelInterface->ValidateIntegrationOptions(true);
}

FReply SIntegrationWidget::OnCloseClicked()
{
	this->ParentWindow->HideWindow();
	return FReply::Handled();	
}

void SIntegrationWidget::UpdateUIOptions()
{
	const TMap<FString, TSharedPtr<FIntegrationOptionBase>>& Options = ModelInterface->GetIntegrationOptions();
	for(const TPair<FString, TSharedPtr<FIntegrationOptionBase>>& Pair : Options)
	{
		if(Pair.Value->FieldDefinition.DependsOn.Num() > 0)
		{
			EVisibility Visibility = EVisibility::Collapsed;
			for(const FString& Dependency : Pair.Value->FieldDefinition.DependsOn)
			{
				if(Options.Contains(Dependency))
				{
					FString ActualValue;
					if(Options[Dependency]->GetJiraValue(ActualValue) || (!Pair.Value->FieldDefinition.DependsOnValue.IsEmpty() && Pair.Value->FieldDefinition.DependsOnValue.Equals(ActualValue)))
					{
						Visibility = EVisibility::All;
					}
				}
			}

			UIOptionsWidget[Pair.Key]->SetVisibility(Pair.Value->FieldDefinition.DependsOn.IsEmpty() ? EVisibility::All : Visibility);
		}
	}
}

FString SIntegrationWidget::GetSwarmLinkText()
{
	FString SwarmLinkText;
	if(ModelInterface->GetSwarmReview().IsValid())
	{
		SwarmLinkText = TEXT("Swarm Review ") + FString::FromInt(ModelInterface->GetSwarmReview()->Id);
	}
	else
	{
		SwarmLinkText = TEXT("No Swarm Review Specified");
	}

	return SwarmLinkText;
}

TSharedRef<SWidget> SIntegrationWidget::BuildOptions()
{
	TSharedRef<SVerticalBox> Parent = SNew(SVerticalBox);
	TSharedRef<SHorizontalBox> Row = SNew(SHorizontalBox);

	bool bHalfRow = false;
	for(const TPair<FString, TSharedPtr<FIntegrationOptionBase>>& Pair : ModelInterface->GetIntegrationOptions())
	{
		bool bExpandsTwoColumns;

		TSharedPtr<SWidget> Widget;
		switch(Pair.Value->FieldDefinition.Type)
		{
		case EFieldType::Bool:
			bExpandsTwoColumns = false;
			Widget = CheckboxWithLabel(StaticCastSharedPtr<FIntegrationBoolOption>(Pair.Value));
			break;

		case EFieldType::Text:
			bExpandsTwoColumns = true;
			Widget = TextWithLabel(StaticCastSharedPtr<FIntegrationTextOption>(Pair.Value));
			break;

		case EFieldType::MultiText:
			bExpandsTwoColumns = true;
			Widget = MultiTextWithLabel(StaticCastSharedPtr<FIntegrationTextOption>(Pair.Value));
			break;

		case EFieldType::Combo:
			bExpandsTwoColumns = true;
			Widget = ComboWithLabel(StaticCastSharedPtr<FIntegrationComboOption>(Pair.Value));
			break;

		case EFieldType::PerforceUser:

			bExpandsTwoColumns = true;
			Widget = PerforceUserSelect(StaticCastSharedPtr<FIntegrationTextOption>(Pair.Value));
			break;
			
		case EFieldType::UILabel:
			bExpandsTwoColumns = true;
			Widget = SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.FillWidth(1.f)
					[
						SNew(STextBlock)
						.AutoWrapText(true)
						.Text(FText::FromString(Pair.Value->FieldDefinition.LabelDisplay))
						.TextStyle(FAppStyle::Get(), "BoldText")
					];
			break;

		case EFieldType::UISpace:
			bExpandsTwoColumns = true;
			Widget = SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.FillWidth(1.f)
					.Padding(0,4)
					[
						SNew(SSpacer)
					];
			break;

		default:
			bExpandsTwoColumns = false;
			Widget = SNullWidget::NullWidget;
			UE_LOGF(LogSubmitTool, Error, "Invalid type specified for Integration Option");
			break;
		}

		UIOptionsWidget.Add(Pair.Value->FieldDefinition.Name, Widget.ToSharedRef());

		if(bHalfRow && bExpandsTwoColumns)
		{
			Parent->AddSlot()
				.AutoHeight()
				[
					Row
				];
			Row = SNew(SHorizontalBox);
		}

		Row->AddSlot()
			.FillWidth(1.f)
			.Padding(FAppStyle::GetMargin("StandardDialog.SlotPadding").Left, 3)
			.AttachWidget(Widget.ToSharedRef());

		if(bExpandsTwoColumns)
		{
			Parent->AddSlot()
				.AutoHeight()
				[
					Row
				];

			Row = SNew(SHorizontalBox);
			bHalfRow = false;
		}
		else
		{
			bHalfRow = !bHalfRow;
			if(!bHalfRow)
			{
				Parent->AddSlot()
					.AutoHeight()
					[
						Row
					];

				Row = SNew(SHorizontalBox);
				bHalfRow = false;
			}
		}
	}

	UpdateUIOptions();
	return Parent;
}

TSharedRef<SHorizontalBox> SIntegrationWidget::CheckboxWithLabel(TSharedPtr<FIntegrationBoolOption> InOutOption)
{	
	return SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SCheckBox)
				.IsChecked_Lambda([&Field = InOutOption->Value]() { return Field ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
				.ToolTipText(FText::FromString(InOutOption->FieldDefinition.Tooltip))
				.OnCheckStateChanged_Lambda([this, IntegrationOption = InOutOption](ECheckBoxState InNewState)
				{
					IntegrationOption->Value = InNewState == ECheckBoxState::Checked;
					IntegrationValueChanged(IntegrationOption);
				})
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "InvisibleButton")
				.IsFocusable(false)
				.OnClicked_Lambda([this, InOutOption = InOutOption]() { InOutOption->Value = !InOutOption->Value; IntegrationValueChanged(InOutOption); return FReply::Handled(); })
				[
					SNew(STextBlock)
					.ColorAndOpacity_Lambda([InOutOption] { return InOutOption->bInvalid ? FStyleColors::Error : FStyleColors::White; })
					.Justification(ETextJustify::Left)
					.MinDesiredWidth(60)
					.ToolTipText(FText::FromString(InOutOption->FieldDefinition.Tooltip))
					.Text(FText::FromString(InOutOption->FieldDefinition.LabelDisplay))
				]
			];
}

TSharedRef<SHorizontalBox> SIntegrationWidget::TextWithLabel(TSharedPtr<FIntegrationTextOption> InOutOption)
{
	TSharedRef<SHorizontalBox> Widget = SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
					.ColorAndOpacity_Lambda([InOutOption] { return InOutOption->bInvalid ? FStyleColors::Error : FStyleColors::White; })
					.Justification(ETextJustify::Left)
					.MinDesiredWidth(60)
					.ToolTipText(FText::FromString(InOutOption->FieldDefinition.Tooltip))
					.Text(FText::FromString(InOutOption->FieldDefinition.LabelDisplay))
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SEditableTextBox)
				.Text_Lambda([InOutOption = InOutOption] {return FText::FromString(InOutOption->Value); })
				.ToolTipText(FText::FromString(InOutOption->FieldDefinition.Tooltip))
				.OnTextChanged_Lambda([this, InOutOption = InOutOption](const FText& InText) {
						InOutOption->Value = InText.ToString();
						IntegrationValueChanged(InOutOption);
					})
			];
			Widget->EnableToolTipForceField(true);

	return Widget;
}

TSharedRef<SVerticalBox> SIntegrationWidget::MultiTextWithLabel(TSharedPtr<FIntegrationTextOption> InOutOption)
{
	TSharedRef<SVerticalBox> Widget = SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()			
			[
				SNew(STextBlock)
					.ColorAndOpacity_Lambda([InOutOption] { return InOutOption->bInvalid ? FStyleColors::Error : FStyleColors::White; })
					.Justification(ETextJustify::Left)
					.MinDesiredWidth(60)
					.ToolTipText(FText::FromString(InOutOption->FieldDefinition.Tooltip))
					.Text(FText::FromString(InOutOption->FieldDefinition.LabelDisplay))
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0, 3)
			.VAlign(VAlign_Center)
			[
				SNew(SMultiLineEditableTextBox)
				.AutoWrapText(true)
				.ToolTipText(FText::FromString(InOutOption->FieldDefinition.Tooltip))
				.Text_Lambda([InOutOption = InOutOption] {return FText::FromString(InOutOption->Value); })
				.OnTextChanged_Lambda([this, InOutOption = InOutOption](const FText& InText) { 
						InOutOption->Value = InText.ToString();
						IntegrationValueChanged(InOutOption); 
					})
			];
			Widget->EnableToolTipForceField(true);

	return Widget;
}

TSharedRef<SHorizontalBox> SIntegrationWidget::ComboWithLabel(TSharedPtr<FIntegrationComboOption> InOutOption)
{
	return SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0, 3)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
					.ColorAndOpacity_Lambda([InOutOption]{ return InOutOption->bInvalid ? FStyleColors::Error : FStyleColors::White; })
					.Justification(ETextJustify::Left)
					.MinDesiredWidth(60)
					.ToolTipText(FText::FromString(InOutOption->FieldDefinition.Tooltip))
					.Text(FText::FromString(InOutOption->FieldDefinition.LabelDisplay))
			]
			+SHorizontalBox::Slot()
			.Padding(FAppStyle::GetMargin("StandardDialog.SlotPadding").Left, 3)
			.FillWidth(1.f)
			.VAlign(VAlign_Center)
			[
				SNew(SComboBox<TSharedPtr<FString>>)
				.ToolTipText(FText::FromString(InOutOption->FieldDefinition.Tooltip))
				.OptionsSource(&InOutOption->ComboValues)
				.OnGenerateWidget_Lambda([](TSharedPtr<FString> InString) { return SNew(STextBlock).Text(FText::FromString(*InString)); })
				.OnSelectionChanged_Lambda([this, InOutOption = InOutOption](TSharedPtr<FString> StringItem, ESelectInfo::Type SelectInfo) 
					{ 
						InOutOption->Value = *StringItem;
						IntegrationValueChanged(InOutOption);
					})
				[
					SNew(STextBlock)
					.Text_Lambda([InOutOption = InOutOption] { return InOutOption->Value.IsEmpty() ? FText::FromString(TEXT("Select...")) : FText::FromString(InOutOption->Value); })
				]
			];
}

TSharedRef<SVerticalBox> SIntegrationWidget::PerforceUserSelect(TSharedPtr<FIntegrationTextOption> InOutOption)
{

	return SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()			
			[
				SNew(STextBlock)
					.ColorAndOpacity_Lambda([InOutOption] { return InOutOption->bInvalid ? FStyleColors::Error : FStyleColors::White; })
					.Justification(ETextJustify::Left)
					.MinDesiredWidth(60)
					.ToolTipText(FText::FromString(InOutOption->FieldDefinition.Tooltip))
					.Text(FText::FromString(InOutOption->FieldDefinition.LabelDisplay))
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0, 3)
			.VAlign(VAlign_Center)
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.FillWidth(1.f)
				.VAlign(VAlign_Center)
				[					
					SNew(SEditableTextBox)
					.Text_Lambda([InOutOption = InOutOption] {return FText::FromString(InOutOption->Value); })
					.ToolTipText(FText::FromString(InOutOption->FieldDefinition.Tooltip))
					.OnTextChanged_Lambda([this, InOutOption = InOutOption](const FText& InText) {
							InOutOption->Value = InText.ToString();
							IntegrationValueChanged(InOutOption);
						})
				]
				+SHorizontalBox::Slot()
				.Padding(3, 0, 0, 0)
				.AutoWidth()
				[
					SNew(SBox)
					.MinDesiredWidth(70)
					[
						SNew(SSelectSourceControlUserWidget)
							.ButtonText(FText::FromString("Users"))
							.ModelInterface(ModelInterface)
							.TargetText(&InOutOption->Value)
							.TargetName(InOutOption->FieldDefinition.Name)
							.IsEnabled_Lambda([this]() { return FModelInterface::GetInputEnabled(); })
					]
				]
			];
}

void SIntegrationWidget::IntegrationValueChanged(const TSharedPtr<FIntegrationOptionBase>& InOption)
{
	UpdateUIOptions();

	if(!InOption->FieldDefinition.ValidationGroups.IsEmpty() || InOption->FieldDefinition.bRequiredValue)
	{
		bAreFieldsValid = ModelInterface->ValidateIntegrationOptions(false);
	}
}

FReply SIntegrationWidget::OnRequestIntegrationClicked()
{
	ModelInterface->RequestIntegration();

	return FReply::Handled();
}
