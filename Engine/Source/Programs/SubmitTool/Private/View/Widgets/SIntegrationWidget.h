// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConfirmDialogWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "Parameters/SubmitToolParameters.h"

class FModelInterface;
class SMultiLineEditableTextBox;
class SDockTab;
class SWindow;
class SVerticalBox;
class SHorizontalBox;
class SButton;
class FTabManager;
class SExpandableArea;
class FJiraService;
class FIntegrationOptionBase;
class FIntegrationBoolOption;
class FIntegrationTextOption;
class FIntegrationComboOption;

class SIntegrationWidget final : public SCompoundWidget
{

public:
	SLATE_BEGIN_ARGS(SIntegrationWidget) {}
		SLATE_ARGUMENT(FModelInterface*, ModelInterface)
		SLATE_ARGUMENT(TSharedPtr<SWindow>, MainWindow)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
	virtual ~SIntegrationWidget() override;

	void Open();

private:

	TSharedRef<SWidget> BuildOptions();
	TSharedRef<SHorizontalBox> CheckboxWithLabel(TSharedPtr<FIntegrationBoolOption> InOutOption);
	TSharedRef<SHorizontalBox> TextWithLabel(TSharedPtr<FIntegrationTextOption> InOutOption);
	TSharedRef<SVerticalBox> MultiTextWithLabel(TSharedPtr<FIntegrationTextOption> InOutOption);
	TSharedRef<SHorizontalBox> ComboWithLabel(TSharedPtr<FIntegrationComboOption> InOutOption);
	TSharedRef<SVerticalBox> PerforceUserSelect(TSharedPtr<FIntegrationTextOption> InOutOption);

	void IntegrationValueChanged(const TSharedPtr<FIntegrationOptionBase>& InOption);

	void UpdateUIOptions();

	FReply OnRequestIntegrationClicked();

	FReply OnCloseClicked();

	TMap<FString, TSharedRef<SWidget>> UIOptionsWidget;

	TSharedPtr<SWindow> MainWindow;
	TSharedPtr<SWindow> ParentWindow;
	FModelInterface* ModelInterface;
	bool bAreFieldsValid;

	FString GetSwarmLinkText();

	FString SwarmReviewID;
};
