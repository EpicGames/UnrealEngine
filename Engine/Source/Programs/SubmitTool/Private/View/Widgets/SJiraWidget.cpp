// Copyright Epic Games, Inc. All Rights Reserved.

#include "SJiraWidget.h"
#include "Models/Tag.h"
#include "Models/ModelInterface.h"
#include "Logic/JiraService.h"

#include "Framework/Application/SlateApplication.h"

#include "View/SubmitToolStyle.h"
#include "Widgets/Layout/SScrollBorder.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/Input/SHyperlink.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/SWindow.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/SToolTip.h"

#include "SubmitToolUtils.h"

namespace JiraIssuesColumns
{
	static const FName Checked("Checked");
	static const FName IssueType("IssueType");
	static const FName Issue("Issue");
	static const FName Status("Status");
	static const FName Summary("Summary");
};

constexpr const TCHAR* FilteredStatus[] = {
	TEXT("Closed"),
	TEXT("Done"),
	TEXT("Work Complete"),
};

void SJiraWidget::Construct(const FArguments& InArgs)
{
	ModelInterface = InArgs._ModelInterface;
	JiraService = ModelInterface->GetJiraService();

	PresubmitCallbackHandle = ModelInterface->PrepareSubmitCallBack.AddSP(this, &SJiraWidget::OnSubmitCallback);

	MainWindow = InArgs._ParentWindow.ToSharedRef();
	ParentWindow = nullptr;

	UsernameField = SNew(SEditableTextBox).Text(FText::FromString(ModelInterface->GetUsername())).IsEnabled_Lambda([this] { return !JiraService.Pin()->bOngoingRequest; });
	PasswordField = SNew(SEditableTextBox).IsPassword(true).IsEnabled_Lambda([this] { return !JiraService.Pin()->bOngoingRequest; });;

	CredentialSection = BuildCredentialsWidget();
	IssuesSection = BuildIssuesWidget();
	ChildSlot.AttachWidget(CredentialSection.ToSharedRef());

	JiraService.Pin()->OnJiraIssuesRetrievedCallback.BindSP(this, &SJiraWidget::JiraIssuesAvailable);

	if(JiraService.Pin()->GetIssues().Num() > 0)
	{
		JiraIssuesAvailable(true);
	}
}

SJiraWidget::~SJiraWidget()
{
	if (this->PresubmitCallbackHandle.IsValid())
	{
		this->ModelInterface->PrepareSubmitCallBack.Remove(this->PresubmitCallbackHandle);
	}
}

void SJiraWidget::OnSubmitCallback()
{
	if (this->ParentWindow != nullptr && this->ParentWindow->IsVisible())
	{
		this->ParentWindow->HideWindow();
	}
}

void SJiraWidget::Open(const FTag* InTargetTag)
{
	if(!ParentWindow.IsValid())
	{
		ParentWindow = SNew(SWindow)
			.SizingRule(ESizingRule::UserSized)
			.SupportsMaximize(false)
			.SupportsMinimize(false)
			.MinWidth(850)
			.MinHeight(400);

		ParentWindow->SetRequestDestroyWindowOverride(FRequestDestroyWindowOverride::CreateLambda([this](const TSharedRef<SWindow>& Window) { Window->HideWindow(); }));
		FSlateApplication::Get().AddWindowAsNativeChild(ParentWindow.ToSharedRef(), MainWindow.ToSharedRef(), false);

		FDeprecateSlateVector2D NewPosition = MainWindow->GetPositionInScreen();
		NewPosition.X += MainWindow->GetSizeInScreen().X;
		ParentWindow->MoveWindowTo(NewPosition);

		FSubmitToolUtils::EnsureWindowIsInView(ParentWindow.ToSharedRef(), true);

		ParentWindow->SetContent(AsShared());
	}

	TargetTag = InTargetTag;
	ParentWindow->SetTitle(FText::FromString(FString::Format(TEXT("Select issues for Tag '{0}'"), { TargetTag->Definition.TagLabel })));
	ParentWindow->BringToFront();
	ParentWindow->ShowWindow();
}

TSharedPtr<SWidget> SJiraWidget::BuildCredentialsWidget()
{ 
	return SNew(SBox)
		.HAlign(EHorizontalAlignment::HAlign_Center)
		.VAlign(EVerticalAlignment::VAlign_Center)
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.Padding(0, -15, 0, 0)
			.HAlign(EHorizontalAlignment::HAlign_Center)
			.VAlign(EVerticalAlignment::VAlign_Center)
			.AutoHeight()
			[
				SNew(STextBlock)
				.Text_Lambda([this]() {return JiraService.Pin()->bOngoingRequest ? NSLOCTEXT("JiraWindow", "Login", "Login in Progress...") : FText(); })
			]
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				.Visibility_Lambda([this]() { return ModelInterface->GetParameters().JiraParameters.OAuthClientId.IsEmpty() ? EVisibility::All : EVisibility::Collapsed; })
				+SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(EHorizontalAlignment::HAlign_Center)
				.VAlign(EVerticalAlignment::VAlign_Center)
				.Padding(5)
				[
					SNew(STextBlock)
					.Text(FText::FromString(TEXT("Username:")))
				]
				+SHorizontalBox::Slot()
				.Padding(5)
				[
					UsernameField.ToSharedRef()
				]
			]
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				.Visibility_Lambda([this]() { return ModelInterface->GetParameters().JiraParameters.OAuthClientId.IsEmpty() ? EVisibility::All : EVisibility::Collapsed; })
				+SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(EHorizontalAlignment::HAlign_Center)
				.VAlign(EVerticalAlignment::VAlign_Center)
				.Padding(5)
				[
					SNew(STextBlock)
					.Text(FText::FromString(TEXT("Password:")))
				]
				+SHorizontalBox::Slot()
				.Padding(5)
				[
					PasswordField.ToSharedRef()
				]				
			]			
			+SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(EHorizontalAlignment::HAlign_Center)
			.VAlign(EVerticalAlignment::VAlign_Center)
			.Padding(5)
			[				
				SNew(STextBlock)
				.Visibility_Lambda([this]() { return ModelInterface->GetParameters().JiraParameters.OAuthClientId.IsEmpty() ? EVisibility::Collapsed : EVisibility::All; })
				.Text(FText::FromString(TEXT("Authentication with Atlassian OAuth is needed.")))				
			]
			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(5)
			[
				SNew(SUniformGridPanel)
				.SlotPadding(FAppStyle::GetMargin("StandardDialog.SlotPadding"))
				.MinDesiredSlotWidth(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotWidth"))
				+SUniformGridPanel::Slot(0, 0)
				[
					SNew(SButton)
					.IsEnabled_Lambda([&JiraService = JiraService] { return !JiraService.Pin()->bOngoingRequest; })
					.OnClicked(this, &SJiraWidget::Login)
					.ButtonStyle(FAppStyle::Get(), "PrimaryButton")
					[
						SNew(STextBlock)
						.MinDesiredWidth(130)
						.Justification(ETextJustify::Center)
						.Text(FText::FromString("Login"))
					]
				]
				+SUniformGridPanel::Slot(1, 0)
				[
					SNew(SButton)
					.IsEnabled_Lambda([&ModelInterface = ModelInterface] { return FModelInterface::GetInputEnabled(); })
					.OnClicked_Lambda([this]() { ParentWindow->HideWindow(); return FReply::Handled(); })
					[
						SNew(STextBlock)
						.MinDesiredWidth(130)
						.Justification(ETextJustify::Center)
						.Text(FText::FromString("Close"))
					]
				]
			]
		];
}

TSharedPtr<SWidget> SJiraWidget::BuildIssuesWidget()
{
	TSharedRef scroll = SNew(SScrollBar);
	return SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(5)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Justification(ETextJustify::Center)
				.Text(FText::FromString("Filter:"))
			]
			+ SHorizontalBox::Slot()
			.FillWidth(1)
			.Padding(5, 0, 0, 0)
			[
				SNew(SEditableTextBox)
				
				.Text_Lambda([&text = FilterText] {return FText::FromString(text); })
				.OnTextChanged_Lambda([this](FText text) { this->ApplyFilter(text.ToString()); })
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(5,0)
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "InvisibleButton")
				.IsFocusable(false)			
				.OnClicked_Lambda([this]() { bIncludeClosedJira = !bIncludeClosedJira; ApplyFilter(FilterText); return FReply::Handled(); })
				[
					SNew(STextBlock)
					.Justification(ETextJustify::Center)
					.MinDesiredWidth(60)
					.Text(FText::FromString(TEXT("Include Closed")))
				]
			]
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(SCheckBox)
				.IsChecked_Lambda([this]() { return bIncludeClosedJira ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
				.OnCheckStateChanged_Lambda([this](ECheckBoxState newState) { bIncludeClosedJira = static_cast<bool>(newState); ApplyFilter(FilterText); })
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()			
			.Padding(5,0,0,0)
			.VAlign(VAlign_Center)
			[			
				SNew(SBox)
				.MinDesiredWidth(110)
				[
					SNew(SButton)
					.HAlign(EHorizontalAlignment::HAlign_Center)
					.IsEnabled_Lambda([this]{ return FModelInterface::GetInputEnabled() && !JiraService.Pin()->bOngoingRequest; })
					.Text_Lambda([this]{ 

						FText RetText;
						if (JiraService.Pin()->bOngoingRequest)
						{
							TStringBuilder<16> Message;
						
							Message << TEXT("Refreshing");
						
							size_t currSec = FDateTime::UtcNow().GetSecond() % 4;
							for (size_t i = 0; i < currSec; ++i)
							{
								Message << TEXT('.');
							}

							RetText = FText::FromString(Message.ToString());
						}	
						else
						{
							RetText = FText::FromString("Refresh");
						}

						return RetText;
					})
				.OnClicked_Lambda([this]{ JiraService.Pin()->FetchJiraTickets(true); return FReply::Handled(); })
				]
			]
		]

		+ SVerticalBox::Slot()
		.Padding(2)
		.FillHeight(1)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.FillWidth(1)
			[
			SNew(SScrollBox)
				.Orientation(EOrientation::Orient_Horizontal)
				+ SScrollBox::Slot()
				.FillSize(1)
				[
					SAssignNew(ListView, SListView<TSharedPtr<FJiraIssue>>)
					.SelectionMode(ESelectionMode::Single)
					.ExternalScrollbar(scroll)
					.ListItemsSource(&this->JiraIssuesFiltered)
					.OnKeyDownHandler(this, &SJiraWidget::OnListKeyDown)
					.OnSelectionChanged_Lambda([this](TSharedPtr<FJiraIssue> InIssue, ESelectInfo::Type InSelectInfo)
						{
							if (InIssue.IsValid())
							{
								SelectedIssue = InIssue;
							}
							else
							{
								SelectedIssue = nullptr;
							}
						})
					.OnMouseButtonDoubleClick(this, &SJiraWidget::OnJiraDoubleClicked)
					.OnGenerateRow(this, &SJiraWidget::GenerateRow)
					.HeaderRow
					(
						SNew(SHeaderRow)
						+SHeaderRow::Column(JiraIssuesColumns::Checked)
						.DefaultLabel(NSLOCTEXT("JiraWindow", "Checked", " "))
						.ManualWidth(24)
						.SortMode(this, &SJiraWidget::GetSortMode, JiraIssuesColumns::Checked)
						.OnSort(this, &SJiraWidget::OnColumnSort)
						+ SHeaderRow::Column(JiraIssuesColumns::Issue)
						.DefaultLabel(NSLOCTEXT("JiraWindow", "Issue", "Issue"))
						.ManualWidth(96)
						.SortMode(this, &SJiraWidget::GetSortMode, JiraIssuesColumns::Issue)
						.InitialSortMode(EColumnSortMode::Ascending)
						.OnSort(this, &SJiraWidget::OnColumnSort)

						+ SHeaderRow::Column(JiraIssuesColumns::IssueType)
						.DefaultLabel(NSLOCTEXT("JiraWindow", "IssueType", "Type"))
						.ManualWidth(96)
						.SortMode(this, &SJiraWidget::GetSortMode, JiraIssuesColumns::IssueType)
						.OnSort(this, &SJiraWidget::OnColumnSort)

						+ SHeaderRow::Column(JiraIssuesColumns::Status)
						.DefaultLabel(NSLOCTEXT("JiraWindow", "Status", "Status"))
						.ManualWidth(124)
						.SortMode(this, &SJiraWidget::GetSortMode, JiraIssuesColumns::Status)
						.OnSort(this, &SJiraWidget::OnColumnSort)

						+ SHeaderRow::Column(JiraIssuesColumns::Summary)
						.DefaultLabel(NSLOCTEXT("JiraWindow", "Summary", "Summary"))
						.FillWidth(1.0)
						.SortMode(this, &SJiraWidget::GetSortMode, JiraIssuesColumns::Summary)
						.OnSort(this, &SJiraWidget::OnColumnSort)
					)
				]
			]
		+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				scroll
			]
		]
		+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Right)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Right)
				.Padding(5)
				[
					SNew(SButton)
					.IsEnabled_Lambda(FModelInterface::GetInputEnabled)
					.Text_Lambda([&issue = SelectedIssue, &targetTag = TargetTag]() -> FText {
						if (issue == nullptr || targetTag == nullptr)
						{
							return FText::FromString("Select issue");
						}

						FString IssueKey = issue->Key;
						TArray<FString> CurrentValues = targetTag->GetValues();
						if (!CurrentValues.ContainsByPredicate([IssueKey](const FString& Value) { return Value.Equals(IssueKey, ESearchCase::IgnoreCase); }))
						{
							return FText::FromString("Add issue");
						}

						return FText::FromString("Remove issue");
					})
					.OnClicked(this, &SJiraWidget::OnSelectIssueClicked)
								.IsEnabled_Lambda([&issue = SelectedIssue]() -> bool {return issue != nullptr; })
								.ButtonStyle(FSubmitToolStyle::Get(), "PrimaryButton")
				]
				+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(5)
					.HAlign(HAlign_Right)
					[
						SNew(SButton)
						.Text(FText::FromString("Close"))
						.OnClicked(this, &SJiraWidget::OnCloseClicked)
					]
		];
}

FReply SJiraWidget::Login()
{
	if (ModelInterface->GetParameters().JiraParameters.OAuthClientId.IsEmpty())
	{
		ModelInterface->SetLogin(UsernameField->GetText().ToString(), PasswordField->GetText().ToString());
	}
	else
	{
		JiraService.Pin()->ObtainOAuth([this](bool bSuccess) {
			if (bSuccess)
			{
				JiraService.Pin()->FetchJiraTickets(true);
			}
		});
	}

	return FReply::Handled();
}


FReply SJiraWidget::OnListKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (SelectedIssue != nullptr && InKeyEvent.GetKey() == EKeys::Enter)
	{
		this->ProcessIssueSelected(SelectedIssue);
		return FReply::Handled();
	}
	return SCompoundWidget::OnPreviewKeyDown(MyGeometry, InKeyEvent);
}

void SJiraWidget::JiraIssuesAvailable(bool bValidResponse)
{
	if(bValidResponse)
	{
		const TMap<FString, FJiraIssue>& Issues = this->JiraService.Pin()->GetIssues();

		this->JiraIssues.Empty(Issues.Num());

		for(TPair<FString, FJiraIssue> Issue : Issues)
		{
			this->JiraIssues.Add(MakeShared<FJiraIssue>(Issue.Value));
		}

		// Force update table
		ApplyFilter(this->FilterText);

		ChildSlot.AttachWidget(IssuesSection.ToSharedRef());
	}
	else
	{
		ChildSlot.AttachWidget(CredentialSection.ToSharedRef());
	}
}

FReply SJiraWidget::OnSelectIssueClicked()
{
	if (SelectedIssue.IsValid())
	{
		this->ProcessIssueSelected(SelectedIssue);
	}
	return FReply::Handled();
}

void SJiraWidget::OnJiraDoubleClicked(TSharedPtr<FJiraIssue> InIssue)
{
	if (InIssue.IsValid())
	{
		this->ProcessIssueSelected(InIssue);
	}
}

void SJiraWidget::ProcessIssueSelected(TSharedPtr<FJiraIssue> InIssue)
{
	if (InIssue.IsValid())
	{
		TArray<FString> CurrentValues = TargetTag->GetValues();
		if (!CurrentValues.ContainsByPredicate([InIssue](const FString& Value) { return Value.Equals(InIssue->Key, ESearchCase::IgnoreCase); }))
		{
			if(CurrentValues.Contains(TEXT("none")))
			{
				CurrentValues.Remove(TEXT("none"));
			}

			CurrentValues.Add(InIssue->Key);
		}
		else
		{
			CurrentValues.Remove(InIssue->Key);
		}

		ModelInterface->SetTagValues(*TargetTag, CurrentValues);
		ModelInterface->ValidateCLDescription();
	}
}

FReply SJiraWidget::OnCloseClicked()
{
	if (ModelInterface != nullptr)
	{
		ModelInterface->ValidateCLDescription();
	}
	ParentWindow->HideWindow();
	return FReply::Handled();
}

class SJiraIssueNode : public SMultiColumnTableRow<TSharedRef<FJiraIssue>>
{
public:
	SLATE_BEGIN_ARGS(SJiraIssueNode) {}
		SLATE_ARGUMENT(const FTag*, TargetTag)
		SLATE_ARGUMENT(SJiraWidget*, ParentWidget)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTable, TSharedPtr<FJiraIssue> InNode)
	{
		Node = InNode;
		TargetTag = InArgs._TargetTag;
		ParentWidget = InArgs._ParentWidget;

		SMultiColumnTableRow<TSharedRef<FJiraIssue>>::Construct(
			FSuperRowType::FArguments()
			.Padding(1.0f),
			InOwnerTable
		);
	}
	
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& InColumnName) override
	{
		TSharedRef<SBorder> box = SNew(SBorder).ToolTip(SNew(SToolTip).Text(FText::FromString(Node->Description)));
		
		if (InColumnName == JiraIssuesColumns::Checked)
		{
			box->SetContent(SNew(SCheckBox)
			.OnCheckStateChanged_Lambda([this](ECheckBoxState InCheckState)
			{
				ParentWidget->ProcessIssueSelected(Node);
			})
			.IsChecked_Lambda([this]
			{
				bool InTag = TargetTag->GetValues().ContainsByPredicate([this](const FString& Value) { return Value.Equals(Node->Key, ESearchCase::IgnoreCase); });

				return InTag ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
			}));
		}
		else if(InColumnName == JiraIssuesColumns::IssueType)
		{
			box->SetContent(SNew(STextBlock)
				.Text(FText::FromString(Node->IssueType)));
		}
		else if(InColumnName == JiraIssuesColumns::Issue)
		{
			box->SetToolTip(SNew(SToolTip).Text(FText::FromString(TEXT("View in Jira"))));
			box->SetContent(SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SHyperlink)
						.Text(FText::FromString(Node->Key))
						.OnNavigate_Lambda([&Link = Node->Link]() {FPlatformProcess::LaunchURL(*Link, NULL, NULL); })
						.HighlightText(FText::FromString(Node->Link))
				]);
		}
		else if(InColumnName == JiraIssuesColumns::Status)
		{
			box->SetContent(SNew(STextBlock).Text(FText::FromString(Node->Status)));
		}
		else if(InColumnName == JiraIssuesColumns::Summary)
		{
			box->SetContent(SNew(STextBlock).Text(FText::FromString(Node->Summary)));
		}

		return box;
	}

	TSharedPtr<FJiraIssue> Node;
	const FTag* TargetTag;
	SJiraWidget* ParentWidget;
};

TSharedRef<ITableRow> SJiraWidget::GenerateRow(TSharedPtr<FJiraIssue> InItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SJiraIssueNode, OwnerTable, InItem).TargetTag(TargetTag).ParentWidget(this);
}

void SJiraWidget::OnColumnSort(EColumnSortPriority::Type InSortPriority, const FName& InColumnId, EColumnSortMode::Type InSortDirection)
{
	SortMode = InSortDirection;
	SortByColumn = InColumnId;

	if(InColumnId == JiraIssuesColumns::IssueType)
	{
		if(InSortDirection == EColumnSortMode::Ascending)
		{
			Algo::SortBy(JiraIssues, &FJiraIssue::IssueType);
		}
		else
		{
			Algo::Reverse(JiraIssues);
		}
	}
	else if(InColumnId == JiraIssuesColumns::Issue)
	{

		if(InSortDirection == EColumnSortMode::Ascending)
		{
			Algo::SortBy(JiraIssues, &FJiraIssue::Key);
		}
		else
		{
			Algo::Reverse(JiraIssues);
		}
	}
	else if(InColumnId == JiraIssuesColumns::Status)
	{

		if(InSortDirection == EColumnSortMode::Ascending)
		{
			Algo::SortBy(JiraIssues, &FJiraIssue::Status);
		}
		else
		{
			Algo::Reverse(JiraIssues);
		}
	}
	else if(InColumnId == JiraIssuesColumns::Summary)
	{

		if(InSortDirection == EColumnSortMode::Ascending)
		{
			Algo::SortBy(JiraIssues, &FJiraIssue::Summary);
		}
		else
		{
			Algo::Reverse(JiraIssues);
		}
	}

	this->ApplyFilter(this->FilterText);
}

EColumnSortMode::Type SJiraWidget::GetSortMode(const FName ColumnId) const
{
	if(ColumnId == SortByColumn)
	{
		return SortMode;
	}

	return EColumnSortMode::None;
}

void SJiraWidget::ApplyFilter(const FString& InText)
{
	FilterText = InText;
	
	JiraIssuesFiltered = JiraIssues.FilterByPredicate([this](const TSharedPtr<FJiraIssue>& Issue) -> bool
	{
		if(!bIncludeClosedJira)
		{
			for(const TCHAR* status : FilteredStatus)
			{
				if(Issue->Status.Equals(status))
				{
					return false;
				}
			}
		}

		if(Issue->Status.Equals("Closed") && !bIncludeClosedJira)
		{
			return false;
		}

		return FilterText.IsEmpty() ? true : Issue->Summary.Contains(FilterText) || Issue->Key.Contains(FilterText);
	});
	

	if(ListView != nullptr)
	{
		ListView->RequestListRefresh();
	}
}
