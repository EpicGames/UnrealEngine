// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SListView.h"

class FTag;
struct FJiraIssue;
class FJiraService;
class ITableRow;
class STableViewBase;
class SEditableTextBox;


DECLARE_DELEGATE_OneParam(FOnJiraIssueSelected, FString)

class FModelInterface;

class SJiraWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SJiraWidget) {}
		SLATE_ARGUMENT(FModelInterface*, ModelInterface)
		SLATE_ARGUMENT(TSharedPtr<SWindow>, ParentWindow)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	virtual ~SJiraWidget();

	void Open(const FTag* InTargetTag);
	void ProcessIssueSelected(TSharedPtr<FJiraIssue> InIssue);

private:
	TSharedPtr<SWidget> BuildCredentialsWidget();
	TSharedPtr<SWidget> BuildIssuesWidget();

	FReply Login();
	void JiraIssuesAvailable(bool bValidResponse);

	FReply OnListKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent);

	FReply OnSelectIssueClicked(); 
	
	void OnJiraDoubleClicked(TSharedPtr<FJiraIssue> InIssue);
	FReply OnCloseClicked();

	FDelegateHandle PresubmitCallbackHandle;
	void OnSubmitCallback();

	TSharedRef<ITableRow> GenerateRow(TSharedPtr<FJiraIssue> InItem, const TSharedRef<STableViewBase>& OwnerTable);

	void OnColumnSort(EColumnSortPriority::Type InSortPriority, const FName& InColumnId, EColumnSortMode::Type InSortDirection);
	
	TSharedPtr<FJiraIssue> SelectedIssue;
	EColumnSortMode::Type GetSortMode(const FName ColumnId) const;

	FName SortByColumn;
	EColumnSortMode::Type SortMode;

	TSharedPtr<SWindow> MainWindow;

	TSharedPtr<SWindow> ParentWindow;

	TSharedPtr<SWidget> IssuesSection;
	TSharedPtr<SListView<TSharedPtr<FJiraIssue>>> ListView;

	TSharedPtr<SWidget> CredentialSection;
	TSharedPtr<SEditableTextBox> UsernameField;
	TSharedPtr<SEditableTextBox> PasswordField;

	TArray<TSharedPtr<FJiraIssue>> JiraIssues;
	TArray<TSharedPtr<FJiraIssue>> JiraIssuesFiltered;
	TWeakPtr<FJiraService> JiraService;
	FModelInterface* ModelInterface;
	const FTag* TargetTag;
	bool bIncludeClosedJira = false;

	FString FilterText;
	void ApplyFilter(const FString& InText);
};