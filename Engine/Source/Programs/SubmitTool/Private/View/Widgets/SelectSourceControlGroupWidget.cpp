// Copyright Epic Games, Inc. All Rights Reserved.

#include "SelectSourceControlGroupWidget.h"

#include "Models/SubmitToolUserPrefs.h"
#include "View/SubmitToolStyle.h"
#include "Widgets/SWindow.h"
#include "Widgets/Images/SThrobber.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/SToolTip.h"
#include "Framework/Application/SlateApplication.h"

namespace P4GroupColumns
{
	static const FName Recent("Recent");
	static const FName GroupName("Group Name");
};

void SSelectSourceControlGroupWidget::Construct(const FArguments& InArgs)
{
	ModelInterface = InArgs._ModelInterface.Get();
	TargetTag = InArgs._Tag.Get();

	PresubmitCallbackHandle = ModelInterface->PrepareSubmitCallBack.AddSP(this, &SSelectSourceControlGroupWidget::OnSubmitCallback);

	ChildSlot
		[
			SNew(SButton)
				.OnClicked_Lambda([this]()
					{
						if (this->DialogWindow == nullptr) 
						{ 
							return OpenDialog(); 
						}

						this->DialogWindow->ShowWindow();

						return FReply::Handled();
					})
				.HAlign(HAlign_Center)
						.VAlign(VAlign_Center)
						[
							SNew(STextBlock)
								.Justification(ETextJustify::Center)
								.Text(InArgs._ButtonText)
								.Font(FAppStyle::Get().GetFontStyle("SmallFont"))
						]
		];

	// Start the async group process
	const FOnGroupsGet::FDelegate Callback = FOnGroupsGet::FDelegate::CreateSP(this, &SSelectSourceControlGroupWidget::OnGetGroups);
	ModelInterface->GetGroups(Callback);
}

SSelectSourceControlGroupWidget::~SSelectSourceControlGroupWidget()
{
	if (this->PresubmitCallbackHandle.IsValid())
	{
		this->ModelInterface->PrepareSubmitCallBack.Remove(this->PresubmitCallbackHandle);
	}
}

void SSelectSourceControlGroupWidget::OnSubmitCallback()
{
	if (this->DialogWindow != nullptr && this->DialogWindow->IsVisible())
	{
		this->DialogWindow->RequestDestroyWindow();
		this->DialogWindow = nullptr;
	}
}

FReply SSelectSourceControlGroupWidget::OpenDialog()
{
	SelectedGroup = nullptr;

	TSharedPtr<SVerticalBox> Contents;
	TSharedPtr<SEditableTextBox> FilterTextBox;

	DialogWindow = SNew(SWindow)
		.Title_Lambda([this] { return FText::FromString(TargetTag ? FString::Format(TEXT("Select p4 groups for Tag '{0}'"), { TargetTag->Definition.TagLabel }) : TEXT("Invalid")); })
		.SizingRule(ESizingRule::UserSized)
		.SupportsMaximize(false)
		.SupportsMinimize(false)
		.MinWidth(850)
		.MinHeight(400)
		[
			SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				.VAlign(VAlign_Fill)
				[
					SAssignNew(Contents, SVerticalBox)
				]
		];

	DialogWindow->SetRequestDestroyWindowOverride(FRequestDestroyWindowOverride::CreateLambda([this](const TSharedRef<SWindow>& Window) { Window->HideWindow(); }));

	TSharedRef<SHorizontalBox> SearchBox = SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
				.Justification(ETextJustify::Center)
				.Text(FText::FromString("Filter:"))
		]
		+ SHorizontalBox::Slot()
		.Padding(5, 0, 0, 0)
		[
			SAssignNew(FilterTextBox, SEditableTextBox)
				.Text_Lambda([&text = FilterText] {return FText::FromString(text); })
				.OnTextChanged_Lambda([this](FText text) { this->OnFilterTextChanged(text.ToString()); })
		]
		+SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(5, 0, 0, 0)
		.VAlign(VAlign_Center)
		[
			SNew(SCheckBox)
			.IsChecked_Lambda([]() { return FSubmitToolUserPrefs::Get()->bAppendAtForP4Groups ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
			.OnCheckStateChanged_Lambda([](ECheckBoxState InNewState) { FSubmitToolUserPrefs::Get()->bAppendAtForP4Groups = InNewState == ECheckBoxState::Checked; })
		]
		+SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "InvisibleButton")
			.IsFocusable(false)			

			.OnClicked_Lambda([]() { FSubmitToolUserPrefs::Get()->bAppendAtForP4Groups = !FSubmitToolUserPrefs::Get()->bAppendAtForP4Groups; return FReply::Handled(); })
			[
				SNew(STextBlock)
				.Justification(ETextJustify::Left)
				.MinDesiredWidth(60)
				.Text(FText::FromString(TEXT("Append @")))
				.ToolTipText(FText::FromString(TEXT("Appends an @ at the beginning of the group name so that p4 notifies when requesting a review")))
			]
		];

	Contents.Get()->AddSlot()
		.AutoHeight()
		.Padding(5)
		[
			SearchBox
		];

	// List view
	TSharedRef scroll = SNew(SScrollBar);
	Contents.Get()->AddSlot()
		.FillHeight(1)
		[
			SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.FillWidth(1)
				[
					SNew(SOverlay)
						+ SOverlay::Slot()
						.VAlign(VAlign_Fill)
						.HAlign(HAlign_Fill)
						[
							SNew(SScrollBox)
								.Orientation(EOrientation::Orient_Horizontal)
								+ SScrollBox::Slot()
								.FillSize(1)
								[
									SAssignNew(GroupListView, SListView<TSharedPtr<FString>>)
										.SelectionMode(ESelectionMode::Single)
										.ListItemsSource(&FilteredGroups)
										.ExternalScrollbar(scroll)
										.OnMouseButtonDoubleClick(this, &SSelectSourceControlGroupWidget::OnSelectGroupDoubleClicked)
										.OnKeyDownHandler(this, &SSelectSourceControlGroupWidget::OnListKeyDown)
										.OnGenerateRow(this, &SSelectSourceControlGroupWidget::GenerateRow)
										.HeaderRow
										(
											SNew(SHeaderRow)

											+ SHeaderRow::Column(P4GroupColumns::Recent)
											.DefaultLabel(FText::FromString(""))
											.ManualWidth(20)

											+ SHeaderRow::Column(P4GroupColumns::GroupName)
											.DefaultLabel(FText::FromString("Group Name"))
											.FillWidth(1.0)
											.SortMode(this, &SSelectSourceControlGroupWidget::GetSortMode, P4GroupColumns::GroupName)
											.InitialSortMode(EColumnSortMode::Ascending)
											.OnSort(this, &SSelectSourceControlGroupWidget::OnColumnSort)
										)
										.OnSelectionChanged_Lambda([this](TSharedPtr<FString> InGroup, ESelectInfo::Type InSelectInfo)
											{
												if (InGroup.IsValid())
												{
													SelectedGroup = InGroup;
												}
												else
												{
													SelectedGroup = nullptr;
												}
											})
								]
						]
						+ SOverlay::Slot()
						.VAlign(VAlign_Fill)
						.HAlign(HAlign_Center)
						[
							SNew(SThrobber)
								.Visibility_Lambda([this]() { return bIsLoadingGroups ? EVisibility::All : EVisibility::Hidden; })
						]
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					scroll
				]
		];

	// Buttons
	Contents.Get()->AddSlot()
		.AutoHeight()
		.HAlign(HAlign_Right)
		[
			SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Right)
				.Padding(5)
				[
					SNew(SButton)
						.IsEnabled_Lambda(FModelInterface::GetInputEnabled)
						.Text_Lambda([&selectedGroup = SelectedGroup, &targetTag = TargetTag]() -> FText {
						if (selectedGroup == nullptr || targetTag == nullptr)
						{
							return FText::FromString("Select group");
						}

						FString GroupName = *selectedGroup;
						TArray<FString> CurrentValues = targetTag->GetValues();
						if (!CurrentValues.ContainsByPredicate([GroupName](const FString& Value) { return Value.TrimChar('@').Equals(GroupName.TrimChar('@'), ESearchCase::IgnoreCase); }))
						{
							return FText::FromString("Add group");
						}

						return FText::FromString("Remove group");
							})
						.OnClicked(this, &SSelectSourceControlGroupWidget::OnSelectGroupClicked)
								.IsEnabled_Lambda([&selectedGroup = SelectedGroup]() -> bool {return selectedGroup != nullptr; })
								.ButtonStyle(FSubmitToolStyle::Get(), "PrimaryButton")
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(5)
				.HAlign(HAlign_Right)
				[
					SNew(SButton)
						.Text(FText::FromString("Close"))
						.OnClicked(this, &SSelectSourceControlGroupWidget::OnCancelClicked)
				]
		];

	DialogWindow->SetWidgetToFocusOnActivate(FilterTextBox);

	// Force update the available group list
	OnFilterTextChanged(this->FilterText);

	TSharedRef<SWindow> MainWindow = FSlateApplication::Get().GetActiveTopLevelWindow().ToSharedRef();

	FSlateApplication::Get().AddWindowAsNativeChild(DialogWindow.ToSharedRef(), MainWindow, true);

	DialogWindow->ShowWindow();

	return FReply::Handled();
}

void SSelectSourceControlGroupWidget::OnFilterTextChanged(const FString& InText)
{
	FilteredGroups = AllGroups;

	FilterText = InText;
	if (!FilterText.IsEmpty())
	{
		FilteredGroups = FilteredGroups.FilterByPredicate([&](const TSharedPtr<FString>& Group) -> bool
			{
				return Group->Contains(FilterText);
			});
	}

	RecentGroups = ModelInterface->GetRecentGroups();
	for (int i = RecentGroups.Num() - 1; i >= 0; --i)
	{
		TSharedPtr<FString>& RecentGroup = RecentGroups[i];
		if (FilteredGroups.Contains(RecentGroup))
		{
			FilteredGroups.Remove(RecentGroup);
			FilteredGroups.EmplaceAt(0, RecentGroup);
		}
	}

	if (GroupListView != nullptr)
	{
		GroupListView->ClearSelection();
		GroupListView->RebuildList();
	}
}

FReply SSelectSourceControlGroupWidget::OnListKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (SelectedGroup != nullptr && InKeyEvent.GetKey() == EKeys::Enter)
	{
		OnSelectGroupClicked();
		return FReply::Handled();
	}
	return SCompoundWidget::OnPreviewKeyDown(MyGeometry, InKeyEvent);
}

FReply SSelectSourceControlGroupWidget::OnSelectGroupClicked()
{
	if (SelectedGroup != nullptr)
	{
		ProcessGroupSelected(SelectedGroup);
	}

	return FReply::Handled();
}

void SSelectSourceControlGroupWidget::OnSelectGroupDoubleClicked(TSharedPtr<FString> InGroup)
{
	if (InGroup != nullptr)
	{
		ProcessGroupSelected(InGroup);
	}
}

void SSelectSourceControlGroupWidget::OnGetGroups(TArray<TSharedPtr<FString>>& Groups)
{
	AllGroups = Groups;
	bIsLoadingGroups = false;

	// Force update the available group list
	OnFilterTextChanged(this->FilterText);
}

FReply SSelectSourceControlGroupWidget::OnCancelClicked()
{
	DialogWindow->RequestDestroyWindow();
	this->DialogWindow = nullptr;

	return FReply::Handled();
}

void SSelectSourceControlGroupWidget::ProcessGroupSelected(TSharedPtr<FString> InGroup)
{
	const FString GroupName = FSubmitToolUserPrefs::Get()->bAppendAtForP4Groups ? TEXT("@") + *InGroup : *InGroup;
	TArray<FString> CurrentValues = TargetTag->GetValues();
	if (!CurrentValues.ContainsByPredicate([GroupName](const FString& Value) { return Value.TrimChar('@').Equals(GroupName.TrimChar('@'), ESearchCase::IgnoreCase); }))
	{
		CurrentValues.Add(GroupName);
	}
	else
	{
		CurrentValues.RemoveAll([GroupName](const FString& Value) { return Value.TrimChar('@').Equals(GroupName.TrimChar('@'), ESearchCase::IgnoreCase); });
	}

	ModelInterface->SetTagValues(*TargetTag, CurrentValues);
	ModelInterface->ValidateCLDescription();

	ModelInterface->AddRecentGroup(InGroup);

	// Force update the available group list
	OnFilterTextChanged(this->FilterText);
}


class SPerforceGroupNode : public SMultiColumnTableRow<TSharedRef<FString>>
{
public:
	SLATE_BEGIN_ARGS(SPerforceGroupNode) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTable, TSharedPtr<FString> InNode, bool bInIsRecentGroup)
	{
		Node = InNode;
		bIsRecentGroup = bInIsRecentGroup;

		SMultiColumnTableRow<TSharedRef<FString>>::Construct(
			FSuperRowType::FArguments()
			.Padding(1.0f),
			InOwnerTable
		);
	}

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& InColumnName) override
	{
		FString ToolTip = *Node;
		TSharedRef<SBorder> box = SNew(SBorder).ToolTip(SNew(SToolTip).Text(FText::FromString(ToolTip)));

		if (InColumnName == P4GroupColumns::Recent)
		{
			if (bIsRecentGroup)
			{
				box->SetContent(SNew(SImage)
					.Image(FSubmitToolStyle::Get().GetBrush("AppIcon.Star16")));
			}
		}
		else if (InColumnName == P4GroupColumns::GroupName)
		{
			box->SetContent(SNew(STextBlock)
				.Text(FText::FromString(*Node)));
		}

		return box;
	}

	TSharedPtr<FString> Node;
	bool bIsRecentGroup;
};

TSharedRef<ITableRow> SSelectSourceControlGroupWidget::GenerateRow(TSharedPtr<FString> InGroup, const TSharedRef<STableViewBase>& InTableView)
{
	return SNew(SPerforceGroupNode, InTableView, InGroup, RecentGroups.Contains(InGroup));
}

void SSelectSourceControlGroupWidget::OnColumnSort(EColumnSortPriority::Type InSortPriority, const FName& InColumnId, EColumnSortMode::Type InSortDirection)
{
	SortMode = InSortDirection;
	SortByColumn = InColumnId;
	Algo::Reverse(AllGroups);

	this->OnFilterTextChanged(this->FilterText);
}

EColumnSortMode::Type SSelectSourceControlGroupWidget::GetSortMode(const FName ColumnId) const
{
	if (ColumnId == SortByColumn)
	{
		return SortMode;
	}

	return EColumnSortMode::None;
}