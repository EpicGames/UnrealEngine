// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Models/ModelInterface.h"
#include "Widgets/Views/SListView.h"

class SSelectSourceControlGroupWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SSelectSourceControlGroupWidget) {}
		SLATE_ATTRIBUTE(FText, ButtonText)
		SLATE_ATTRIBUTE(FModelInterface*, ModelInterface)
		SLATE_ATTRIBUTE(const FTag*, Tag)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	virtual ~SSelectSourceControlGroupWidget();


	FReply OnListKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent);

private:
	FReply OpenDialog();
	TSharedPtr<SWindow> DialogWindow;
	FModelInterface* ModelInterface;
	const FTag* TargetTag;

	FDelegateHandle PresubmitCallbackHandle;
	void OnSubmitCallback();

	TSharedPtr<SListView<TSharedPtr<FString>>> GroupListView;
	TSharedRef<ITableRow> GenerateRow(TSharedPtr<FString> InGroup, const TSharedRef<STableViewBase>& InTableView);

	FString FilterText;
	void OnFilterTextChanged(const FString& InText);
	TArray<TSharedPtr<FString>> AllGroups;
	TArray<TSharedPtr<FString>> FilteredGroups;
	TArray<TSharedPtr<FString>> RecentGroups;

	FReply OnSelectGroupClicked();
	TSharedPtr<FString> SelectedGroup;

	void OnSelectGroupDoubleClicked(TSharedPtr<FString> InGroup);

	void OnGetGroups(TArray<TSharedPtr<FString>>& Groups);

	bool bIsLoadingGroups = true;

	FReply OnCancelClicked();

	void ProcessGroupSelected(TSharedPtr<FString> InGroup);

	FName SortByColumn;
	EColumnSortMode::Type SortMode;
	void OnColumnSort(EColumnSortPriority::Type InSortPriority, const FName& InColumnId, EColumnSortMode::Type InSortDirection);
	EColumnSortMode::Type GetSortMode(const FName ColumnId) const;
};