// Copyright Epic Games, Inc. All Rights Reserved.

#include "SelectSourceControlUserWidget.h"

#include "Models/SubmitToolUserPrefs.h"
#include "View/SubmitToolStyle.h"
#include "Widgets/SWindow.h"
#include "Widgets/Images/SThrobber.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/SToolTip.h"
#include "Framework/Application/SlateApplication.h"
#include "SubmitToolUtils.h"

namespace P4UserColumns
{
	static const FName Recent("Recent");
	static const FName Name("Name");
	static const FName Username("Username");
	static const FName Email("Email");
};

void SSelectSourceControlUserWidget::Construct(const FArguments& InArgs)
{
	ModelInterface = InArgs._ModelInterface.Get();
	TargetTag = InArgs._Tag.Get();
	TargetText = InArgs._TargetText.Get();
	TargetName = InArgs._TargetName.Get();

	PresubmitCallbackHandle = ModelInterface->PrepareSubmitCallBack.AddSP(this, &SSelectSourceControlUserWidget::OnSubmitCallback);

	ChildSlot
		[
			SNew(SButton)
			.OnClicked_Lambda([this]()
			{
				if(this->DialogWindow == nullptr)
				{
					return OpenDialog();
				}

				this->DialogWindow->ShowWindow();

				return FReply::Handled();
			})
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Justification(ETextJustify::Center)
				.Text(InArgs._ButtonText)
				.Font(FAppStyle::Get().GetFontStyle("SmallFont"))
			]
		];

	// Start the async user process
	const FOnUsersGet::FDelegate Callback = FOnUsersGet::FDelegate::CreateSP(this, &SSelectSourceControlUserWidget::OnGetUsers);
	ModelInterface->GetUsers(Callback);
}

SSelectSourceControlUserWidget::~SSelectSourceControlUserWidget()
{
	if(this->PresubmitCallbackHandle.IsValid())
	{
		this->ModelInterface->PrepareSubmitCallBack.Remove(this->PresubmitCallbackHandle);
	}
}

void SSelectSourceControlUserWidget::OnSubmitCallback()
{
	if(this->DialogWindow != nullptr && this->DialogWindow->IsVisible())
	{
		this->DialogWindow->RequestDestroyWindow();
		this->DialogWindow = nullptr;
	}
}

FReply SSelectSourceControlUserWidget::OpenDialog()
{
	SelectedUser = nullptr;

	TSharedPtr<SVerticalBox> Contents;
	TSharedPtr<SEditableTextBox> FilterTextBox;

	DialogWindow = SNew(SWindow)
		.Title_Lambda([this] { return FText::FromString(TargetTag ? FString::Format(TEXT("Select p4 users for Tag '{0}'"), { TargetTag->Definition.TagLabel }) : FString::Format(TEXT("Select User for {0}"), { TargetName })); })
		.SizingRule(ESizingRule::UserSized)
		.SupportsMaximize(false)
		.SupportsMinimize(false)
		.MinWidth(850)
		.MinHeight(400)
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			.VAlign(VAlign_Fill)
			[
				SAssignNew(Contents, SVerticalBox)
			]
		];

	DialogWindow->SetRequestDestroyWindowOverride(FRequestDestroyWindowOverride::CreateLambda([this](const TSharedRef<SWindow>& Window) { Window->HideWindow(); }));

	TSharedRef<SHorizontalBox> SearchBox = SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Justification(ETextJustify::Center)
			.Text(FText::FromString("Filter:"))
		]
		+SHorizontalBox::Slot()
		.Padding(5, 0, 0, 0)
		[
			SAssignNew(FilterTextBox, SEditableTextBox)
			.Text_Lambda([&text = FilterText] {return FText::FromString(text); })
			.OnTextChanged_Lambda([this](FText text) { this->OnFilterTextChanged(text.ToString()); })
		]
		+SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(5, 0, 0, 0)
		.VAlign(VAlign_Center)
		[
			SNew(SCheckBox)
			.Visibility_Lambda([this](){ return TargetTag->Definition.InputSubType == TEXT("SwarmApproved") ? EVisibility::Collapsed : EVisibility::All ;})
			.IsChecked_Lambda([]() { return FSubmitToolUserPrefs::Get()->bAppendAtForP4Users ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
			.OnCheckStateChanged_Lambda([](ECheckBoxState InNewState) { FSubmitToolUserPrefs::Get()->bAppendAtForP4Users = InNewState == ECheckBoxState::Checked; })
		]
		+SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "InvisibleButton")
			.Visibility_Lambda([this]() { return TargetTag->Definition.InputSubType == TEXT("SwarmApproved") ? EVisibility::Collapsed : EVisibility::All; })
			.IsFocusable(false)		
			.OnClicked_Lambda([]() { FSubmitToolUserPrefs::Get()->bAppendAtForP4Users = !FSubmitToolUserPrefs::Get()->bAppendAtForP4Users; return FReply::Handled(); })
			[
				SNew(STextBlock)
				.Justification(ETextJustify::Left)
				.MinDesiredWidth(60)
				.Text(FText::FromString(TEXT("Append @")))
				.ToolTipText(FText::FromString(TEXT("Appends an @ at the beginning of the user name so that p4 notifies when requesting a review")))
			]
		];

	if(TargetTag && TargetTag->Definition.Filters.Num() != 0)
	{
		SearchBox->AddSlot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(5, 0)
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "InvisibleButton")
				.IsFocusable(false)
				.OnClicked_Lambda([this]() { bFilterUsers = !bFilterUsers; OnFilterTextChanged(FilterText); return FReply::Handled(); })
				[
					SNew(STextBlock)
					.Justification(ETextJustify::Center)
					.MinDesiredWidth(60)
					.Text(FText::FromString(TEXT("Exclude Externals")))
				]
			];

		SearchBox->AddSlot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(SCheckBox)
				.IsChecked_Lambda([this]() { return bFilterUsers ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
				.OnCheckStateChanged_Lambda([this](ECheckBoxState newState) { bFilterUsers = static_cast<bool>(newState); OnFilterTextChanged(FilterText); })
			];

	}
	else
	{
		bFilterUsers = false;
	}


	Contents.Get()->AddSlot()
		.AutoHeight()
		.Padding(5)
		[
			SearchBox
		];

	// List view
	TSharedRef scroll = SNew(SScrollBar);
	Contents.Get()->AddSlot()
	.FillHeight(1)
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.FillWidth(1)
		[
			SNew(SOverlay)
			+ SOverlay::Slot()
			.VAlign(VAlign_Fill)
			.HAlign(HAlign_Fill)
			[
				SNew(SScrollBox)
				.Orientation(EOrientation::Orient_Horizontal)
				+ SScrollBox::Slot()
				.FillSize(1)
				[
					SAssignNew(UserListView, SListView<TSharedPtr<FUserData>>)
					.SelectionMode(ESelectionMode::Single)
					.ListItemsSource(&FilteredUsers)
					.ExternalScrollbar(scroll)
					.OnMouseButtonDoubleClick(this, &SSelectSourceControlUserWidget::OnSelectUserDoubleClicked)
					.OnKeyDownHandler(this, &SSelectSourceControlUserWidget::OnListKeyDown)
					.OnGenerateRow(this, &SSelectSourceControlUserWidget::GenerateRow)
					.HeaderRow
					(
						SNew(SHeaderRow)

						+ SHeaderRow::Column(P4UserColumns::Recent)
						.DefaultLabel(FText::FromString(""))
						.ManualWidth(20)

						+ SHeaderRow::Column(P4UserColumns::Username)
						.DefaultLabel(FText::FromString("Username"))
						.ManualWidth(124)
						.SortMode(this, &SSelectSourceControlUserWidget::GetSortMode, P4UserColumns::Username)
						.InitialSortMode(EColumnSortMode::Ascending)
						.OnSort(this, &SSelectSourceControlUserWidget::OnColumnSort)

						+ SHeaderRow::Column(P4UserColumns::Email)
						.DefaultLabel(FText::FromString("Email"))
						.ManualWidth(224)
						.SortMode(this, &SSelectSourceControlUserWidget::GetSortMode, P4UserColumns::Email)
						.InitialSortMode(EColumnSortMode::Ascending)
						.OnSort(this, &SSelectSourceControlUserWidget::OnColumnSort)

						+ SHeaderRow::Column(P4UserColumns::Name)
						.DefaultLabel(FText::FromString("Name"))
						.FillWidth(1.0)
						.SortMode(this, &SSelectSourceControlUserWidget::GetSortMode, P4UserColumns::Name)
						.InitialSortMode(EColumnSortMode::Ascending)
						.OnSort(this, &SSelectSourceControlUserWidget::OnColumnSort)
					)
					.OnSelectionChanged_Lambda([this](TSharedPtr<FUserData> InUser, ESelectInfo::Type InSelectInfo)
						{
							if (InUser.IsValid())
							{
								SelectedUser = InUser;
							}
							else
							{
								SelectedUser = nullptr;
							}
						})
					]
				]
			+ SOverlay::Slot()
			.VAlign(VAlign_Fill)
			.HAlign(HAlign_Center)
			[
				SNew(SThrobber)
				.Visibility_Lambda([this]() { return bIsLoadingUsers ? EVisibility::All : EVisibility::Hidden; })
			]
		]
	+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			scroll
		]
	];
	
	// Buttons
	Contents.Get()->AddSlot()
	.AutoHeight()
	.HAlign(HAlign_Right)
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Right)
		.Padding(5)
		[
			SNew(SButton)
			.IsEnabled_Lambda(FModelInterface::GetInputEnabled)
			.Text_Lambda([this]() -> FText {
				if (SelectedUser == nullptr)
				{
					return FText::FromString("Select user");
				}

				FString UserName = SelectedUser->Username;
				bool bExists = false;
				if(TargetTag)
				{
					TArray<FString> CurrentValues = TargetTag->GetValues();
					bExists = CurrentValues.ContainsByPredicate([UserName](const FString& Value) { return Value.TrimChar('@').Equals(UserName.TrimChar('@'), ESearchCase::IgnoreCase); });
				}
				else
				{
					bExists = TargetText->Find(UserName, ESearchCase::IgnoreCase) != INDEX_NONE;
				}

				return bExists ? FText::FromString("Remove user") : FText::FromString("Add user");
			})
			.OnClicked(this, &SSelectSourceControlUserWidget::OnSelectUserClicked)
			.IsEnabled_Lambda([&selectedUser = SelectedUser]() -> bool {return selectedUser != nullptr; })
			.ButtonStyle(FSubmitToolStyle::Get(), "PrimaryButton")
		]
		+SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(5)
		.HAlign(HAlign_Right)
		[
			SNew(SButton)
			.Text(FText::FromString("Close"))
			.OnClicked(this, &SSelectSourceControlUserWidget::OnCancelClicked)
		]
	];

	DialogWindow->SetWidgetToFocusOnActivate(FilterTextBox);

	// Force update the available user list
	OnFilterTextChanged(this->FilterText);

	TSharedRef<SWindow> MainWindow = FSlateApplication::Get().GetActiveTopLevelWindow().ToSharedRef();

	FSlateApplication::Get().AddWindowAsNativeChild(DialogWindow.ToSharedRef(), MainWindow, true);

	DialogWindow->ShowWindow();

	FSubmitToolUtils::EnsureWindowIsInView(DialogWindow.ToSharedRef(), true);

	return FReply::Handled();
}

void SSelectSourceControlUserWidget::OnFilterTextChanged(const FString& InText)
{
	if(bFilterUsers && TargetTag)
	{
		FilteredUsers.Reset(AllUsers.Num());

		for(TSharedPtr<FUserData> User : AllUsers)
		{
			for(FString Filter : TargetTag->Definition.Filters)
			{
				if(User->Email.Contains(Filter, ESearchCase::IgnoreCase))
				{
					FilteredUsers.Add(User);
					break;
				}
			}
		}
	}
	else
	{
		FilteredUsers = AllUsers;
	}

	FilterText = InText;
	if(!FilterText.IsEmpty())
	{
		FilteredUsers = FilteredUsers.FilterByPredicate([&](const TSharedPtr<FUserData>& User) -> bool
			{
				return User->Contains(FilterText);
			});
	}

	RecentUsers = ModelInterface->GetRecentUsers();
	for(int i = RecentUsers.Num() - 1; i >= 0; --i)
	{
		TSharedPtr<FUserData>& RecentUser = RecentUsers[i];
		if(FilteredUsers.Contains(RecentUser))
		{
			FilteredUsers.Remove(RecentUser);
			FilteredUsers.EmplaceAt(0, RecentUser);
		}
	}

	if(UserListView != nullptr)
	{
		UserListView->ClearSelection();
		UserListView->RebuildList();
	}
}

FReply SSelectSourceControlUserWidget::OnListKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if(SelectedUser != nullptr && InKeyEvent.GetKey() == EKeys::Enter)
	{
		OnSelectUserClicked();
		return FReply::Handled();
	}
	return SCompoundWidget::OnPreviewKeyDown(MyGeometry, InKeyEvent);
}

FReply SSelectSourceControlUserWidget::OnSelectUserClicked()
{
	if(SelectedUser != nullptr)
	{
		ProcessUserSelected(SelectedUser);
	}

	return FReply::Handled();
}

void SSelectSourceControlUserWidget::OnSelectUserDoubleClicked(TSharedPtr<FUserData> InUser)
{
	if(InUser != nullptr)
	{
		ProcessUserSelected(InUser);
	}
}

void SSelectSourceControlUserWidget::OnGetUsers(TArray<TSharedPtr<FUserData>>& Users)
{
	AllUsers = Users;
	bIsLoadingUsers = false;

	// Force update the available user list
	OnFilterTextChanged(this->FilterText);
}

FReply SSelectSourceControlUserWidget::OnCancelClicked()
{
	DialogWindow->RequestDestroyWindow();
	this->DialogWindow = nullptr;

	return FReply::Handled();
}

void SSelectSourceControlUserWidget::ProcessUserSelected(TSharedPtr<FUserData> InUser)
{
	const FString Username = TargetTag->Definition.InputSubType != TEXT("SwarmApproved") && FSubmitToolUserPrefs::Get()->bAppendAtForP4Users ? TEXT("@") + InUser->Username : InUser->Username;

	TArray<FString> CurrentValues;
	if(TargetTag)
	{
		CurrentValues = TargetTag->GetValues();
	}
	else
	{
		const TArray<const TCHAR*> Delimiters =
		{
			TEXT(","),
			TEXT(" ")
		};

		TargetText->ParseIntoArray(CurrentValues, Delimiters.GetData(), 2);
	}

	if(!CurrentValues.ContainsByPredicate([Username](const FString& Value) { return Value.TrimChar('@').Equals(Username.TrimChar('@'), ESearchCase::IgnoreCase); }))
	{
		CurrentValues.Add(Username);
		ModelInterface->AddRecentUser(InUser);
	}
	else
	{
		CurrentValues.RemoveAll([Username](const FString& Value) { return Value.TrimChar('@').Equals(Username.TrimChar('@'), ESearchCase::IgnoreCase); });
	}

	if(TargetTag)
	{
		ModelInterface->SetTagValues(*TargetTag, CurrentValues);
		ModelInterface->ValidateCLDescription();
	}
	else
	{
		*TargetText = FString::Join(CurrentValues, TEXT(", "));
	}

	// Force update the available user list
	OnFilterTextChanged(this->FilterText);
}


class SPerforceUserNode : public SMultiColumnTableRow<TSharedRef<FUserData>>
{
public:
	SLATE_BEGIN_ARGS(SPerforceUserNode)
	{}
	SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTable, TSharedPtr<FUserData> InNode, bool bInIsRecentUser)
	{
		Node = InNode;
		bIsRecentUser = bInIsRecentUser;

		SMultiColumnTableRow<TSharedRef<FUserData>>::Construct(
			FSuperRowType::FArguments()
			.Padding(1.0f),
			InOwnerTable
		);
	}

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& InColumnName) override
	{
		FString ToolTip = FString::Format(TEXT("{0}\n{1}\n{2}"), { Node->Name, Node->Username, Node->Email });
		TSharedRef<SBorder> box = SNew(SBorder).ToolTip(SNew(SToolTip).Text(FText::FromString(ToolTip)));

		if(InColumnName == P4UserColumns::Recent)
		{
			if(bIsRecentUser)
			{
				box->SetContent(SNew(SImage)
					.Image(FSubmitToolStyle::Get().GetBrush("AppIcon.Star16")));
			}
		}
		else if(InColumnName == P4UserColumns::Name)
		{
			box->SetContent(SNew(STextBlock)
				.Text(FText::FromString(Node->Name)));
		}
		else if(InColumnName == P4UserColumns::Username)
		{
			box->SetContent(SNew(STextBlock)
				.Text(FText::FromString(Node->Username)));
		}
		else if(InColumnName == P4UserColumns::Email)
		{
			box->SetContent(SNew(STextBlock)
				.Text(FText::FromString(Node->Email)));
		}

		return box;
	}

	TSharedPtr<FUserData> Node;
	bool bIsRecentUser;
};

TSharedRef<ITableRow> SSelectSourceControlUserWidget::GenerateRow(TSharedPtr<FUserData> InUser, const TSharedRef<STableViewBase>& InTableView)
{
	return SNew(SPerforceUserNode, InTableView, InUser, RecentUsers.Contains(InUser));
}

void SSelectSourceControlUserWidget::OnColumnSort(EColumnSortPriority::Type InSortPriority, const FName& InColumnId, EColumnSortMode::Type InSortDirection)
{
	SortMode = InSortDirection;
	SortByColumn = InColumnId;

	if(InColumnId == P4UserColumns::Name)
	{
		if(InSortDirection == EColumnSortMode::Ascending)
		{
			Algo::SortBy(AllUsers, &FUserData::Name);
		}
		else
		{
			Algo::Reverse(AllUsers);
		}
	}
	else if(InColumnId == P4UserColumns::Username)
	{
		if(InSortDirection == EColumnSortMode::Ascending)
		{
			Algo::SortBy(AllUsers, &FUserData::Username);
		}
		else
		{
			Algo::Reverse(AllUsers);
		}
	}
	else if(InColumnId == P4UserColumns::Email)
	{
		if(InSortDirection == EColumnSortMode::Ascending)
		{
			Algo::SortBy(AllUsers, &FUserData::Email);
		}
		else
		{
			Algo::Reverse(AllUsers);
		}
	}

	this->OnFilterTextChanged(this->FilterText);
}

EColumnSortMode::Type SSelectSourceControlUserWidget::GetSortMode(const FName ColumnId) const
{
	if(ColumnId == SortByColumn)
	{
		return SortMode;
	}

	return EColumnSortMode::None;
}