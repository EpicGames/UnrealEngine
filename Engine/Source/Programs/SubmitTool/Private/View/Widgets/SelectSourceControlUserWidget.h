// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Models/ModelInterface.h"
#include "Widgets/Views/SListView.h"

class SSelectSourceControlUserWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SSelectSourceControlUserWidget) {}
		SLATE_ATTRIBUTE(FText, ButtonText)
		SLATE_ATTRIBUTE(FModelInterface*, ModelInterface)
		SLATE_ATTRIBUTE(FString*, TargetText)
		SLATE_ATTRIBUTE(FString, TargetName)
		SLATE_ATTRIBUTE(const FTag*, Tag)
	SLATE_END_ARGS()
	
	void Construct(const FArguments& InArgs);

	virtual ~SSelectSourceControlUserWidget();


	FReply OnListKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent);

private:
	FReply OpenDialog();
	TSharedPtr<SWindow> DialogWindow;
	FModelInterface* ModelInterface;
	const FTag* TargetTag;
	FString* TargetText;
	FString TargetName;

	FDelegateHandle PresubmitCallbackHandle;
	void OnSubmitCallback();

	TSharedPtr<SListView<TSharedPtr<FUserData>>> UserListView;
	TSharedRef<ITableRow> GenerateRow(TSharedPtr<FUserData> InUser, const TSharedRef<STableViewBase>& InTableView);

	FString FilterText;
	void OnFilterTextChanged(const FString& InText);
	TArray<TSharedPtr<FUserData>> AllUsers;
	TArray<TSharedPtr<FUserData>> FilteredUsers;
	TArray<TSharedPtr<FUserData>> RecentUsers;

	FReply OnSelectUserClicked();
	TSharedPtr<FUserData> SelectedUser;

	void OnSelectUserDoubleClicked(TSharedPtr<FUserData> InUser);

	void OnGetUsers(TArray<TSharedPtr<FUserData>>& Users);
	
	bool bIsLoadingUsers = true;
	bool bFilterUsers = true;

	FReply OnCancelClicked();

	void ProcessUserSelected(TSharedPtr<FUserData> InUser);

	FName SortByColumn;
	EColumnSortMode::Type SortMode;
	void OnColumnSort(EColumnSortPriority::Type InSortPriority, const FName& InColumnId, EColumnSortMode::Type InSortDirection);
	EColumnSortMode::Type GetSortMode(const FName ColumnId) const;
};