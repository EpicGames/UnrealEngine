// Copyright Epic Games, Inc. All Rights Reserved.

#include "SubmitToolHelpWidget.h"

#include "View/SubmitToolStyle.h"
#include "Widgets/Input/SHyperlink.h"
#include "Widgets/SBoxPanel.h"
#include "Version/AppVersion.h"

#define LOCTEXT_NAMESPACE "SubmitToolHelpWidget"

// TODO BC: data drive all the data for the help page
void SSubmitToolHelpWidget::Construct(const FArguments& InArgs)
{
	ModelInterface = InArgs._ModelInterface.Get();

	TSharedPtr<SVerticalBox> Contents;
	ChildSlot
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			[
				SAssignNew(Contents, SVerticalBox)
			]
		];

	Contents->AddSlot()
		.AutoHeight()
		.HAlign(HAlign_Center)
		.Padding(5)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("Header", "The Submit Tool is a tool dedicated to help developers catch code and\ncontent issues locally before submitting them to source control.\n"))
		];

	Contents->AddSlot()
		.AutoHeight()
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.Padding(5)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("Documentation", "Please refer to the documentation page(s):"))
		];

	for(const FDocumentationLink& DocLink : ModelInterface->GetParameters().GeneralParameters.HelpLinks)
	{
		Contents->AddSlot()
			.AutoHeight()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.Padding(10)
			[
				SNew(SHyperlink)
				.Style(FSubmitToolStyle::Get(), TEXT("NavigationHyperlink"))
				.Text(FText::FromString(DocLink.Text))
				.ToolTipText(FText::FromString(DocLink.Tooltip))
				.OnNavigate_Lambda([&DocLink]() { FPlatformProcess::LaunchURL(*DocLink.Link, nullptr, nullptr); })
			];
	}

	Contents->AddSlot()
		.AutoHeight()
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.Padding(10)
		[
			SNew(STextBlock)
			.Text(
				FText::FromString(
					FString::Format(
						TEXT("Application version: {0}"), {FAppVersion::GetVersion()}
					)
				)
			)
		];

}

#undef LOCTEXT_NAMESPACE