// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

#include "Models/ModelInterface.h"

class SSubmitToolHelpWidget final : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SSubmitToolHelpWidget) {}
		SLATE_ATTRIBUTE(FModelInterface*, ModelInterface)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
private:
	const FModelInterface* ModelInterface;
};