// Copyright Epic Games, Inc. All Rights Reserved.

#include "SubmitToolWidget.h"

#include "ConfirmDialogWidget.h"
#include "ISettingsModule.h"
#include "OutputLogModule.h"
#include "OutputLogSettings.h"
#include "OutputLogCreationParams.h"

#include "View/SubmitToolStyle.h"
#include "View/SubmitToolCommandHandler.h"
#include "View/SubmitToolMenu.h"
#include "View/Widgets/SIntegrationWidget.h"

#include "Widgets/Layout/SExpandableArea.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Input/SHyperlink.h"
#include "Widgets/Input/SEditableText.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Images/SThrobber.h"
#include "Widgets/Docking/SDockTab.h"

#include "Logging/SubmitToolOutputLogHistory.h"
#include "Logging/SubmitToolOutputLog.h"
#include "Models/ModelInterface.h"
#include "Models/SubmitToolUserPrefs.h"
#include "TagSectionWidget.h"
#include "ValidatorsWidget.h"

#include "Version/AppVersion.h"

#include "Async/Async.h"
#include "HAL/FileManager.h"
#include "Modules/ModuleManager.h"
#include "Framework/Docking/TabManager.h"
#include "Framework/Application/SlateApplication.h"

#include "CommandLine/CmdLineParameters.h"
#include "SubmitToolCoreUtils.h"
#include "Configuration/Configuration.h"

#define LOCTEXT_NAMESPACE "SubmitToolWidget"

void SubmitToolWidget::OnCLDescriptionUpdated()
{
	AsyncTask(ENamedThreads::GameThread, [this] { DescriptionBox->Refresh(); });	
}

void SubmitToolWidget::HandleApplicationActivationStateChanged(bool bActive)
{
	if(ModelInterface == nullptr)
	{
		return;
	}
	if(ModelInterface->IsP4OperationRunning() || ModelInterface->IsBlockingOperationRunning())
	{
		return;
	}
	
	if(bActive)
	{
		ModelInterface->CheckForFileEdits();
		ModelInterface->UpdateCLFromP4Async();
	}
	else
	{
		FSlateApplication::Get().ClearKeyboardFocus();
		ModelInterface->SendDescriptionToP4();
	}
}

void SubmitToolWidget::Construct(const FArguments& InArgs, TSharedPtr<FSubmitToolOutputLogHistory> InLogHistory)
{
	ParentTab = InArgs._ParentTab.Get();
	ModelInterface = InArgs._ModelInterface;

	IntegrationWidget = SNew(SIntegrationWidget)
						.ModelInterface(ModelInterface)
						.MainWindow(InArgs._ParentWindow);

	FSlateApplication::Get().OnApplicationActivationStateChanged().AddRaw(this, &SubmitToolWidget::HandleApplicationActivationStateChanged);
	OnValidatorFinishedHandle = ModelInterface->AddSingleValidatorFinishedCallback(FOnSingleTaskFinished::FDelegate::CreateRaw(this, &SubmitToolWidget::OnSingleValidatorFinished));
	OnValidationQueueFinishedHandle = ModelInterface->AddValidationFinishedCallback(FOnTaskFinished::FDelegate::CreateRaw(this, &SubmitToolWidget::OnValidationQueueFinished));
	OnValidationUpdateHandle = ModelInterface->AddValidationUpdatedCallback(FOnTaskRunStateChanged::FDelegate::CreateRaw(this, &SubmitToolWidget::OnValidationUpdated));
	OnCLDescriptionUpdatedHandle = ModelInterface->GetCLDescriptionUpdatedDelegate().Add(FOnCLDescriptionUpdated::FDelegate::CreateRaw(this, &SubmitToolWidget::OnCLDescriptionUpdated));

	TSharedPtr<SVerticalBox> Contents;
	ChildSlot
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			[
				SAssignNew(Contents, SVerticalBox)
			]
		];

	/**** Tags ****/
	TSharedRef<STagSectionWidget> TagSection = SNew(STagSectionWidget)
		.ParentWindow(InArgs._ParentWindow)
		.ModelInterface(InArgs._ModelInterface);

	TSharedRef<SBox> TagSectionBox =
		SNew(SBox)
		.MaxDesiredHeight_Lambda([TagSection]{ return FMath::Min(TagSection->GetDesiredSize().Y, FSubmitToolUserPrefs::Get()->TagSectionSize);})
		[
			TagSection
		];

	Contents->AddSlot()
		.AutoHeight()
		[
			TagSectionBox
		];

	TSharedPtr<SBorder> ResizeBorder;

	Contents->AddSlot()
		.AutoHeight()
		[
			SAssignNew(ResizeBorder, SBorder)
			.OnMouseButtonUp_Lambda([](const FGeometry& Geometry, const FPointerEvent& PointerEvent) { return FReply::Handled().ReleaseMouseCapture(); })
			.OnMouseMove_Lambda([TagSection, this](const FGeometry& Geometry, const FPointerEvent& PointerEvent){
				if (PointerEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
				{
					FSubmitToolUserPrefs::Get()->TagSectionSize = FMath::Clamp(FSubmitToolUserPrefs::Get()->TagSectionSize + PointerEvent.GetCursorDelta().Y, ModelInterface->GetTagsArray().Num() == 0 ? 0 : 35, TagSection->GetDesiredSize().Y);
					return FReply::Handled();
				}

				return FReply::Unhandled();
			
			})
			[
				SNew(SSeparator).Thickness(5)
				.Cursor(EMouseCursor::ResizeUpDown)
			]			
		];

	ResizeBorder->SetOnMouseButtonDown(FPointerEventHandler::CreateLambda([ResizeBorder](const FGeometry& Geometry, const FPointerEvent& PointerEvent) { return FReply::Handled().CaptureMouse(ResizeBorder.ToSharedRef()); }));
	TSharedPtr<SSplitter> Splitter;

	Contents->AddSlot()
		.FillHeight(1.f)
		[
			SAssignNew(Splitter, SSplitter)
			.Orientation(EOrientation::Orient_Vertical)
		];


	TSharedPtr<SHorizontalBox> BottomLine = 
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.HAlign(HAlign_Left)
		[
			SNew(SEditableText)
			.IsReadOnly(true)
			.Text(FText::FromString(FAppVersion::GetVersion()))
		];

	for(size_t i = 0; i < ModelInterface->GetParameters().GeneralParameters.HelpLinks.Num(); ++i)
	{
		const FDocumentationLink& DocLink = ModelInterface->GetParameters().GeneralParameters.HelpLinks[i];
		SHorizontalBox::FScopedWidgetSlotArguments Slot = BottomLine->AddSlot();
		Slot.AttachWidget(
				SNew(SHyperlink)
					.Style(FSubmitToolStyle::Get(), TEXT("NavigationHyperlink"))
					.Text(FText::FromString(DocLink.Text))
					.ToolTipText(FText::FromString(DocLink.Tooltip))
					.OnNavigate_Lambda([&DocLink]() { FPlatformProcess::LaunchURL(*DocLink.Link, nullptr, nullptr); })
			);

		if(i != ModelInterface->GetParameters().GeneralParameters.HelpLinks.Num() - 1)
		{
			Slot.HAlign(HAlign_Center);
		}
		else
		{
			Slot.HAlign(HAlign_Right);
		}
	}

	/**** Version + feedback ****/
	Contents->AddSlot()
		.AutoHeight()
		[
			BottomLine.ToSharedRef()
		];

	/**** Description + Buttons ****/
	DescriptionBox = SNew(SMultiLineEditableTextBox)
		.Text_Lambda([&ModelInterface = ModelInterface]() { return FText::FromString(ModelInterface->GetCLDescription()); })
		.OnTextChanged_Lambda([&ModelInterface = ModelInterface](const FText& newText) { ModelInterface->SetCLDescription(newText); })
		.OnTextCommitted_Lambda([this](const FText& Text, ETextCommit::Type CommitType) { if(CommitType != ETextCommit::OnEnter) { ModelInterface->ValidateCLDescription(); }})
		.AutoWrapText(true)
		.OnIsTypedCharValid(FOnIsTypedCharValid::CreateLambda([](const TCHAR) { return true; }))
		.IsReadOnly_Lambda([this]{ return !FModelInterface::GetInputEnabled() || ModelInterface->IsDescriptionGenerationRunning();});
	
	ModelInterface->SetDescriptionBox(DescriptionBox);
	
	FString PerforceClientName;
	FCmdLineParameters::Get().GetValue(FSubmitToolCmdLine::P4Client, PerforceClientName);

	P4SectionSlot = Splitter->AddSlot()
		.Resizable(true)
		.MinSize(150)
		[
			SNew(SBox)			
			.WidthOverride(520)
			[
				SNew(SVerticalBox)
				+SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SBorder)
					.BorderBackgroundColor(FAppStyle::GetColor("ValidatorStateFail"))
					.Visibility_Lambda([this]() { return !ModelInterface->GetTitleMessage().IsEmpty() ? EVisibility::All : EVisibility::Collapsed;})
					[
						SNew(STextBlock)
							.Justification(ETextJustify::Center)
							.TextStyle(FAppStyle::Get(), "BoldTextNormalSize")
							.Text_Lambda([this]() { return FText::FromString(ModelInterface->GetTitleMessage()); })
					]
				]
				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(FMargin(5))
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					.AutoWidth()
					[
						SNew(STextBlock)
						.Font(FAppStyle::Get().GetFontStyle("SmallFontBold"))
						.Text(NSLOCTEXT("SourceControl.SubmitPanel", "ChangeListDesc", "Changelist Description"))
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SButton)
						.Visibility_Lambda([this] { return ModelInterface->GetParameters().GeneralParameters.DescriptionGenDocumentationUrl.IsEmpty() ? EVisibility::Collapsed : EVisibility::Visible; })
						.ButtonStyle(FAppStyle::Get(), "InvisibleButton")
						.IsFocusable(false)
						.ToolTipText(FText::FromString(ModelInterface->GetParameters().GeneralParameters.DescriptionGenTooltip))
						.Cursor(EMouseCursor::Hand)
						.OnClicked_Lambda([this]()
							{
								if (!ModelInterface->GetParameters().GeneralParameters.DescriptionGenDocumentationUrl.IsEmpty())
								{
									FPlatformProcess::LaunchURL(*ModelInterface->GetParameters().GeneralParameters.DescriptionGenDocumentationUrl, nullptr, nullptr);
								}

								return FReply::Handled();
							})
						[
							SNew(SImage)
								.Image(FSubmitToolStyle::Get().GetBrush("AppIcon.DocumentationHelp"))
						]
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(5, 0, 0, 0)
					[
						SNew(SButton)
						.ContentPadding(-3)
						.Visibility_Lambda([this] { return ModelInterface->GetParameters().GeneralParameters.DescriptionGenProcess.IsEmpty() ? EVisibility::Collapsed : EVisibility::Visible; })
						.IsEnabled_Lambda([this] { return !ModelInterface->IsDescriptionGenerationRunning(); })
						.IsFocusable(true)
						.ToolTipText(FText::FromString(TEXT("Generate Description")))
						.OnClicked_Lambda([this]()
						{
							ModelInterface->GenerateCLDescription();
							return FReply::Handled();
						})
						[
							SNew(SHorizontalBox)
							+SHorizontalBox::Slot()
							.Padding(0,0,2,0)
							.VAlign(VAlign_Center)
							.HAlign(HAlign_Center)
							.AutoWidth()
							[
								SNew(SImage)
								.Image(FSubmitToolStyle::Get().GetBrush("AppIcon.Generate"))
							]
							+SHorizontalBox::Slot()
							.VAlign(VAlign_Center)
							.HAlign(HAlign_Center)
							.AutoWidth()
							[
								SNew(STextBlock)
								.Font(FAppStyle::Get().GetFontStyle("SmallFontBold"))
								.Text(FText::FromString(TEXT("Generate")))
							]
						]
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(5, 0)
					.VAlign(VAlign_Center)
					[
						SNew(SThrobber)
							.NumPieces(3)
							.Visibility_Lambda([this]()
							{
								return ModelInterface->IsDescriptionGenerationRunning()
									? EVisibility::Visible
									: EVisibility::Collapsed;
							})
					]

					+ SHorizontalBox::Slot()
					.FillWidth(1.f)
					[
						SNew(SSpacer)
					]

					// STREAM 
					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Right)
					.VAlign(VAlign_Center)
					.AutoWidth()
					[
						SNew(STextBlock)
						.Font(FAppStyle::Get().GetFontStyle("SmallFontBold"))
						.Text(NSLOCTEXT("SourceControl.SubmitPanel", "Stream", "Stream"))
					]
					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Right)
					.VAlign(VAlign_Center)
					.Padding(5, 0, 0, 0)
					.AutoWidth()
					[
						SNew(STextBlock)
						.Font(FAppStyle::Get().GetFontStyle("SmallFont"))
						.Text_Lambda([this]() { return FText::FromString(ModelInterface->GetCurrentStream()); })
					]

					// WORKSPACE
					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Right)
					.VAlign(VAlign_Center)
					.Padding(15, 0, 0, 0)
					.AutoWidth()
					[
						SNew(STextBlock)
						.Font(FAppStyle::Get().GetFontStyle("SmallFontBold"))
						.Text(NSLOCTEXT("SourceControl.SubmitPanel", "Workspace", "Workspace"))
					]
					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Right)
					.VAlign(VAlign_Center)
					.Padding(5, 0, 0, 0)
					.AutoWidth()
					[
						SNew(STextBlock)
						.Font(FAppStyle::Get().GetFontStyle("SmallFont"))
						.Text(FText::FromString(PerforceClientName))
					]
				]
				+SVerticalBox::Slot()
				.FillHeight(.2f)
				.Padding(FMargin(5, 0, 5, 5))
				[
					DescriptionBox.ToSharedRef()
				] 
				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(FMargin(5, 0, 5, 5))
				[
					BuildButtonRow()
				]
				
			]
		].GetSlot();
	
	P4SectionSlot->SetSizeValue(FSubmitToolUserPrefs::Get()->P4SectionSize);

	TSharedPtr<SWidget> LogSection = BuildOutputLogWidget(InLogHistory);
	TSharedPtr<SWidget> FilesInCLSection = BuildFilesInCLWidget();
	InLogHistory.Reset();

	// Files & Validators
	ValidatorSectionSlot = Splitter->AddSlot()		
		.MinSize(45)
		.SizeRule(SSplitter::ESizeRule::FractionOfParent)
		.Resizable(true)
		[
			SNew(SBox)
			.Padding(0.f,4.f)
			[
				SNew(SScrollBox)
				.Orientation(EOrientation::Orient_Vertical)
				+SScrollBox::Slot()
				.Padding(FMargin(0.f, 2.f))
				.AutoSize()
				[
					FilesInCLSection.ToSharedRef()
				]
				+SScrollBox::Slot()
				.Padding(FMargin(0.f, 2.f))
				.AutoSize()
				[
					SNew(SValidatorsWidget)
					.OnViewLog_Lambda([this](TSharedPtr<const FValidatorBase> validator) { LogTabManager->DrawAttention(ValidatorLogTab.ToSharedRef()); })
					.ModelInterface(ModelInterface)
				]
			]
		].GetSlot();

	ValidatorSectionSlot->SetSizeValue(FSubmitToolUserPrefs::Get()->ValidatorSectionSize);

	// Log
	LogSectionSlot = Splitter->AddSlot()
		.MinSize(200)
		.SizeRule(SSplitter::ESizeRule::FractionOfParent)
		.Resizable(true)
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.Padding(FMargin(0.f, 4.f))
			.FillHeight(1.f)
			[
				LogSection.ToSharedRef()
			]
		].GetSlot();

	LogSectionSlot->SetSizeValue(FSubmitToolUserPrefs::Get()->LogSectionSize);

}

SubmitToolWidget::~SubmitToolWidget()
{
	if(ModelInterface != nullptr)
	{
		FSubmitToolUserPrefs* UserPrefs = FSubmitToolUserPrefs::Get();

		UserPrefs->P4SectionSize = P4SectionSlot->GetSizeValue();
		UserPrefs->ValidatorSectionSize = ValidatorSectionSlot->GetSizeValue();
		UserPrefs->LogSectionSize = LogSectionSlot->GetSizeValue();

		ModelInterface->RemoveValidationFinishedCallback(OnValidationQueueFinishedHandle);
		ModelInterface->RemoveSingleValidatorFinishedCallback(OnValidatorFinishedHandle);
		ModelInterface->RemoveValidationUpdatedCallback(OnValidationUpdateHandle);
		ModelInterface->GetCLDescriptionUpdatedDelegate().Remove(OnCLDescriptionUpdatedHandle);
		ModelInterface = nullptr;
	}
}

TSharedRef<SHorizontalBox> SubmitToolWidget::BuildButtonRow()
{

	ValidateBtn = SNew(SButton)
		.ToolTipText_Raw(this, &SubmitToolWidget::GetValidateButtonTooltip)
		.IsEnabled_Lambda([&ModelInterface = ModelInterface] { return (FModelInterface::GetInputEnabled() && !ModelInterface->GetFilesInCL().IsEmpty()) || ModelInterface->IsP4OperationRunning(); })
		.ButtonStyle(FAppStyle::Get(), "PrimaryButton")
		.OnClicked(this, &SubmitToolWidget::ValidateClicked)
		[
			SNew(STextBlock)
			.MinDesiredWidth(130)
			.Justification(ETextJustify::Center)
			.Text_Raw(this, &SubmitToolWidget::GetValidateButtonText)
		];

	return SNew(SHorizontalBox)
#if PLATFORM_WINDOWS
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "Button")
				.OnClicked(this, &SubmitToolWidget::CopyAllLogsClicked)
				[
					SNew(STextBlock)
						.MinDesiredWidth(130)
						.Justification(ETextJustify::Center)
						.Text(FText::FromString("Copy All Logs"))
				]
		]
#endif
		+SHorizontalBox::Slot()
		.FillWidth(1.f)
		[
			SNew(SSpacer)
		]
		+ SHorizontalBox::Slot()
		.FillWidth(1.f)
		.HAlign(HAlign_Center)
		.MaxWidth(700)
		.Padding(8, 0)
		.AutoWidth()
		[
			SNew(SThrobber)
			.NumPieces(8)
			.Visibility_Lambda([this]() { return ModelInterface->IsP4OperationRunning() || ModelInterface->IsBlockingOperationRunning() ? EVisibility::Visible : EVisibility::Collapsed; })
		]
		+SHorizontalBox::Slot()
		.FillWidth(1.f)
		.HAlign(HAlign_Right)
		.MaxWidth(700)
		.Padding(4, 0)
		.AutoWidth()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SCheckBox)
				.IsChecked_Lambda([this]() { return ModelInterface->bSubmitOnSuccessfulValidation ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
				.OnCheckStateChanged_Lambda([this](ECheckBoxState InNewState) { ModelInterface->bSubmitOnSuccessfulValidation = !ModelInterface->bSubmitOnSuccessfulValidation; })
				.IsFocusable(false)
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "InvisibleButton")
				.IsFocusable(false)	
				.OnClicked_Lambda([this]() { ModelInterface->bSubmitOnSuccessfulValidation = !ModelInterface->bSubmitOnSuccessfulValidation; return FReply::Handled(); })
				[
					SNew(STextBlock)
					.Justification(ETextJustify::Left)
					.Font(FAppStyle::Get().GetFontStyle("SmallFont"))
					.MinDesiredWidth(60)
					.Text(FText::FromString(TEXT("Submit On Successful Validation")))
				]
			]
		]
		+SHorizontalBox::Slot()
		.FillWidth(1.f)
		.HAlign(HAlign_Right)
		.MaxWidth(700)
		.AutoWidth()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SCheckBox)
				.IsChecked_Lambda([this]() { return FSubmitToolUserPrefs::Get()->bOpenJiraOnSubmit ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
				.OnCheckStateChanged_Lambda([this](ECheckBoxState InNewState) { FSubmitToolUserPrefs::Get()->bOpenJiraOnSubmit = !FSubmitToolUserPrefs::Get()->bOpenJiraOnSubmit; })
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "InvisibleButton")
				.IsFocusable(false)	
				.OnClicked_Lambda([this]() { FSubmitToolUserPrefs::Get()->bOpenJiraOnSubmit = !FSubmitToolUserPrefs::Get()->bOpenJiraOnSubmit; return FReply::Handled(); })
				[
					SNew(STextBlock)
					.Justification(ETextJustify::Left)
					.Font(FAppStyle::Get().GetFontStyle("SmallFont"))
					.MinDesiredWidth(60)
					.Text(FText::FromString(TEXT("Open Ticket on Submit")))
				]
			]
		]
		+SHorizontalBox::Slot()
		.FillWidth(1.f)
		.HAlign(HAlign_Right)
		.MaxWidth(700)
		.AutoWidth()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SCheckBox)
				.IsChecked_Lambda([this]() { return FSubmitToolUserPrefs::Get()->bCloseOnSubmit ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
				.OnCheckStateChanged_Lambda([this](ECheckBoxState InNewState) { FSubmitToolUserPrefs::Get()->bCloseOnSubmit = !FSubmitToolUserPrefs::Get()->bCloseOnSubmit; })
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "InvisibleButton")
				.IsFocusable(false)	
				.OnClicked_Lambda([this]() { FSubmitToolUserPrefs::Get()->bCloseOnSubmit = !FSubmitToolUserPrefs::Get()->bCloseOnSubmit; return FReply::Handled(); })
				[
					SNew(STextBlock)
					.Justification(ETextJustify::Left)
					.Font(FAppStyle::Get().GetFontStyle("SmallFont"))
					.MinDesiredWidth(60)
					.Text(FText::FromString(TEXT("Close on Submit Success")))
				]
			]
		]
		+SHorizontalBox::Slot()
		.FillWidth(1.f)
		.HAlign(HAlign_Right)
		.MaxWidth(700)
		.AutoWidth()
		[
			SNew(SButton)
			.Text(FText::FromString(TEXT("Open Integration Window")))
			.Visibility_Lambda([this] { return ModelInterface->IsIntegrationRequired() && ModelInterface->LockdownStatus.bIsAllowlisted ? EVisibility::Visible : EVisibility::Collapsed; })
			.OnClicked_Lambda([this] { IntegrationWidget->Open(); return FReply::Handled(); })
			.IsEnabled_Lambda([&ModelInterface = ModelInterface]
				{
					if (ModelInterface->IsPreflightRequestInProgress())
					{
						return false;
					}

					return ModelInterface->IsCLValid();
				})
			[
				SNew(STextBlock)
				.MinDesiredWidth(130)
					.Justification(ETextJustify::Center)
					.Text(FText::FromString(TEXT("Open Integration Window")))
			]
		]
		+ SHorizontalBox::Slot()
			.FillWidth(1.f)
			.HAlign(HAlign_Right)
			.MaxWidth(700)
			.AutoWidth()
			[
				SNew(SUniformGridPanel)
				.SlotPadding(FAppStyle::GetMargin("StandardDialog.SlotPadding"))
				.MinDesiredSlotWidth(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotWidth"))
				+ SUniformGridPanel::Slot(0, 0)
				[
					SNew(SButton)
					.IsEnabled_Lambda([&ModelInterface = ModelInterface] 
					{
						if (ModelInterface->IsPreflightRequestInProgress() || ModelInterface->IsSubmissionBlocked())
						{
							return false;
						}

						if (ModelInterface->GetState() == ESubmitToolAppState::Finished)
						{
							return true;
						}
					
						if (ModelInterface->GetState() == ESubmitToolAppState::WaitingUserInput || (ModelInterface->IsIntegrationRequired() && ModelInterface->LockdownStatus.bIsAllowlisted))
						{
							return ModelInterface->IsCLValid();
						}

						if(ModelInterface->IsIntegrationRequired())
						{
							return ModelInterface->IsCLValid() || (ModelInterface->GetFilesInCL().IsEmpty() && ModelInterface->HasShelvedFiles() && ModelInterface->HasSubmitToolTag());
						}

						return false;
					})
					.OnClicked(this, &SubmitToolWidget::SubmitClicked)
					.ButtonStyle(FAppStyle::Get(), "PrimaryButton")
					[
						SNew(STextBlock)
						.MinDesiredWidth(130)
						.Justification(ETextJustify::Center)
						.Text_Raw(this, &SubmitToolWidget::GetMainButtonText)
					]
				]
			+SUniformGridPanel::Slot(1, 0)
			[
				ValidateBtn.ToSharedRef()
			]
		];
}

TSharedRef<SWidget> SubmitToolWidget::BuildOutputLogWidget(const TSharedPtr<FSubmitToolOutputLogHistory>& InLogHistory)
{
	/*** Output Log Widget ***/
	FOutputLogModule& OutputLogModule = FModuleManager::Get().LoadModuleChecked<FOutputLogModule>("OutputLog");

	// hide the debug console
	IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("OutputLogModule.HideConsole"));
	if(ensure(CVar))
	{
		CVar->Set(true);
	}

	// setup OutputLog settings
	UOutputLogSettings* Settings = GetMutableDefault<UOutputLogSettings>();
	if(Settings)
	{
		Settings->CategoryColorizationMode = ELogCategoryColorizationMode::ColorizeWholeLine;
	}

	ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");
	if(SettingsModule)
	{
		SettingsModule->RegisterSettings("Editor", "General", "Output Log",
			NSLOCTEXT("OutputLog", "OutputLogSettingsName", "Output Log"),
			NSLOCTEXT("OutputLog", "OutputLogSettingsDescription", "Set up preferences for the Output Log appearance and workflow."),
			Settings
		);
	}

	LogTabManager = FGlobalTabmanager::Get()->NewTabManager(ParentTab.Pin().ToSharedRef());
	LogTabManager->SetCanDoDragOperation(false);

	// Menu
	// TODO BC: this is a poor solution to have the menu maintained between the two tabs please revisit
	TSharedRef<FUICommandList> CommandList = MakeShared<FUICommandList>();
	FSubmitToolCommandHandler CommandHandler;
	CommandHandler.AddToCommandList(ModelInterface, CommandList);

	FMenuBarBuilder MenuBarBuilder = FMenuBarBuilder(CommandList);
	MenuBarBuilder.AddPullDownMenu(LOCTEXT("MainMenu", "Main Menu"), LOCTEXT("OpensMainMenu", "Opens Main Menu"), FNewMenuDelegate::CreateStatic(&FSubmitToolMenu::FillMainMenuEntries));
#if !UE_BUILD_SHIPPING
	if(!FPaths::IsStaged())
	{
		MenuBarBuilder.AddPullDownMenu(LOCTEXT("Debug Tools", "Debug"), LOCTEXT("OpensDebugMenu", "Opens Debug Menu"), FNewMenuDelegate::CreateStatic(&FSubmitToolMenu::FillDebugMenuEntries));
	}
#endif

	const TSharedRef<SWidget> MenuWidget = MenuBarBuilder.MakeWidget();
	LogTabManager->SetAllowWindowMenuBar(true);
	LogTabManager->SetMenuMultiBox(MenuBarBuilder.GetMultiBox(), MenuWidget);

	// Release the Outputlog redirect to avoid races
	InLogHistory->Release();

	// ----------------------------------------------------------------------
	// Summary Log Tab
	// ----------------------------------------------------------------------
	FDefaultCategorySelectionMap DefaultCategorySelection;
	DefaultCategorySelection.Emplace(LogSubmitTool.GetCategoryName(), true);
	DefaultCategorySelection.Emplace(LogValidators.GetCategoryName(), false);
	DefaultCategorySelection.Emplace(LogValidatorsResult.GetCategoryName(), true);
	DefaultCategorySelection.Emplace(LogPresubmitResult.GetCategoryName(), true);
	DefaultCategorySelection.Emplace(LogOutputDevice.GetCategoryName(), true);
	DefaultCategorySelection.Emplace(LogSubmitToolP4.GetCategoryName(), true);
	DefaultCategorySelection.Emplace(LogPresubmit.GetCategoryName(), false);

	SummaryLogDockTab = SNew(SDockTab)
		.CanEverClose(false)
		.TabRole(ETabRole::PanelTab)
		.Label(FText::FromString(TEXT("Summary")))
		.OnCanCloseTab(SDockTab::FCanCloseTab::CreateLambda([] { return false; }))
		[
			SAssignNew(SummaryLogWidget, SSubmitToolOutputLog, false)
				.Messages(&InLogHistory->GetMessages())
				.DefaultCategorySelection(DefaultCategorySelection)
				.AllowInitialLogCategory(FAllowLogCategoryCallback::CreateLambda([](const FName LogCategory) { return LogCategory == LogSubmitTool.GetCategoryName(); }))
		];
	
	LogTabManager->RegisterTabSpawner(SummaryTabId, FOnSpawnTab::CreateLambda([this](const FSpawnTabArgs& SpawnArgs) {	return SummaryLogDockTab.ToSharedRef(); }));

	// ----------------------------------------------------------------------
	// Validators Log Tab
	// ----------------------------------------------------------------------
	DefaultCategorySelection.Reset();
	DefaultCategorySelection.Emplace(LogSubmitTool.GetCategoryName(), false);
	DefaultCategorySelection.Emplace(LogValidators.GetCategoryName(), true);
	DefaultCategorySelection.Emplace(LogValidatorsResult.GetCategoryName(), false);
	DefaultCategorySelection.Emplace(LogPresubmitResult.GetCategoryName(), false);
	DefaultCategorySelection.Emplace(LogOutputDevice.GetCategoryName(), false);
	DefaultCategorySelection.Emplace(LogSubmitToolP4.GetCategoryName(), false);
	DefaultCategorySelection.Emplace(LogPresubmit.GetCategoryName(), false);


	ValidatorLogTab = SNew(SDockTab)
		.CanEverClose(false)
		.Label(FText::FromString(TEXT("Validators Log")))
		.TabRole(ETabRole::PanelTab)
		.OnCanCloseTab(SDockTab::FCanCloseTab::CreateLambda([] { return false; }))
		[
			SAssignNew(ValidatorLogWidget, SSubmitToolOutputLog, false)
				.Messages(&InLogHistory->GetMessages())
				.DefaultCategorySelection(DefaultCategorySelection)
				.AllowInitialLogCategory(FAllowLogCategoryCallback::CreateLambda([](const FName LogCategory) {	return LogCategory == LogValidators.GetCategoryName(); }))
		];

	LogTabManager->RegisterTabSpawner(ValidatorLogTabId, FOnSpawnTab::CreateLambda([this](const FSpawnTabArgs& SpawnArgs) { return ValidatorLogTab.ToSharedRef(); }));

	// ----------------------------------------------------------------------
	// Pre Submit Log Tab
	// ----------------------------------------------------------------------
	DefaultCategorySelection.Reset();
	DefaultCategorySelection.Emplace(LogPresubmit.GetCategoryName(), true);
	DefaultCategorySelection.Emplace(LogSubmitTool.GetCategoryName(), false);
	DefaultCategorySelection.Emplace(LogValidators.GetCategoryName(), false);
	DefaultCategorySelection.Emplace(LogValidatorsResult.GetCategoryName(), false);
	DefaultCategorySelection.Emplace(LogPresubmitResult.GetCategoryName(), false);
	DefaultCategorySelection.Emplace(LogOutputDevice.GetCategoryName(), false);
	DefaultCategorySelection.Emplace(LogSubmitToolP4.GetCategoryName(), false);

	PresubmitLogTab = SNew(SDockTab)
		.CanEverClose(false)
		.Label(FText::FromString(TEXT("Presubmit Log")))
		.TabRole(ETabRole::PanelTab)
		.OnCanCloseTab(SDockTab::FCanCloseTab::CreateLambda([] { return false; }))
		[
			SAssignNew(PresubmitLogWidget, SSubmitToolOutputLog, false)
				.Messages(&InLogHistory->GetMessages())
				.DefaultCategorySelection(DefaultCategorySelection)
				.AllowInitialLogCategory(FAllowLogCategoryCallback::CreateLambda([](const FName LogCategory) {	return LogCategory == LogPresubmit.GetCategoryName(); }))
		];

	LogTabManager->RegisterTabSpawner(PresubmitLogTabId, FOnSpawnTab::CreateLambda([this](const FSpawnTabArgs& SpawnArgs) { return PresubmitLogTab.ToSharedRef(); }));

	// ----------------------------------------------------------------------
	// Logs Layout
	// ----------------------------------------------------------------------
	TSharedRef<FTabManager::FLayout> Layout = FTabManager::NewLayout("SubmitToolLogLayout")
		->AddArea
		(
			FTabManager::NewPrimaryArea()
			->SetOrientation(Orient_Horizontal)
			->Split
			(
				FTabManager::NewStack()
				->SetSizeCoefficient(1.0f)
				->SetHideTabWell(true)
				->AddTab(SummaryTabId, ETabState::OpenedTab)
				->AddTab(ValidatorLogTabId, ETabState::OpenedTab)
				->AddTab(PresubmitLogTabId, ETabState::OpenedTab)
				->SetForegroundTab(SummaryTabId)
			)
		);

	return LogTabManager->RestoreFrom(Layout, ParentTab.Pin()->GetParentWindow()).ToSharedRef();
}

static FSlateIcon GetIcon(ESCFileState State)
{
	switch (State)
	{
	case ESCFileState::Unknown:
	case ESCFileState::Edit:
		return FSlateIcon(FSubmitToolStyle::Get().GetStyleSetName(), "RevisionControl.CheckedOut");
	case ESCFileState::Add:
		return FSlateIcon(FSubmitToolStyle::Get().GetStyleSetName(), "RevisionControl.OpenForAdd");
	case ESCFileState::Delete:
		return FSlateIcon(FSubmitToolStyle::Get().GetStyleSetName(), "RevisionControl.MarkedForDelete");
	case ESCFileState::Integrate:
		return FSlateIcon(FSubmitToolStyle::Get().GetStyleSetName(), "RevisionControl.Branched");
	default:
		ensureMsgf(false, TEXT("GetLocalFileState: Unknown perforce file state: '%s'"), *StaticEnum<ESCFileState>()->GetNameStringByValue(static_cast<int64>(State)));
		return {};
	}
}

TSharedRef<SExpandableArea> SubmitToolWidget::BuildFilesInCLWidget()
{
	TSharedPtr<SListView<FSCFileRef>> FileList = SNew(SListView<FSCFileRef>)
		.SelectionMode(ESelectionMode::Single)
		.ListItemsSource(&ModelInterface->GetFilesInCL())
		.OnGenerateRow_Lambda([this](FSCFileRef InItem, const TSharedRef<STableViewBase>& OwnerTable)
			{
				return SNew(STableRow<FSCFileRef>, OwnerTable)
					.Padding(2.0f)
					[
						SNew(SHorizontalBox)
				
						+ SHorizontalBox::Slot()
						.MaxWidth(24)
						.HAlign(HAlign_Left)
						[
							SNew(SImage)
							.Image(GetIcon(InItem->GetState()).GetIcon())
						]

						+ SHorizontalBox::Slot()
						.AutoWidth()
						.HAlign(HAlign_Left)
						[
							SNew(STextBlock)
							.Text(FText::FromString(InItem->GetFilename()))
						]
					];

			});

	ModelInterface->FileRefreshedCallback.AddLambda([FileList]() {FileList->RequestListRefresh(); });
	ModelInterface->SubmitFinishedCallback.AddLambda([this](bool bSuccess) { SummaryLogDockTab->ActivateInParent(ETabActivationCause::SetDirectly); SummaryLogWidget->ScrollToEnd(); });
	ModelInterface->PrepareSubmitCallBack.AddLambda([this]() { SummaryLogDockTab->ActivateInParent(ETabActivationCause::SetDirectly); SummaryLogWidget->ScrollToEnd(); });

	return SNew(SExpandableArea)
		.InitiallyCollapsed(!FSubmitToolUserPrefs::Get()->bExpandFilesInCL)
		.OnAreaExpansionChanged_Lambda([](bool bExpanded) { FSubmitToolUserPrefs::Get()->bExpandFilesInCL = bExpanded; })
		.BorderImage(FAppStyle::Get().GetBrush("Brushes.Header"))
		.BodyBorderImage(FAppStyle::Get().GetBrush("Brushes.Recessed"))
		.HeaderPadding(FMargin(4.0f, 2.0f))
		.Padding(1.0f)
		.MaxHeight(200.0f)
		.AllowAnimatedTransition(true)
		.HeaderContent()
		[
			SNew(STextBlock)
			.Text(NSLOCTEXT("SourceControl.SubmitPanel", "ChangeListFiles", "Files in Changelist"))
		]
		.BodyContent()
		[
			SNew(SBox)
			.Padding(2.5)
			[
				FileList.ToSharedRef()
			]
		];
}

FReply SubmitToolWidget::SubmitClicked()
{
	if(ModelInterface->GetState() == ESubmitToolAppState::Finished)
	{
		ParentTab.Pin()->RequestCloseTab();
	}
	else
	{
		if (ModelInterface->IsIntegrationRequired() && !ModelInterface->LockdownStatus.bIsAllowlisted)
		{
			IntegrationWidget->Open();
			return FReply::Handled();
		}

		if(ModelInterface->IsCLValid())
		{
			ModelInterface->StartSubmitProcess();
		}
	}

	return FReply::Handled();
}

FReply SubmitToolWidget::ValidateClicked()
{
	TRACE_BOOKMARK(TEXT("ValidateClicked"));

	if(ModelInterface->IsValidationRunning())
	{
		ModelInterface->CancelValidations();
	}
	else
	{
		if(ModelInterface->IsP4OperationRunning())
		{
			ModelInterface->CancelP4Operations();
		}
		else
		{
			ModelInterface->ValidateChangelist();
		}
	}

	return FReply::Handled();
}

FText SubmitToolWidget::GetMainButtonText() const
{
	FText FinalText;

	if(ModelInterface->GetState() == ESubmitToolAppState::Finished)
	{
		FinalText = NSLOCTEXT("SourceControl.SubmitPanel", "CloseButton", "Close");
	}
	else
	{
		FinalText = ModelInterface->IsIntegrationRequired() && !ModelInterface->LockdownStatus.bIsAllowlisted ? NSLOCTEXT("SourceControl.SubmitPanel", "IntegrationButton", "Open Integration Window") : NSLOCTEXT("SourceControl.SubmitPanel", "SubmitButton", "Submit");
	}

	return FinalText;
}

FText SubmitToolWidget::GetValidateButtonTooltip() const
{

	FText FinalText;

	if(ModelInterface->IsValidationRunning())
	{
		FinalText = NSLOCTEXT("SourceControl.SubmitPanel", "CancelValidateButtonTooltip", "Stops the currently running validations.");
	}
	else
	{
		if(ModelInterface->IsP4OperationRunning())
		{
			FinalText = NSLOCTEXT("SourceControl.SubmitPanel", "CancelP4OpButtonTooltip", "Cancels the currently running P4 Operations.");
		}
		else
		{
			FinalText = NSLOCTEXT("SourceControl.SubmitPanel", "ValidateButtonTooltip", "Run all the validators for this changelist.");
		}
	}
	return FinalText;
}

FText SubmitToolWidget::GetValidateButtonText() const
{
	FText FinalText;

	if(ModelInterface->IsValidationRunning())
	{
		FinalText = NSLOCTEXT("SourceControl.SubmitPanel", "CancelValidateButton", "Stop Validations");
	}
	else
	{
		if(ModelInterface->IsP4OperationRunning())
		{
			FinalText = NSLOCTEXT("SourceControl.SubmitPanel", "CancelP4OpButtonText", "Cancel P4 Operations");
		}
		else
		{
			FinalText = NSLOCTEXT("SourceControl.SubmitPanel", "ValidateButtonText", "Validate");
		}
	}
	return FinalText;
}

void SubmitToolWidget::OnSingleValidatorFinished(const FValidatorBase& InValidator)
{
	if(!InValidator.GetHasPassed())
	{
		ValidatorLogTab->FlashTab();
		ValidatorLogWidget->ScrollToEnd();
	}
}

void SubmitToolWidget::OnValidationQueueFinished(const bool bInSuccess)
{
	if (!bInSuccess)
	{
		SummaryLogWidget->ScrollToEnd();
	}
}

void SubmitToolWidget::OnValidationUpdated(bool bValid)
{
	if(bValid)
	{
		ValidateBtn->SetButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("Button"));
	}
	else
	{
		ValidateBtn->SetButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("PrimaryButton"));
		ValidatorLogWidget->ScrollToEnd();
	}
}

#if PLATFORM_WINDOWS
FReply SubmitToolWidget::CopyAllLogsClicked()
{
	TRACE_BOOKMARK(TEXT("CopyAllLogsClicked"));
	TRACE_CPUPROFILER_EVENT_FLUSH();

	TArray<FString> Files;
	for (const FString& Path : ModelInterface->GetParameters().CopyLogParameters.LogsToCollect)
	{
		FString FullPath = FPaths::ConvertRelativePathToFull(FConfiguration::Substitute(Path));
		if (IFileManager::Get().FileExists(*FullPath))
		{
			Files.Emplace(MoveTemp(FullPath));
		}
	}

	FSubmitToolCoreUtils::CopyDiagnosticFilesToClipboard(Files);

	UE_LOGF(LogSubmitTool,Display, "Log files have been copied to the clipboard");
	return FReply::Handled();
}
#endif


#undef LOCTEXT_NAMESPACE
