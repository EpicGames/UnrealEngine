// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConfirmDialogWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Layout/SSplitter.h"
#include "Logic/DialogFactory.h"

class FModelInterface;
class SMultiLineEditableTextBox;
class SDockTab;
class SWindow;
class SButton;
class FTabManager;
class FValidatorBase;
class SExpandableArea;
class SIntegrationWidget;
class FSubmitToolOutputLogHistory;
class SSubmitToolOutputLog;

class SubmitToolWidget final : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SubmitToolWidget) {}
		SLATE_ATTRIBUTE(TSharedPtr<SDockTab>, ParentTab)
		SLATE_ARGUMENT(FModelInterface*, ModelInterface)
		SLATE_ARGUMENT(TSharedPtr<SWindow>, ParentWindow)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedPtr<FSubmitToolOutputLogHistory> InLogHistory);
	virtual ~SubmitToolWidget() override;
	
	FReply SubmitClicked();
	FReply ValidateClicked();
private:
	void OnCLDescriptionUpdated();
	void HandleApplicationActivationStateChanged(bool bActive);
	TWeakPtr<SDockTab> ParentTab;
	FModelInterface* ModelInterface;
	FText GetMainButtonText() const;
	FText GetValidateButtonText() const;
	FText GetValidateButtonTooltip() const;

	TSharedPtr<FTabManager> LogTabManager;
	TSharedPtr<SButton> ValidateBtn;
	TSharedRef<SHorizontalBox> BuildButtonRow();
	TSharedRef<SWidget> BuildOutputLogWidget(const TSharedPtr<FSubmitToolOutputLogHistory>& InLogHistory);
	TSharedRef<SExpandableArea> BuildFilesInCLWidget();


	TSharedPtr<SDockTab> ValidatorLogTab;
	const FName ValidatorLogTabId = "ValidatorLogTab";
	TSharedPtr<SSubmitToolOutputLog> ValidatorLogWidget;
	TSharedPtr<SDockTab> PresubmitLogTab;
	const FName PresubmitLogTabId = "PresubmitLogTab";
	TSharedPtr<SSubmitToolOutputLog> PresubmitLogWidget;
	TSharedPtr<SDockTab> SummaryLogDockTab;
	const FName SummaryTabId = "SummaryLogTab";
	TSharedPtr<SSubmitToolOutputLog> SummaryLogWidget;

	TSharedPtr<SMultiLineEditableTextBox> DescriptionBox;
	TSharedPtr<SIntegrationWidget> IntegrationWidget;

	SSplitter::FSlot* P4SectionSlot;
	SSplitter::FSlot* ValidatorSectionSlot;
	SSplitter::FSlot* LogSectionSlot;

	FDelegateHandle OnValidatorFinishedHandle;
	FDelegateHandle OnValidationQueueFinishedHandle;
	FDelegateHandle OnValidationUpdateHandle;
	FDelegateHandle OnCLDescriptionUpdatedHandle;
	void OnSingleValidatorFinished(const FValidatorBase& InValidator);
	void OnValidationQueueFinished(const bool bInSuccess);
	void OnValidationUpdated(bool bValid);

#if PLATFORM_WINDOWS
	FReply CopyAllLogsClicked();
#endif
};