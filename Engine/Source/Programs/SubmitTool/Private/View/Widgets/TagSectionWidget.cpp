// Copyright Epic Games, Inc. All Rights Reserved.

#include "TagSectionWidget.h"

#include "TagWidget.h"
#include "Models/ModelInterface.h"
#include "Models/Tag.h"
#include "View/Widgets/SJiraWidget.h"
#include "Widgets/Layout/SScrollBox.h"


#define LOCTEXT_NAMESPACE "TagSectionWidget"

void STagSectionWidget::Construct(const FArguments& InArgs)
{
	TSharedPtr<SVerticalBox> ColumnOne;
	TSharedPtr<SVerticalBox> ColumnTwo;

	TSharedRef<SJiraWidget> Issues = SNew(SJiraWidget)
		.ParentWindow(InArgs._ParentWindow)
		.ModelInterface(InArgs._ModelInterface);

	ChildSlot
	[
		SNew(SScrollBox)
		.Orientation(EOrientation::Orient_Vertical)
		+SScrollBox::Slot()
		.AutoSize()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			[
				SAssignNew(ColumnOne, SVerticalBox)
			]
			+SHorizontalBox::Slot()
			[
				SAssignNew(ColumnTwo, SVerticalBox)
			]
		]
	];

	TArray<const FTag*> Tags = InArgs._ModelInterface->GetTagsArray();

	int TagCount = 0;
	for(const FTag* TagInstance : Tags)
	{
		SVerticalBox* TargetColumn = TagCount % 2 == 0 ? ColumnOne.Get() : ColumnTwo.Get();
		
		TargetColumn->AddSlot()
		.AutoHeight()
		.Padding(5)
		.HAlign(HAlign_Fill)
		[
			SNew(STagWidget)
			.ModelInterface(InArgs._ModelInterface)
			.Tag(TagInstance)
			.JiraWidget(Issues)
		];

		TagCount++;
	}
}

#undef LOCTEXT_NAMESPACE
