// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

class FModelInterface;

class STagSectionWidget final : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(STagSectionWidget) { }
		SLATE_ARGUMENT(FModelInterface*, ModelInterface)			
		SLATE_ARGUMENT(TSharedPtr<SWindow>, ParentWindow)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
};
