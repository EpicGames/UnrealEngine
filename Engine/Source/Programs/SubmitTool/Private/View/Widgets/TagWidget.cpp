// Copyright Epic Games, Inc. All Rights Reserved.

#include "TagWidget.h"

#include "Models/ModelInterface.h"
#include "Models/Tag.h"
#include "Models/PreflightData.h"

#include "Widgets/Input/SHyperlink.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/STextComboBox.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Input/SHyperlink.h"

#include "View/SubmitToolStyle.h"
#include "View/Widgets/SJiraWidget.h"

#include "Logging/SubmitToolLog.h"
#include "SelectSourceControlUserWidget.h"
#include "SelectSourceControlGroupWidget.h"

#define LOCTEXT_NAMESPACE "TagWidget"

void STagWidget::Construct(const FArguments& InArgs)
{
	using namespace UE::Slate::Containers;
	this->ModelInterface = InArgs._ModelInterface;
	this->Tag = InArgs._Tag;

	PreflightListUI = MakeShared<UE::Slate::Containers::TObservableArray<TSharedPtr<FPreflightData>>>();

	PreflightUpdatedHandle = ModelInterface->AddPreflightUpdateCallback(FOnPreflightDataUpdated::FDelegate::CreateLambda([this](const TUniquePtr<FPreflightList>& InPFList, const TMap<FString, FPreflightData>& InUnlinkedPreflights)
		{
			if(!PreflightListUI->IsEmpty())
			{
				PreflightListUI->RemoveAt(0, PreflightListUI->Num(), EAllowShrinking::No);
			}

			for(const FPreflightData& PFData : InPFList->PreflightList)
			{
				PreflightListUI->Add(MakeShared<FPreflightData>(PFData));
			}

			for(const TPair<FString, FPreflightData>& Pair : InUnlinkedPreflights)
			{
				PreflightListUI->Add(MakeShared<FPreflightData>(Pair.Value));
			}

		}));

	TSharedPtr<SHorizontalBox> HorizontalBox;
	
	ChildSlot
		[
			SAssignNew(HorizontalBox, SHorizontalBox)
		];

	// add the check box
	HorizontalBox->AddSlot()
	.AutoWidth()
	.HAlign(HAlign_Fill)
	[
		SNew(SCheckBox)
		.IsFocusable(false)
		.IsEnabled_Static(&FModelInterface::GetInputEnabled)
		.IsChecked_Lambda([&tag = Tag]()->ECheckBoxState {return tag->IsEnabled() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
		.CheckBoxContentUsesAutoWidth(false)
		.OnCheckStateChanged(this, &STagWidget::OnCheckboxChangedEvent)
	];
	
	// add the label
	HorizontalBox->AddSlot()
	.Padding(0,3)
	.AutoWidth()
	.VAlign(VAlign_Center)
	[
		SNew(SButton)
		.IsEnabled_Static(&FModelInterface::GetInputEnabled)
		.ButtonStyle(FAppStyle::Get(), "InvisibleButton")
		.IsFocusable(false)
		.OnClicked_Raw(this, &STagWidget::OnLabelClick)
		[
			SNew(STextBlock)
			.MinDesiredWidth(70)
			.ColorAndOpacity_Lambda([this](){ 
			switch(Tag->GetTagState())
			{			
				case ETagState::Unchecked:
					if(Tag->GetCurrentValidationConfig(ModelInterface->GetFilesInCL()).bIsMandatory && !Tag->IsEnabled())
					{
						return FAppStyle::GetColor("ValidatorStateFail");
					}
					else
					{
						return FLinearColor::White;
					}
					break;

				case ETagState::Failed:
					return FAppStyle::GetColor("ValidatorStateFail");
					break;

				case ETagState::Success:
					if(Tag->IsEnabled())
					{
						return FAppStyle::GetColor("ValidatorStateSuccess");
					}
					else
					{
						return FLinearColor::White;
					}
					break;

				default:
					return FLinearColor::White;
			}})
			.Text_Lambda([this](){ return FText::FromString(FString::Printf(TEXT("%s%s"), *this->Tag->Definition.TagLabel, Tag->GetCurrentValidationConfig(ModelInterface->GetFilesInCL()).bIsMandatory ? TEXT(" *") : TEXT(""))); })
			.ToolTipText(FText::FromString(Tag->Definition.ToolTip))
			.Font(FAppStyle::Get().GetFontStyle("SmallFont"))
		]
	];

	// add the documentation
	HorizontalBox->AddSlot()
	.Padding(0)
	.AutoWidth()
	.VAlign(VAlign_Center)
	[
		SNew(SButton)
		.IsEnabled(!this->Tag->Definition.DocumentationUrl.IsEmpty())
		.ButtonStyle(FAppStyle::Get(), "InvisibleButton")
		.IsFocusable(false)
		.ToolTipText(FText::FromString(this->Tag->Definition.ToolTip))
		.Cursor(EMouseCursor::Hand)
		.OnClicked_Lambda([Url = this->Tag->Definition.DocumentationUrl]()
			{
				if (!Url.IsEmpty())
				{
					FPlatformProcess::LaunchURL(*Url, nullptr, nullptr);
				}

				return FReply::Handled();
			})
		[
			SNew(SImage)
				.Image(FSubmitToolStyle::Get().GetBrush("AppIcon.DocumentationHelp"))
		]
	];

	if(!Tag->Definition.InputType.Equals(TEXT("Boolean"), ESearchCase::IgnoreCase))
	{
		// add the text value
		HorizontalBox->AddSlot()
		.Padding(3, 0, 0, 0)
		.FillWidth(1.f)
		[
			SNew(SEditableTextBox)
			.MinDesiredWidth(256)
			.OnTextCommitted_Lambda([this](const FText& Text, ETextCommit::Type CommitType) { if(CommitType != ETextCommit::OnEnter) {
				if(Tag->Definition.InputSubType.Equals("Preflight", ESearchCase::IgnoreCase))
				{
					ModelInterface->RefreshPreflightInformation();
				}
				ModelInterface->ValidateCLDescription(); 
			}})
			.Text_Lambda([&tag = Tag] {return FText::FromString(tag->GetValuesText()); })
			.OnTextChanged(this, &STagWidget::OnTextChanged)
			.IsReadOnly_Lambda([] { return !FModelInterface::GetInputEnabled(); })
		];
	}

	// optionally add the perforce users input
	if (this->Tag->Definition.InputType.Equals("PerforceUser", ESearchCase::IgnoreCase))
	{
		if (this->Tag->Definition.InputSubType.Equals("SwarmApproved", ESearchCase::IgnoreCase))
		{
			HorizontalBox->AddSlot()
				.Padding(3, 0, 0, 0)
				.AutoWidth()
				[
					SNew(SBox)
						.MaxDesiredWidth(35)
						[
							SNew(SButton)
								.IsEnabled_Static(&FModelInterface::GetInputEnabled)
								.IsFocusable(true)
								.ContentPadding(-5)
								.ToolTipText(FText::FromString(TEXT("Refresh information from swarm")))
								.OnClicked_Lambda([this]()
									{
										ModelInterface->RefreshSwarmReview();
										return FReply::Handled();
									})
								[
									SNew(SBox)
										.VAlign(VAlign_Center)
										.HAlign(HAlign_Center)
										.MaxDesiredHeight(12)
										.MaxDesiredWidth(12)
										[
											SNew(SImage)
												.Image(FSubmitToolStyle::Get().GetBrush("AppIcon.Refresh"))
										]
								]
						]
				];
		}
		else if (this->Tag->Definition.InputSubType.Equals("Swarm", ESearchCase::IgnoreCase))
		{
			HorizontalBox->AddSlot()
				.Padding(3, 0, 0, 0)
				.AutoWidth()
				[
					SNew(SBox)
					.MinDesiredWidth(60)
					[
						SNew(SButton)
						.HAlign(HAlign_Center)
						.VAlign(VAlign_Center)
						.ContentPadding(-3)
						.OnClicked_Raw(this, &STagWidget::OnSwarmClick)
						.ToolTipText_Lambda([&ModelInterface = ModelInterface](){
							if (ModelInterface->HasSwarmReview())
							{
								return FText::FromString(TEXT("Show the review in Swarm"));
							}
							else
							{
								return FText::FromString(TEXT("Request a review using the current users as reviewers."));
							}})
						.IsEnabled_Lambda([&ModelInterface = ModelInterface]() { 
							if (!ModelInterface->HasSwarmReview())
							{
								return ModelInterface->GetInputEnabled() && ModelInterface->IsSwarmServiceValid() && !ModelInterface->GetSwarmService().Pin()->IsRequestRunning();
							}
							else
							{
								return ModelInterface->IsSwarmServiceValid() && !ModelInterface->GetSwarmService().Pin()->IsRequestRunning();
							}
						})
						[									
							SNew(STextBlock)
							.Justification(ETextJustify::Center)
							.Font(FAppStyle::Get().GetFontStyle("SmallFont"))
							.Text_Lambda([&ModelInterface = ModelInterface]()
							{
								if (ModelInterface->HasSwarmReview())
								{
									return FText::FromString(TEXT("Show Review"));
								}
								else
								{
									return FText::FromString(TEXT("Request Review"));
								}
							})
						]
					]
				];
		}

		HorizontalBox->AddSlot()
		.Padding(3, 0, 0, 0)
		.AutoWidth()
		[
			SNew(SBox)
			.MinDesiredWidth(60)
			[
				SNew(SSelectSourceControlUserWidget)
				.ButtonText(FText::FromString("User"))
				.ModelInterface(ModelInterface)
				.Tag(this->Tag)
				.IsEnabled_Lambda([this]() { return FModelInterface::GetInputEnabled();})
			]
		];

		HorizontalBox->AddSlot()
		.Padding(3, 0, 0, 0)
		.AutoWidth()
		[
			SNew(SBox)
			.MinDesiredWidth(60)
			[
				SNew(SSelectSourceControlGroupWidget)
				.ButtonText(FText::FromString("Group"))
				.ModelInterface(ModelInterface)
				.Tag(this->Tag)
				.IsEnabled_Lambda([this]() { return FModelInterface::GetInputEnabled();})
			]
		];
	}

	// optionally add the multi select input
	if (this->Tag->Definition.InputType.Equals("MultiSelect", ESearchCase::IgnoreCase))
	{
		this->SelectValues.Reset();

		for (int Idx = 0; Idx < this->Tag->Definition.SelectValues.Num(); Idx++)
		{
			this->SelectValues.Add(MakeShared<FString>(this->Tag->Definition.SelectValues[Idx]));
		}

		TSharedPtr<SVerticalBox> Contents = SNew(SVerticalBox);

		HorizontalBox->AddSlot()
			.Padding(3, 0, 0, 0)
			.AutoWidth()
			[
				SNew(SBox)
				.MinDesiredWidth(60)
				[
					SNew(SComboButton)
					.IsEnabled_Lambda([this](){ return FModelInterface::GetInputEnabled() && SelectValues.Num() > 0;})
					.OnMenuOpenChanged_Lambda([this](bool bIsOpen) {
						if (!bIsOpen)
						{
							ModelInterface->ValidateCLDescription();
						}
					})
					.ButtonContent()
					[
						SNew(STextBlock)
						.Font(FAppStyle::Get().GetFontStyle("SmallFont"))
						.Text(FText::FromString("Values"))
					]
					.MenuContent()
					[
						SAssignNew(Contents, SVerticalBox)
					]
				]
			];

		for (int i = 0; i < this->SelectValues.Num(); i++)
		{
			Contents.Get()->AddSlot()
				.AutoHeight()
				.Padding(2)
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						SNew(SCheckBox)
						.IsChecked_Lambda([this, i]() { return this->Tag->GetValues().Contains(*this->SelectValues[i]) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
						.IsEnabled_Static(&FModelInterface::GetInputEnabled)
						.OnCheckStateChanged_Lambda([this, i](ECheckBoxState InNewState)
						{
							 OnSelectedChangedFromMultiselect(SelectValues[i]);
						})
					]
					+SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						SNew(SButton)
						.ButtonStyle(FAppStyle::Get(), "InvisibleButton")
						.IsFocusable(false)
						.IsEnabled_Static(&FModelInterface::GetInputEnabled)
						.OnClicked_Lambda([this, i]() { OnSelectedChangedFromMultiselect(SelectValues[i]); return FReply::Handled(); })
						[
							SNew(STextBlock)
							.Justification(ETextJustify::Left)
							.MinDesiredWidth(60)
							.Text(FText::FromString(*this->SelectValues[i]))
						]
					]
				];
		}
	}

	// optionally add the jira issue
	if (this->Tag->Definition.InputType.Equals("JiraIssue", ESearchCase::IgnoreCase))
	{
		JiraWidget = InArgs._JiraWidget;
		HorizontalBox->AddSlot()
			.Padding(3, 0, 0, 0)
			.AutoWidth()
			[
				SNew(SBox)
				.MinDesiredWidth(60)
				[
					SNew(SButton)
					.IsEnabled_Lambda([this] { return ModelInterface->GetInputEnabled() && !ModelInterface->GetParameters().JiraParameters.bDisableJira; })
					.ToolTipText_Lambda([this] { return ModelInterface->GetParameters().JiraParameters.bDisableJira ? FText::FromString(TEXT("Jira integration is disabled")) : FText::FromString(TEXT("Browse Jira issues")); })
					.OnClicked(this, &STagWidget::OnJiraClick)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Justification(ETextJustify::Center)
						.Text(FText::FromString(TEXT("Jira")))
						.Font(FAppStyle::Get().GetFontStyle("SmallFont"))
					]
				]
			];
	}

	// optionally add the preflight
	if (this->Tag->Definition.TagLabel.Equals("Preflight", ESearchCase::IgnoreCase))
	{		
		TSharedPtr<SVerticalBox> Contents = SNew(SVerticalBox);

		HorizontalBox->AddSlot()
			.Padding(3, 0, 0, 0)
			.AutoWidth()
			[
				SNew(SBox)
				.MaxDesiredWidth(35)
				[
					SNew(SButton)
					.ContentPadding(-5)
					.IsEnabled_Static(&FModelInterface::GetInputEnabled)
					.IsFocusable(true)
					.ToolTipText(FText::FromString(TEXT("Refresh information from horde")))
					.OnClicked_Lambda([this]()
					{
						ModelInterface->RefreshPreflightInformation();
						return FReply::Handled();
					})
					[
						SNew(SBox)
						.VAlign(VAlign_Center)
						.HAlign(HAlign_Center)
						.MaxDesiredHeight(12)
						.MaxDesiredWidth(12)
						[
							SNew(SImage)
							.Image(FSubmitToolStyle::Get().GetBrush("AppIcon.Refresh"))
						]
					]
				]
			];

		HorizontalBox->AddSlot()
			.Padding(3, 0, 0, 0)
			.AutoWidth()
			[
				SNew(SBox)
				.MinDesiredWidth(60)
				[
					SNew(SComboButton)
					.IsEnabled_Lambda([this](){ return ModelInterface->GetInputEnabled() && PreflightListUI->Num() > 0; })
					.ForegroundColor_Raw(this, &STagWidget::GetPreflightGlobalColor)
					.ButtonContent()
					[
						SNew(STextBlock)
						.Font(FAppStyle::Get().GetFontStyle("SmallFont"))
						.ColorAndOpacity_Raw(this, &STagWidget::GetPreflightGlobalColor)
						.Text(FText::FromString("Preflight Jobs"))
					]
					.MenuContent()
					[
						SAssignNew(Contents, SVerticalBox)
					]					
				]
			];
			
		HorizontalBox->AddSlot()
			.Padding(3, 0, 0, 0)
			.AutoWidth()
			[
				SNew(SBox)
				.MaxDesiredWidth(35)
				[
					SNew(SButton)
					.ContentPadding(-5)
					.IsEnabled_Lambda([this]{ return Tag->GetValues().Num() > 0; })
					.IsFocusable(true)
					.ToolTipText(FText::FromString(TEXT("Open in browser")))
					.OnClicked_Lambda([this]()
					{
						for (const TSharedPtr<FPreflightData>& PFData : *PreflightListUI)
						{
							if (Tag->GetValues().ContainsByPredicate([&PFData](const FString& InTagValue) { return InTagValue.Contains(PFData->ID); }))
							{
								FPlatformProcess::LaunchURL(*FString::Printf(TEXT("%sjob/%s"), *ModelInterface->GetParameters().HordeParameters.HordeServerAddress, *PFData->ID), nullptr, nullptr);
							}
						}

						return FReply::Handled();
					})
					[
						SNew(SBox)
						.VAlign(VAlign_Center)
						.HAlign(HAlign_Center)
						.MaxDesiredHeight(12)
						.MaxDesiredWidth(12)
						[
							SNew(SImage)
							.Image(FSubmitToolStyle::Get().GetBrush("AppIcon.OpenExternal"))
						]
					]
				]
			];

		Contents.Get()->AddSlot()
			.AutoHeight()
			.Padding(2)
			[
				SNew(SListView<TSharedPtr<FPreflightData>>)
				.SelectionMode(ESelectionMode::Single)
				.ListItemsSource(PreflightListUI)
				.OnGenerateRow_Lambda([this](TSharedPtr<FPreflightData> InItem, const TSharedRef<STableViewBase>& OwnerTable)
				{
					return SNew(STableRow<TSharedPtr<FPreflightData>>, OwnerTable)
						.Padding(2.0f)
						[
							SNew(SBorder)
							.BorderBackgroundColor_Lambda([InItem]()
							{
								if(InItem->CachedResults.State == EPreflightState::Running)
								{
									return FAppStyle::GetColor("ValidatorStateNormal");
								}
								else if(InItem->CachedResults.Outcome == EPreflightOutcome::Success)
								{
									return FAppStyle::GetColor("ValidatorStateSuccess");
								}
								else if(InItem->CachedResults.Outcome == EPreflightOutcome::Warnings)
								{
									return FAppStyle::GetColor("ValidatorStateSuccess");
								}
								else if(InItem->CachedResults.Outcome == EPreflightOutcome::Failure || InItem->CachedResults.Outcome == EPreflightOutcome::Unspecified)
								{
									return FAppStyle::GetColor("ValidatorStateFail");
								}
								else
								{
									return FAppStyle::GetColor("ValidatorStateNormal");
								}
							})
							[
								SNew(SHorizontalBox)
								+SHorizontalBox::Slot()
								.AutoWidth()
								.VAlign(VAlign_Center)
								[
									SNew(SCheckBox)
									.Padding(2.f)
									.IsChecked_Lambda([this, InItem]() { return Tag->GetValues().ContainsByPredicate([InItem](const FString& InTagValue){ return InTagValue.Contains(InItem->ID); }) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
									.OnCheckStateChanged_Lambda([this, InItem](ECheckBoxState InNewState)
									{
										OnSelectedChangedFromMultiselect(MakeShared<FString>(InItem->ID));
									})
								]
								+SHorizontalBox::Slot()
								.AutoWidth()
								.VAlign(VAlign_Center)
								[
									SNew(SHyperlink)
									.Padding(2.f)
									.Style(FSubmitToolStyle::Get(), TEXT("NavigationHyperlink"))
									.Text(FText::FromString(*FString::Printf(TEXT("%s - %s"), *InItem->Name, *InItem->ID)))
									.ToolTipText(FText::FromString(*InItem->Name))
									.OnNavigate_Lambda([this, InItem]() { FPlatformProcess::LaunchURL(*FString::Printf(TEXT("%sjob/%s"), *ModelInterface->GetParameters().HordeParameters.HordeServerAddress, *InItem->ID), nullptr, nullptr); })
								]
							]
						];
				})					
			];
			
		HorizontalBox->AddSlot()
			.Padding(3, 0, 0, 0)
			.AutoWidth()
			[
				SNew(SBox)
				.MinDesiredWidth(60)
				[
					SNew(SButton)
					.OnClicked_Lambda([this]{ ModelInterface->RequestPreflight(); return FReply::Handled(); })
					.IsEnabled_Lambda([&ModelInterface = ModelInterface] { return FModelInterface::GetInputEnabled() && !ModelInterface->IsPreflightQueued() && !ModelInterface->IsPreflightRequestInProgress(); })
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Justification(ETextJustify::Center)
						.Font(FAppStyle::Get().GetFontStyle("SmallFont"))
						.Text_Lambda([&ModelInterface = ModelInterface]()
							{
								if (ModelInterface->IsPreflightRequestInProgress())
								{
									if(ModelInterface->IsShelfReady())
									{ 
										return FText::FromString(TEXT("Requesting..."));
									}
									else
									{ 
										return FText::FromString(TEXT("Waiting for Perforce"));
									}
								}
								else if (ModelInterface->IsPreflightQueued())
								{
									return FText::FromString(TEXT("Queued"));
								}
								else
								{
									return FText::FromString(TEXT("Queue"));
								}
							})
						.ToolTipText_Lambda([&ModelInterface = ModelInterface]()
							{
								if (ModelInterface->IsPreflightRequestInProgress())
								{
									return FText::FromString(TEXT("Preflight request in progress..."));
								}
								else if (ModelInterface->IsPreflightQueued())
								{
									return FText::FromString(TEXT("Preflight will start once the CL has run local validations"));
								}
								else
								{
									return FText::FromString(TEXT("Enqueues a Preflight for when local validations have finished"));
								}
							})
					]
				]
			];
			
		HorizontalBox->AddSlot()
			.Padding(3, 0, 0, 0)
			.AutoWidth()
			[
				SNew(SBox)
				.MinDesiredWidth(60)
				[
					SNew(SButton)
					.OnClicked_Lambda([this]{
						ModelInterface->RequestPreflight(true); 
						return FReply::Handled();
					})
					.IsEnabled_Lambda([&ModelInterface = ModelInterface] { return FModelInterface::GetInputEnabled() && !ModelInterface->IsPreflightRequestInProgress(); })
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Justification(ETextJustify::Center)
						.Font(FAppStyle::Get().GetFontStyle("SmallFont"))
						.Text(FText::FromString(TEXT("Start")))
						.ToolTipText(FText::FromString(TEXT("Starts a Preflight in Horde")))
					]
				]
			];
	}
}

STagWidget::~STagWidget()
{
	if(ModelInterface != nullptr)
	{
		ModelInterface->RemovePreflightUpdateCallback(PreflightUpdatedHandle);
	}
}

void STagWidget::OnSelectedChangedFromMultiselect(TSharedPtr<FString> Value)
{
	if (Value->IsEmpty())
	{
		return;
	}

	TArray<FString> Values = this->Tag->GetValues();

	if(!Values.Contains(*Value))
	{
		Values.Add(FString(*Value));
	}
	else
	{
		while (Values.Contains(*Value))
		{
			Values.Remove(FString(*Value));
		}
	}

	ModelInterface->SetTagValues(*Tag, Values);
	
	if(Values.Num() == 0)
	{
		ModelInterface->RemoveTag(*Tag);
	}
}
FSlateColor STagWidget::GetPreflightGlobalColor() const
{
	bool bHasErrors = false;
	bool bHasWarnings = false;
	bool bHasRunning = false;
	bool bHasSuccess = false;


	const TUniquePtr<FPreflightList>& PreflightDataList = ModelInterface->GetPreflightData();
	if(!PreflightDataList.IsValid())
	{
		return FSlateColor::UseForeground();
	}

	for(const FString& SelectedPreflight : Tag->GetValues())
	{
		const FPreflightData* FoundData = PreflightDataList->PreflightList.FindByPredicate([&SelectedPreflight](const FPreflightData& InData) { return InData.ID == SelectedPreflight; });

		if(FoundData != nullptr)
		{
			if(FoundData->CachedResults.State == EPreflightState::Running)
			{
				bHasRunning = true;
			}
			else if(FoundData->CachedResults.Outcome == EPreflightOutcome::Warnings)
			{
				bHasWarnings = true;
			}
			else if(FoundData->CachedResults.Outcome == EPreflightOutcome::Failure || FoundData->CachedResults.Outcome == EPreflightOutcome::Unspecified)
			{
				bHasErrors = true;
			}
			else if(FoundData->CachedResults.Outcome == EPreflightOutcome::Success)
			{
				bHasSuccess = true;
			}
		}
	}

	if(bHasErrors)
	{
		return FAppStyle::GetColor("ValidatorStateFail");
	}
	else if(bHasRunning)
	{
		return FAppStyle::GetColor("ValidatorStateNormal");
	}
	else if(bHasWarnings)
	{
		return FAppStyle::GetColor("ValidatorStateWarning");
	}
	else if(bHasSuccess)
	{
		return FAppStyle::GetColor("ValidatorStateSuccess");
	}
	else
	{
		return FSlateColor::UseForeground();
	}
}

void STagWidget::OnCheckboxChangedEvent(ECheckBoxState newState)
{
	if (newState == ECheckBoxState::Checked)
	{
		ModelInterface->ApplyTag(*Tag);
	}
	else
	{
		ModelInterface->RemoveTag(*Tag);
	}

	ModelInterface->ValidateCLDescription();
}

void STagWidget::OnTextChanged(const FText& InText)
{
	if (!Tag->GetValuesText().Equals(InText.ToString(), ESearchCase::IgnoreCase))
	{
		ModelInterface->SetTagValues(*Tag, InText.ToString());
	}
}

FReply STagWidget::OnLabelClick()
{
	OnCheckboxChangedEvent(Tag->IsEnabled() ? ECheckBoxState::Unchecked : ECheckBoxState::Checked);
	return FReply::Handled();
}

FReply STagWidget::OnJiraClick()
{
	JiraWidget->Open(Tag);
	return FReply::Handled();
}

FReply STagWidget::OnPreflightClick()
{
	ModelInterface->RequestPreflight();
	return FReply::Handled();
}

FReply STagWidget::OnSwarmClick()
{
	if (ModelInterface->HasSwarmReview())
	{
		ModelInterface->ShowSwarmReview();
	}
	else
	{
		ModelInterface->RequestSwarmReview(Tag->GetValues());
	}

	return FReply::Handled();
}
#undef LOCTEXT_NAMESPACE
