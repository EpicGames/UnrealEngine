// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Styling/SlateTypes.h"
#include "Logic/SwarmService.h"
#include "Containers/ObservableArray.h"

class FModelInterface;
class FTag;
class SJiraWidget;
struct FPreflightData;

DECLARE_DELEGATE_OneParam(FOnCheckboxChangedSignature, ECheckBoxState)

class STagWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(STagWidget) {}
		SLATE_ARGUMENT(FModelInterface*, ModelInterface)
		SLATE_ARGUMENT(TSharedPtr<SJiraWidget>, JiraWidget)
		SLATE_ARGUMENT(const FTag*, Tag)
		SLATE_EVENT(FOnCheckboxChangedSignature, OnCheckboxChanged)
	SLATE_END_ARGS()

	virtual ~STagWidget();

	void Construct(const FArguments& InArgs);

private:	
	void OnTextChanged(const FText& InText);
	void OnCheckboxChangedEvent(ECheckBoxState newState);
	void OnSelectedChangedFromMultiselect(TSharedPtr<FString> Value);
	FSlateColor GetPreflightGlobalColor() const;

	FReply OnLabelClick();	
	FReply OnJiraClick();
	FReply OnPreflightClick();
	FReply OnSwarmClick();

	FModelInterface* ModelInterface;
	const FTag* Tag;
	FDelegateHandle PreflightUpdatedHandle;
	UE::Slate::Containers::TObservableArray<TSharedPtr<FString>> SelectValues;
	TSharedPtr<UE::Slate::Containers::TObservableArray<TSharedPtr<FPreflightData>>> PreflightListUI;


	TSharedPtr<SJiraWidget> JiraWidget;


};