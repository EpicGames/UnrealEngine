// Copyright Epic Games, Inc. All Rights Reserved.

#include "ValidatorsWidget.h"

#include "HAL/FileManager.h"
#include "Misc/Paths.h"
#include "Misc/FileHelper.h"

#include "Widgets/Layout/SExpandableArea.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Text/STextBlock.h"

#include "Models/SubmitToolUserPrefs.h"
#include "Models/JiraIssue.h"
#include "Styling/StyleColors.h"

#define LOCTEXT_NAMESPACE "ValidatorStatesWidget"

namespace ValidatorsColumns
{
	static const FName Type("Type");
	static const FName Name("Name");
	static const FName Required("Required");
	static const FName Status("Status");
	static const FName Actions("Actions");
};

void SValidatorsWidget::Construct(const FArguments& InArgs)
{
	Columns = {
		FValidatorColumn(ValidatorsColumns::Name, 1, true, [](TWeakPtr<const FValidatorBase> In) {return In.Pin()->GetValidatorName(); }),
		FValidatorColumn(ValidatorsColumns::Required, 130, false, [](TWeakPtr<const FValidatorBase> In) {return In.Pin()->Definition->IsRequired ? TEXT("Yes") : TEXT("No"); }),
		FValidatorColumn(ValidatorsColumns::Status, 150, false, [](TWeakPtr<const FValidatorBase> In) {return In.Pin()->GetStatusText(); }),
		FValidatorColumn(ValidatorsColumns::Actions, 200, false, nullptr),
	};

	ModelInterface = InArgs._ModelInterface;
	OnViewLog = InArgs._OnViewLog;

	ChildSlot
	[
		BuildValidatorsView()
	];

	OnFilesRefreshed = ModelInterface->FileRefreshedCallback.AddLambda([this] { ReevaluateValidatorSection(); });
	OnPrepareSubmit = FModelInterface::OnStateChanged.AddLambda([this] (const ESubmitToolAppState InFrom, const ESubmitToolAppState InTo)
		{ 
			if (InTo == ESubmitToolAppState::Submitting || InFrom == ESubmitToolAppState::Submitting)
			{
				RefreshValidatorView(InTo == ESubmitToolAppState::Submitting);
			}
		});
}

SValidatorsWidget::~SValidatorsWidget()
{
	ModelInterface->FileRefreshedCallback.Remove(OnFilesRefreshed);
	FModelInterface::OnStateChanged.Remove(OnPrepareSubmit);
}

void SValidatorsWidget::ReevaluateValidatorSection()
{
	for (const TPair<FName, TArray<TWeakPtr<const FValidatorBase>>>& Pair : Validators)
	{
		for (size_t i = 0;i < Pair.Value.Num();++i)
		{
			FName Section;
			
			if (!Pair.Value[i].Pin()->Definition->bIsDisabled && !Pair.Value[i].Pin()->IsRelevantToCL())
			{
				Section = InactiveSection;
			}
			else
			{
				Section = ActiveSection;
			}

			if (Section != Pair.Key)
			{
				Validators.FindOrAdd(Section).Add(Pair.Value[i]);
				Validators[Pair.Key].RemoveAt(i);
				--i;
			}
		}
	}
}

TSharedRef<SWidget> SValidatorsWidget::BuildValidatorsView(bool bListPreSubmitOperations)
{
	Validators.Reset();

	TArray<TWeakPtr<const FValidatorBase>> ValidatorsArray = bListPreSubmitOperations ? ModelInterface->GetPreSubmitOperations() : ModelInterface->GetValidators();
	ActiveSection = bListPreSubmitOperations ? FName(TEXT("Pre-Submit Operations")) : FName(TEXT("Active Validations"));
	InactiveSection = bListPreSubmitOperations ? FName(TEXT("Inactive Pre-Submit Operations (Not Applicable to your CL)")) : FName(TEXT("Inactive Validators (Not Applicable to your CL)"));

	TSharedRef<SVerticalBox> VBox = SNew(SVerticalBox);
	
	for(const TWeakPtr<const FValidatorBase>& Validator : ValidatorsArray)
	{
		FName DisplayName;

		if (!Validator.Pin()->Definition->bIsDisabled && !Validator.Pin()->IsRelevantToCL())
		{
			DisplayName = InactiveSection;
		}
		else
		{
			DisplayName = ActiveSection;
		}

		Validators.FindOrAdd(DisplayName).Add(Validator);
	}

	AddValidatorSectionToWidget(VBox, ActiveSection, bListPreSubmitOperations);
	AddValidatorSectionToWidget(VBox, InactiveSection, bListPreSubmitOperations);

	return VBox;
}

void SValidatorsWidget::AddValidatorSectionToWidget(TSharedRef<SVerticalBox> InVBox, const FName& InDisplayName, bool InbListPreSubmitOperations)
{
	bool bInitiallyCollapsed = false;
	if (!(InbListPreSubmitOperations && InDisplayName.IsEqual(ActiveSection)))
	{
		bInitiallyCollapsed = !FSubmitToolUserPrefs::Get()->UISectionExpandState.FindOrAdd(InDisplayName, false);
	}

	InVBox->AddSlot()
	.Padding(FMargin(0.f, 2.f))
	.AutoHeight()
	.AttachWidget
	(
		SNew(SExpandableArea)
		.InitiallyCollapsed(bInitiallyCollapsed)
		.BorderImage(FAppStyle::Get().GetBrush("Brushes.Header"))
		.BodyBorderImage(FAppStyle::Get().GetBrush("Brushes.Recessed"))
		.OnAreaExpansionChanged_Lambda([DisplayName = InDisplayName](bool bExpanded) { FSubmitToolUserPrefs::Get()->UISectionExpandState.FindOrAdd(DisplayName, false) = bExpanded; })
		.BorderBackgroundColor_Lambda([DisplayName = InDisplayName, this]
			{
				if (DisplayName.IsEqual(InactiveSection))
				{
					return FAppStyle::GetColor("ValidatorStateSuccess");
				}

				bool bRunning = false;
				bool bValid = true;
				for (const TWeakPtr<const FValidatorBase>& Validator : Validators.FindOrAdd(DisplayName))
				{
					bRunning |= Validator.Pin()->GetIsRunning();
					bValid = bValid && Validator.Pin()->GetHasPassed();

					if (Validator.Pin()->GetValidatorState() == EValidationStates::Failed ||
						Validator.Pin()->GetValidatorState() == EValidationStates::Timeout)
					{
						return FAppStyle::GetColor("ValidatorStateFail");
					}
				}

				if (bValid)
				{
					return FAppStyle::GetColor("ValidatorStateSuccess");
				}

				if (bRunning)
				{
					return FAppStyle::GetColor("ValidatorStateWarning");
				}

				return FAppStyle::GetColor("ValidatorStateNormal"); })
		.HeaderPadding(FMargin(4.0f, 2.0f))
		.Padding(1.0f)
		.AllowAnimatedTransition(true)
		.HeaderContent()
		[
			SNew(STextBlock)
				.Text_Lambda([DisplayName = InDisplayName] {return FText::FromName(DisplayName); })
		]
		.BodyContent()
		[
			SNew(SBox)
				.Padding(2.5)
				[
					SAssignNew(ValidatorsListView, SListView<TWeakPtr<const FValidatorBase>>)
						.SelectionMode(ESelectionMode::None)
						.ListItemsSource(&Validators.FindOrAdd(InDisplayName))
						.HeaderRow(ConstructHeadersRow(InDisplayName))
						.OnGenerateRow(this, &SValidatorsWidget::GenerateRow)
						.IsFocusable(false)
				]
		]
	);
}

void SValidatorsWidget::RefreshValidatorView(bool bListPreSubmitOperations)
{
	ChildSlot.DetachWidget();
	ChildSlot.AttachWidget(BuildValidatorsView(bListPreSubmitOperations));
}

TSharedRef<SHeaderRow> SValidatorsWidget::ConstructHeadersRow(const FName& GroupName)
{
	TSharedRef<SHeaderRow> HeaderRow = SNew(SHeaderRow);

	for (size_t i = 0;i<Columns.Num();++i)
	{
		SHeaderRow::FColumn::FArguments args = SHeaderRow::Column(Columns[i].Name);
		args.DefaultLabel(FText::FromName(Columns[i].Name));

		if(Columns[i].SortingFunc != nullptr)
		{
			args.SortMode(this, &SValidatorsWidget::GetSortMode, Columns[i].Name)
				.InitialSortMode(EColumnSortMode::Ascending)
				.OnSort_Lambda([&GroupName, this](EColumnSortPriority::Type InSortPriority, const FName& InColumnId, EColumnSortMode::Type InSortDirection) {
					OnColumnSort(GroupName, InSortPriority, InColumnId, InSortDirection);
				});
		}

		if(Columns[i].bIsFill)
		{
			args.FillWidth(Columns[i].Width);
		}
		else
		{
			args.FixedWidth(Columns[i].Width);
		}

		HeaderRow->AddColumn(args);
	}

	return HeaderRow;
}

class SValidatorNode : public SMultiColumnTableRow<TWeakPtr<const FValidatorBase>>
{
public:
	SLATE_BEGIN_ARGS(SValidatorNode) {}
		SLATE_ARGUMENT(SValidatorsWidget*, ParentWidget)
		SLATE_ARGUMENT(FModelInterface*, ModelInterface)
		SLATE_ARGUMENT(bool, IsRelevantToCL)
		SLATE_EVENT(FOnViewValidatorLog, ViewLogCallback)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTable, TWeakPtr<const FValidatorBase> InNode)
	{
		ModelInterface = InArgs._ModelInterface;
		ParentWidget = InArgs._ParentWidget;
		bIsRelevantToCL = InArgs._IsRelevantToCL;
		Node = InNode;
		ViewLogCallback = InArgs._ViewLogCallback;
		Options = TMap<TSharedPtr<FString>, TArray<TSharedPtr<FString>>>();
		for(const TPair<FString, TMap<FString, FString>>& Pair : Node.Pin()->GetValidatorOptions())
		{
			TArray<TSharedPtr<FString>> Values;
			for (const TPair<FString, FString>& Value : Pair.Value)
			{
				Values.Add(MakeShared<FString>(Value.Key));
			}

			Options.Add(MakeShared<FString>(Pair.Key), Values);
		}

		SMultiColumnTableRow<TWeakPtr<const FValidatorBase>>::Construct(
			FSuperRowType::FArguments()
			.Padding(1.0f),
			InOwnerTable
		);
	}
	
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& InColumnName) override
	{
		TSharedRef<SBorder> box = SNew(SBorder).VAlign(EVerticalAlignment::VAlign_Center);

		if(InColumnName == ValidatorsColumns::Name)
		{
			const FString NameStr = Node.Pin()->Definition->IsRequired || Node.Pin()->Definition->bRequireCompleteWhenOptional ? Node.Pin()->GetValidatorName() : Node.Pin()->GetValidatorName() + TEXT(" - Optional");
			TSharedRef<STextBlock> Name = SNew(STextBlock)
				.Text(FText::FromString(NameStr))
				.ToolTipText(FText::FromString(Node.Pin()->Definition->ToolTip));

			TSharedRef<SHorizontalBox> Horizontal = SNew(SHorizontalBox);

			Horizontal->AddSlot()
			.VAlign(VAlign_Center)
			.FillWidth(1.0f)
			[
				Name
			];

			if(Options.Num() != 0)
			{
				for (const TPair<TSharedPtr<FString>, TArray<TSharedPtr<FString>>>& OptionItem : Options)
				{
					TSharedRef<STextBlock> ComboContent = SNew(STextBlock)
						.Text_Lambda([this, OptionItem]() {	return FText::FromString(Node.Pin()->GetSelectedOptionKey(*OptionItem.Key));	})
						.Font(FAppStyle::GetFontStyle("SmallFont"))
						.ColorAndOpacity_Lambda([this, OptionItem]
							{
								if (Node.Pin()->GetOptionType(*OptionItem.Key) == EValidatorOptionType::FilePath)
								{
									return IFileManager::Get().FileExists(*Node.Pin()->GetSelectedOptionValue(*OptionItem.Key))
										? FStyleColors::White
										: FStyleColors::Error;
								}

								return FStyleColors::White;
							});

					Horizontal->AddSlot()
					.AutoWidth()
					[
						SNew(SBox)
						.MinDesiredWidth(120)
						[
							SNew(SComboBox<TSharedPtr<FString>>)
							.IsEnabled(OptionItem.Value.Num() > 1)
							.OptionsSource(&OptionItem.Value)
							.OnGenerateWidget_Lambda([this, OptionItem](TSharedPtr<FString> ChoiceEntry)
								{
									return SNew(STextBlock)
										.Text(FText::FromString(*ChoiceEntry))
										.ToolTipText(FText::FromString(Node.Pin()->GetValidatorOptions()[*OptionItem.Key][*ChoiceEntry]))
										.Font(FAppStyle::GetFontStyle("SmallFont"))
										.ColorAndOpacity_Lambda([this, OptionItem, ChoiceEntry]
											{
												if (Node.Pin()->GetOptionType(*OptionItem.Key) == EValidatorOptionType::FilePath)
												{
													return IFileManager::Get().FileExists(*Node.Pin()->GetValidatorOptions()[*OptionItem.Key][*ChoiceEntry])
														? FStyleColors::White
														: FStyleColors::Error;
												}

												return FStyleColors::White;
											});
								})
							.OnSelectionChanged_Lambda([this, OptionItem](TSharedPtr<FString> NewChoice, ESelectInfo::Type SelectType) {
									const_cast<FValidatorBase*>(Node.Pin().Get())->SetSelectedOption(*OptionItem.Key, *NewChoice);
								})
									[
										ComboContent
									]
						]
					];
				}

				box->SetContent(Horizontal);
			}
			else
			{
				box->SetContent(Name);
			}
		}
		else if (InColumnName == ValidatorsColumns::Required)
		{
			FText Text;			
			if (Node.Pin()->Definition->IsRequired)
			{
				Text = FText::FromString(TEXT("Success"));
			}
			else if(Node.Pin()->Definition->bRequireCompleteWhenOptional)
			{
				Text = FText::FromString(TEXT("Complete"));
			}
			else
			{
				Text = FText::FromString(TEXT("Optional"));
			}

			box->SetContent(
				SNew(STextBlock)
				.Text(Text)
			);
		}
		else if(InColumnName == ValidatorsColumns::Status)
		{
			box->SetContent(
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "InvisibleButton")
				.IsFocusable(false)
				.ToolTipText(FText::FromString(TEXT("View Log")))
				.OnClicked_Lambda([this] {	ViewLogCallback.ExecuteIfBound(Node.Pin()); return FReply::Handled();	})
				[
					SNew(STextBlock)
					.Cursor(EMouseCursor::Hand)
					.Text_Lambda([this]() { return FText::FromString(Node.Pin()->GetStatusText()); })
					.ColorAndOpacity_Lambda([this]{
						switch(Node.Pin()->GetValidatorState())
						{
						case EValidationStates::Failed:
						case EValidationStates::Timeout:
							return Node.Pin()->Definition->IsRequired ? FAppStyle::GetColor("ValidatorStateFail") : FAppStyle::GetColor("ValidatorStateWarning");
							break;
						case EValidationStates::Valid:
						case EValidationStates::Skipped:
						case EValidationStates::Not_Applicable:
							return FAppStyle::GetColor("ValidatorStateSuccess");
							break;
						case EValidationStates::Running:
							return FAppStyle::GetColor("ValidatorStateWarning");
							break;
						case EValidationStates::Disabled:
							return FAppStyle::GetColor("ValidatorStateDisabled");
							break;
						case EValidationStates::Not_Run:
						default:
							return FAppStyle::GetColor("ValidatorStateNormal");
							break;
						}})
				]
			);
		}
		else if(InColumnName == ValidatorsColumns::Actions)
		{
			box->SetContent(
				SNew(SHorizontalBox)+SHorizontalBox::Slot()
				.Padding(3,0,0,0)
				[
					SNew(SButton)
					.IsEnabled_Lambda([this]() {return (!Node.Pin()->GetIsRunningOrQueued() || Node.Pin()->Definition->bCanBeStoppedByUser) && ModelInterface->GetInputEnabled() && Node.Pin()->GetValidatorState() != EValidationStates::Disabled && bIsRelevantToCL; })
					.HAlign(EHorizontalAlignment::HAlign_Center)
					.VAlign(EVerticalAlignment::VAlign_Center)
					.ToolTipText_Lambda([this]
					{
						if (Node.Pin()->GetIsRunningOrQueued())
						{
							return FText::FromString(Node.Pin()->Definition->bCanBeStoppedByUser ? TEXT("Stops this validator.") : TEXT("Stopping this validator is disabled"));
						}
						else
						{
							return FText::FromString(TEXT("Runs this validator. To force run, hold CTRL."));
						}
					})						
					.OnClicked_Lambda([this]()
					{
						if (Node.Pin()->GetIsRunningOrQueued())
						{
							ModelInterface->CancelValidations(Node.Pin()->GetValidatorNameId(), true);
						}
						else
						{
							ModelInterface->ValidateSingle(Node.Pin()->GetValidatorNameId(), FSlateApplication::Get().GetModifierKeys().IsControlDown());
						}
						return FReply::Handled();
					})
					[
						SNew(STextBlock)
						.Justification(ETextJustify::Center)
						.Text_Lambda([this]() 
						{ 
							if (Node.Pin()->GetIsRunningOrQueued())
							{
								return FText::FromString("Stop");
							}
							else
							{
								if(FSlateApplication::Get().GetModifierKeys().IsControlDown())
								{
									return FText::FromString("Force Run");
								}
								else
								{
									return FText::FromString("Run");
								}
							}
						})
						.Font(FAppStyle::Get().GetFontStyle("SmallFont"))
					]
				]
				+SHorizontalBox::Slot()
				.Padding(3, 0, 3, 0)
				[
					SNew(SButton)
					.IsEnabled_Lambda([this] { return Node.Pin()->Definition->bCanBeDisabledByUser && ModelInterface->GetInputEnabled(); })
					.HAlign(EHorizontalAlignment::HAlign_Center)
					.VAlign(EVerticalAlignment::VAlign_Center)
					.OnClicked_Lambda([this]() 
					{
						if (Node.Pin()->Definition->bIsDisabled)
						{
							ModelInterface->ToggleValidator(Node.Pin()->GetValidatorNameId());
							ParentWidget->ReevaluateValidatorSection();
							return FReply::Handled();
						}

						bool bConfirmed = ParentWidget->bDontShowDisablePopup;

						if (!bConfirmed)
						{
							FText Title = FText::FromString(FString::Printf(TEXT("Disable %s"), *Node.Pin()->GetValidatorName()));
							FText MessageBody = FText::FromString(FString::Printf(TEXT("You are <RichTextBlock.TextHighlight>disabling %s.</>\n\nThis means some potentially necessary checks may not be performed, and %s will be recorded in this CL as <RichTextBlock.Red>DISABLED</>.\n\nIf this CL causes breakages you will be asked to provide justifications for having disabled validations.\n"), *Node.Pin()->GetValidatorName(), *Node.Pin()->GetValidatorName()));

							TSharedRef<SWidget> AdditionalContent = 
								SNew(SCheckBox)
								.OnCheckStateChanged_Lambda([this](ECheckBoxState InChkState) { ParentWidget->bDontShowDisablePopup = InChkState == ECheckBoxState::Checked; })
								[
									SNew(STextBlock)
									.Text(FText::FromString(TEXT("Don't show again in this submit tool instance")))
								];

							bConfirmed = FDialogFactory::ShowConfirmDialog(Title, MessageBody, AdditionalContent) == EDialogFactoryResult::Confirm;
						}

						if (bConfirmed)
						{
							ModelInterface->ToggleValidator(Node.Pin()->GetValidatorNameId());
							ParentWidget->ReevaluateValidatorSection();
						}						

						return FReply::Handled();
					})
					[
						SNew(STextBlock)
						.Justification(ETextJustify::Center)
						.Text_Lambda([this]{ return Node.Pin()->Definition->bIsDisabled ? FText::FromString("Enable") : FText::FromString("Disable"); })
						.Font(FAppStyle::Get().GetFontStyle("SmallFont"))
					]

				]);	
		}

		return box;
	}


	TMap<TSharedPtr<FString>, TArray<TSharedPtr<FString>>> Options;

	TWeakPtr<const FValidatorBase> Node;
	FModelInterface* ModelInterface;
	SValidatorsWidget* ParentWidget;
	bool bIsRelevantToCL = false;
	FOnViewValidatorLog ViewLogCallback;
};

TSharedRef<ITableRow> SValidatorsWidget::GenerateRow(TWeakPtr<const FValidatorBase> InItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SValidatorNode, OwnerTable, InItem)
		.ParentWidget(this)
		.ModelInterface(ModelInterface)
		.IsRelevantToCL(InItem.Pin()->IsRelevantToCL())
		.ViewLogCallback(OnViewLog);
}

void SValidatorsWidget::OnColumnSort(const FName& GroupName, EColumnSortPriority::Type InSortPriority, const FName& InColumnId, EColumnSortMode::Type InSortDirection)
{
	SortMode = InSortDirection;
	SortByColumn = InColumnId;

	for (const FValidatorColumn& Column : Columns)
	{
		if(!Column.Name.IsEqual(InColumnId))
		{
			continue;
		}

		if(Column.SortingFunc == nullptr)
		{
			break;
		}

		if(InSortDirection == EColumnSortMode::Ascending)
		{
			Algo::SortBy(Validators[GroupName], Column.SortingFunc);
		}
		else
		{
			Algo::Reverse(Validators[GroupName]);
		}
		
		break;
	}
	
	ValidatorsListView->RequestListRefresh();
}

EColumnSortMode::Type SValidatorsWidget::GetSortMode(const FName ColumnId) const
{
	if(ColumnId == SortByColumn)
	{
		return SortMode;
	}

	return EColumnSortMode::None;
}

#undef LOCTEXT_NAMESPACE