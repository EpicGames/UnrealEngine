// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SListView.h"

#include "Models/ModelInterface.h"

DECLARE_DELEGATE_OneParam(FOnViewValidatorLog, TSharedPtr<const FValidatorBase>)

struct FValidatorColumn
{
	FValidatorColumn(FName InName, float InWidth, bool InbIsFill, const TFunction<FString(TWeakPtr<const FValidatorBase>)> InSortingFunc)
		: SortingFunc(InSortingFunc)
	{
		Name = InName;
		Width = InWidth;
		bIsFill = InbIsFill;
	}

	FName Name;
	float Width;
	bool bIsFill;
	const TFunction<FString(TWeakPtr<const FValidatorBase>)> SortingFunc;
};

class SValidatorsWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SValidatorsWidget) {}
		SLATE_ARGUMENT(FModelInterface*, ModelInterface)
		SLATE_EVENT(FOnViewValidatorLog, OnViewLog)
	SLATE_END_ARGS()
	
	void Construct(const FArguments& InArgs);
	virtual ~SValidatorsWidget();

	void ReevaluateValidatorSection();
	bool bDontShowDisablePopup = false;
private:

	TSharedRef<SWidget> BuildValidatorsView(bool bListPreSubmitOperations = false);
	void AddValidatorSectionToWidget(TSharedRef<SVerticalBox> InVBox, const FName& InDisplayName, bool InbListPreSubmitOperations);
	void RefreshValidatorView(bool bListPreSubmitOperations = false);

	FModelInterface* ModelInterface;
	
	TSharedPtr<SListView<TWeakPtr<const FValidatorBase>>> ValidatorsListView;
	TSharedPtr<SListView<TWeakPtr<const FValidatorBase>>> PreSubmitListView;
	
	TMap<FName, TArray<TWeakPtr<const FValidatorBase>>> Validators;
	TArray<FValidatorColumn> Columns;

	TSharedRef<SHeaderRow> ConstructHeadersRow(const FName& GroupName);
	TSharedRef<ITableRow> GenerateRow(TWeakPtr<const FValidatorBase> InItem, const TSharedRef<STableViewBase>& OwnerTable);
	
	void OnColumnSort(const FName& GroupName, EColumnSortPriority::Type InSortPriority, const FName& InColumnId, EColumnSortMode::Type InSortDirection);
	EColumnSortMode::Type GetSortMode(const FName ColumnId) const;

	FName ActiveSection;
	FName InactiveSection;

	FName SortByColumn;
    EColumnSortMode::Type SortMode;
	FOnViewValidatorLog OnViewLog;
	FDelegateHandle OnFilesRefreshed;
	FDelegateHandle OnPrepareSubmit;
};
