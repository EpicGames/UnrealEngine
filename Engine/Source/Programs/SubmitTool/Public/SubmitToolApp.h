// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Tasks/Task.h"

class ISTSourceControlService;
class FModelInterface;

/**
 * Run the SubmitTool .
 */
int RunSubmitTool(const TCHAR* Commandline);
UE::Tasks::TTask<TUniquePtr<FModelInterface>> SyncConfigAndInitializeModelInterface();
FString GetUserPrefsPath();
