// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RPCLibraryBase.h"

#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "Serialization/JsonWriter.h"

#include "CommandLine/CmdLineParameters.h"
#include "Logic/Services/SourceControl/SubmitToolPerforce.h"
#include "Models/ModelInterface.h"
#include "Parameters/SubmitToolParametersBuilder.h"

#include "RPCLibrary.generated.h"

UCLASS(MinimalAPI)
class USubmitToolRPCLibrary : public URPCLibraryBase
{
	GENERATED_BODY()
public:
	USubmitToolRPCLibrary();
	virtual void BeginDestroy() override
	{
		if (ModelInterface && OnSingleValidatorFinishedHandle.IsValid())
		{
			ModelInterface->RemoveValidationFinishedCallback(OnSingleValidatorFinishedHandle);
		}
		Super::BeginDestroy();
	}
	void SetModelInterface(FModelInterface* InModelInterface) { ModelInterface = InModelInterface; }
	void RegisterLibraryRPCs();
private:
	FModelInterface* ModelInterface;

	TMap<FString, EValidationStates> ValidatorStates;

	FDelegateHandle OnSingleValidatorFinishedHandle;
	void OnSingleValidatorFinished(const FValidatorBase& InValidator);
#if WITH_RPC_REGISTRY
	bool HandleRunValidatorRoute(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete);
	bool HandleGetValidatorStateRoute(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete);
	bool HandleSetChangelistDescriptionRoute(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete);
	bool HandleCloseApplicationRoute(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete);
#endif // WITH_RPC_REGISTRY
};