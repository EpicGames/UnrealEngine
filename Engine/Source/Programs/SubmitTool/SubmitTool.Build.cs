// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using UnrealBuildTool;
public class SubmitTool : ModuleRules
{
	public SubmitTool(ReadOnlyTargetRules Target) : base(Target)
	{
		NumIncludedBytesPerUnityCPPOverride = 64 * 1024;
		bRequiresPlatformSDK = true;

		PublicIncludePathModuleNames.AddRange(
			new string[] {
				"Launch",
				"TargetPlatform"
			}
		);

		PrivateIncludePaths.Add(Path.Combine(EngineDirectory, "Source/Developer/OutputLog/Private"));

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"SubmitToolCore",
				"AppFramework",
				"Core",
				"CoreUObject",
				"InputCore",
				"ApplicationCore",
				"Projects",
				//"RenderCore", // Stripping both of these for now, this was only used by ShaderValidator but it added 20 mb to the packaged size for a validator that's not in use
				//"ShaderCompilerCommon",
				"Slate",
				"SlateCore",
				"StandaloneRenderer",
				"StudioTelemetry",
				"ToolWidgets",
				"DesktopPlatform", // Open file dialog
				"Settings",
				"OutputLog",
				"HTTP",
				"HTTPServer",
				"Json",
				"JsonUtilities",
				"Analytics",
				"AnalyticsET",
				"PakFile",
				"Virtualization"
			}
		);


		if (Target.Configuration != UnrealTargetConfiguration.Shipping)
		{
			PrivateIncludePathModuleNames.AddRange(
				new string[] {
				"SlateReflector",
				}
			);

			DynamicallyLoadedModuleNames.AddRange(
				new string[] {
				"SlateReflector",
				}
			);

			// Test dependencies
			PublicDependencyModuleNames.Add("RPCBase");
		}

		// Include the restricted module if it exists
		string RestrictedModulePath = Path.Combine(EngineDirectory,
			"Restricted", "NotForLicensees", "Source", "Programs", "SubmitToolRestricted", "SubmitToolRestricted.Build.cs");
		if (File.Exists(RestrictedModulePath))
		{
			PrivateDependencyModuleNames.Add("SubmitToolRestricted");
		}

		AddEngineThirdPartyPrivateStaticDependencies(Target, "Perforce");

		if (Target.IsInPlatformGroup(UnrealPlatformGroup.Linux))
		{
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"UnixCommonStartup"
				}
			);
		}
		
		RuntimeDependencies.Add("$(EngineDir)/Content/Editor/Slate/Common/...*.png", StagedFileType.UFS);
		RuntimeDependencies.Add("$(EngineDir)/Content/Editor/Slate/Docking/...*.png", StagedFileType.UFS);
		RuntimeDependencies.Add("$(EngineDir)/Content/Editor/Slate/Icons/Cross_12x.png", StagedFileType.UFS);
		RuntimeDependencies.Add("$(EngineDir)/Content/Editor/Slate/Icons/cursor_cardinal_cross.png", StagedFileType.UFS);
		RuntimeDependencies.Add("$(EngineDir)/Content/Editor/Slate/Icons/cursor_grab.png", StagedFileType.UFS);
		RuntimeDependencies.Add("$(EngineDir)/Content/Editor/Slate/Icons/denied_16x.png", StagedFileType.UFS);
		RuntimeDependencies.Add("$(EngineDir)/Content/Editor/Slate/Icons/Edit/icon_Edit_Duplicate_16x.png", StagedFileType.UFS);
		RuntimeDependencies.Add("$(EngineDir)/Content/Editor/Slate/Icons/EditorAppIcon.png", StagedFileType.UFS);
		RuntimeDependencies.Add("$(EngineDir)/Content/Editor/Slate/Icons/ellipsis_12x.png", StagedFileType.UFS);
		RuntimeDependencies.Add("$(EngineDir)/Content/Editor/Slate/Icons/Empty_14x.png", StagedFileType.UFS);
		RuntimeDependencies.Add("$(EngineDir)/Content/Editor/Slate/Icons/eyedropper_16px.png", StagedFileType.UFS);
		RuntimeDependencies.Add("$(EngineDir)/Content/Editor/Slate/Icons/Help/icon_Help_Documentation_16x.png", StagedFileType.UFS);
		RuntimeDependencies.Add("$(EngineDir)/Content/Editor/Slate/Icons/Help/icon_Help_support_16x.png", StagedFileType.UFS);
		RuntimeDependencies.Add("$(EngineDir)/Content/Editor/Slate/Icons/icon_Downloads_16x.png", StagedFileType.UFS);
		RuntimeDependencies.Add("$(EngineDir)/Content/Editor/Slate/Icons/icon_error_16x.png", StagedFileType.UFS);
		RuntimeDependencies.Add("$(EngineDir)/Content/Editor/Slate/Icons/icon_help_16x.png", StagedFileType.UFS);
		RuntimeDependencies.Add("$(EngineDir)/Content/Editor/Slate/Icons/icon_info_16x.png", StagedFileType.UFS);
		RuntimeDependencies.Add("$(EngineDir)/Content/Editor/Slate/Icons/icon_redo_16px.png", StagedFileType.UFS);
		RuntimeDependencies.Add("$(EngineDir)/Content/Editor/Slate/Icons/icon_tab_toolbar_16px.png", StagedFileType.UFS);
		RuntimeDependencies.Add("$(EngineDir)/Content/Editor/Slate/Icons/icon_undo_16px.png", StagedFileType.UFS);
		RuntimeDependencies.Add("$(EngineDir)/Content/Editor/Slate/Icons/icon_warning_16x.png", StagedFileType.UFS);
		RuntimeDependencies.Add("$(EngineDir)/Content/Editor/Slate/Icons/PlusSymbol_12x.png", StagedFileType.UFS);
		RuntimeDependencies.Add("$(EngineDir)/Content/Editor/Slate/Icons/refresh_12x.png", StagedFileType.UFS);
		RuntimeDependencies.Add("$(EngineDir)/Content/Editor/Slate/Icons/Star_16x.png", StagedFileType.UFS);
		RuntimeDependencies.Add("$(EngineDir)/Content/Editor/Slate/Icons/toolbar_expand_16x.png", StagedFileType.UFS);
		RuntimeDependencies.Add("$(EngineDir)/Content/Editor/Slate/Old/HyperlinkDotted.png", StagedFileType.UFS);
		RuntimeDependencies.Add("$(EngineDir)/Content/Editor/Slate/Old/HyperlinkUnderline.png", StagedFileType.UFS);
		RuntimeDependencies.Add("$(EngineDir)/Content/Editor/Slate/Old/White.png", StagedFileType.UFS);
		RuntimeDependencies.Add("$(EngineDir)/Content/Editor/Slate/Old/Menu_Background.png", StagedFileType.UFS); 
		RuntimeDependencies.Add("$(EngineDir)/Content/Editor/Slate/Old/DashedBorder.png", StagedFileType.UFS); 
		RuntimeDependencies.Add("$(EngineDir)/Content/Editor/Slate/Old/ToolTip_Background.png", StagedFileType.UFS);
		RuntimeDependencies.Add("$(EngineDir)/Content/Editor/Slate/Checkerboard.png", StagedFileType.UFS);
		RuntimeDependencies.Add("$(EngineDir)/Content/Slate/Starship/Common/AutomationTools.svg", StagedFileType.UFS);
		RuntimeDependencies.Add("$(EngineDir)/Content/Slate/Starship/SourceControl/SCC_CheckedOut.svg", StagedFileType.UFS);
		RuntimeDependencies.Add("$(EngineDir)/Content/Slate/Starship/SourceControl/RC_MarkedForAdd.svg", StagedFileType.UFS);
		RuntimeDependencies.Add("$(EngineDir)/Content/Slate/Starship/SourceControl/SCC_MarkedForDelete.svg", StagedFileType.UFS);
		RuntimeDependencies.Add("$(EngineDir)/Content/Slate/Starship/SourceControl/SCC_Branched.svg", StagedFileType.UFS);
	}
}
