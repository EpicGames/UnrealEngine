// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedPlatforms(UnrealPlatformClass.Desktop)]
public class SubmitToolTarget : TargetRules
{
	public SubmitToolTarget(TargetInfo Target) : base(Target)
	{
		Type = TargetType.Program;
		LinkType = TargetLinkType.Monolithic;
		IncludeOrderVersion = EngineIncludeOrderVersion.Latest;
		LaunchModuleName = "SubmitTool";


		bUseXGEController				= false;
		bLoggingToMemoryEnabled			= true;
		bUseLoggingInShipping			= true;
		bCompileWithAccessibilitySupport= false;
		bWithServerCode					= false;
		bCompileNavmeshClusterLinks		= false;
		bCompileNavmeshSegmentLinks		= false;
		bCompileRecast					= false;
		bCompileICU 					= true;
		bWithLiveCoding					= false;
		bBuildDeveloperTools			= false;
		bBuildWithEditorOnlyData		= false;
		bCompileAgainstEngine			= false;
		bCompileAgainstCoreUObject		= true;
		bUsesSlate						= true;
		bIsBuildingConsoleApplication	= false;
		bCompileWithPluginSupport		= true;
		bBuildRequiresCookedData		= false;
		bEnableTrace					= true;
		bForceDisableAutomationTests	= true;

		bHasExports = false;

		WindowsPlatform.bStripUnreferencedSymbols = true;
		WindowsPlatform.bMergeIdenticalCOMDATs = true;
		WindowsPlatform.bUseBundledDbgHelp = false;
		WindowsPlatform.bPixProfilingEnabled = false;

		GlobalDefinitions.Add("UE_TASK_TRACE_ENABLED=1");
		GlobalDefinitions.Add("AUTOSDKS_ENABLED=0");
		GlobalDefinitions.Add("EXCLUDE_NONPAK_UE_EXTENSIONS=0");
		GlobalDefinitions.Add("WITH_AUTOMATION_WORKER=0");
		GlobalDefinitions.Add("UE_EXTERNAL_PROFILING_ENABLED=0");
		GlobalDefinitions.Add("STATS=0");
		GlobalDefinitions.Add("ENABLE_STATNAMEDEVENTS=0");
		GlobalDefinitions.Add("ALLOW_HITCH_DETECTION=0");
		GlobalDefinitions.Add("UE_USE_MALLOC_FILL_BYTES=0");

		OptedInModulePlatforms = new UnrealTargetPlatform[] { UnrealTargetPlatform.Win64, UnrealTargetPlatform.Mac,
															  UnrealTargetPlatform.Linux, UnrealTargetPlatform.LinuxArm64 };
	}
}
