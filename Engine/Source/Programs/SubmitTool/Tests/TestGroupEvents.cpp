// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_LOW_LEVEL_TESTS

#include "TestCommon/Initialization.h"

#include <catch2/catch_test_macros.hpp>

GROUP_BEFORE_GLOBAL(Catch::DefaultGroup)
{
	InitAll(true, true);
}

GROUP_AFTER_GLOBAL(Catch::DefaultGroup)
{
	CleanupAll();
}

#endif